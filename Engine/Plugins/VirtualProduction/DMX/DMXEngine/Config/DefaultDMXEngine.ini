; UE4.26 to UE4.27
[CoreRedirects]
+EnumRedirects=(OldName="EDMXCommunicationTypes", NewName="EDMXCommunicationType") ; Only used for DMX Libraries that used deprecated UDMXEntitiyController, to upgrade them to 4.27

; UE5.4 to UE5.5
+FunctionRedirects=(OldName="DMXSubsystem.GetAllDMXLibraries",NewName="DMXSubsystem.LoadDMXLibrariesSynchronous")
+StructRedirects=(OldName="DMXColorCIE1931xyY", NewName="/Script/DMXGDTF.DMXGDTFColorCIE1931xyY")
+FunctionRedirects=(OldName="DMXEntityFixturePatch.GetAttributesValues",NewName="DMXEntityFixturePatch.GetAttributeValues")
+PropertyRedirects=(OldName="DMXEntityFixturePatch.GetAttributeValues.AttributesValues", NewName="GetAttributeValues.AttributeValues")
+FunctionRedirects=(OldName="DMXEntityFixturePatch.GetNormalizedAttributesValues",NewName="DMXEntityFixturePatch.GetNormalizedAttributeValues")
+PropertyRedirects=(OldName="DMXEntityFixturePatch.GetNormalizedAttributeValues.AttributesValues", NewName="GetNormalizedAttributeValues.AttributeValues")
