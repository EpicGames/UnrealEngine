{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "DMX Engine",
	"Description": "Functionality and assets for communication with DigitalMultiplexer (DMX) enabled devices",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Modules": [
		{
			"Name": "DMXBlueprintGraph",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DMXEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DMXRuntime",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "DatasmithContent",
			"Enabled": true
		},
		{
			"Name": "DMXGDTF",
			"Enabled": true
		},
		{
			"Name": "DMXModularFeatures",
			"Enabled": true
		},
		{
			"Name": "DMXProtocol",
			"Enabled": true
		},
		{
			"Name": "Takes",
			"Enabled": true
		}
	]
}
