// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DMXBlueprintGraph : ModuleRules
{
	public DMXBlueprintGraph(ReadOnlyTargetRules Target) : base(Target)
    {
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
                "DMXProtocol",
				"DMXRuntime",
            }
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
                "ApplicationCore",
                "AssetRegistry",
                "AssetTools",
                "CoreUObject",
				"Kismet",
                "KismetCompiler",
				"EditorFramework",
                "UnrealEd",
                
                "PropertyEditor",
                
                "KismetWidgets",
                "Engine",
                "Slate",
                "SlateCore",
                "InputCore",
                "Json",
                "Projects",
                "BlueprintGraph",
                "GraphEditor",
                "DMXEditor",
            }
		);
	}
}