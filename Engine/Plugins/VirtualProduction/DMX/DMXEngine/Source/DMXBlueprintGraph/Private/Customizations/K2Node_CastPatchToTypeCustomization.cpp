// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_CastPatchToTypeCustomization.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DetailCategoryBuilder.h"
#include "IDetailsView.h"
#include "K2Node_CastPatchToType.h"

#define LOCTEXT_NAMESPACE "K2Node_CastPatchToTypeCustomization"

TSharedRef<IDetailCustomization> K2Node_CastPatchToTypeCustomization::MakeInstance()
{
	return MakeShared<K2Node_CastPatchToTypeCustomization>();
}

void K2Node_CastPatchToTypeCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailLayout)
{

	DetailLayout = &InDetailLayout;
	UDEPRECATED_K2Node_CastPatchToType* Node = GetK2Node_CastPatchToType();

	static const FName FixtureSettingsCategoryName = TEXT("Fixture Settings");
	InDetailLayout.EditCategory(FixtureSettingsCategoryName, FText::GetEmpty(), ECategoryPriority::Important);
	IDetailCategoryBuilder& FunctionActionsCategory = InDetailLayout.EditCategory("DMXFunctionActions",
																				  LOCTEXT("FunctionActionCategoryName", "Function Actions"),
																				  ECategoryPriority::Important);

	FunctionActionsCategory.AddCustomRow(FText::GetEmpty())
		.WholeRowContent()
		.HAlign(HAlign_Left)
		[
			SNew( SBox )
			.MaxDesiredWidth(300.f)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(2.0f)
				+ SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.OnClicked(this, &K2Node_CastPatchToTypeCustomization::ExposeAttributesClicked)
					.ToolTipText(LOCTEXT("ExposeAttributesButtonTooltip", "Expose Attributes to Node Pins"))
					.IsEnabled_Lambda([Node]() -> bool { return Node && !Node->IsExposed(); })
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ExposeAttributesButton", "Expose Attributes"))
					]
				]
				+ SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.OnClicked(this, &K2Node_CastPatchToTypeCustomization::ResetAttributesClicked)
					.ToolTipText(LOCTEXT("ResetAttributesButtonTooltip", "Resets Attributes from Node Pins."))
					.IsEnabled_Lambda([Node]() -> bool { return Node && Node->IsExposed(); })
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ResetEmitterButton", "Reset Attributes"))
					]
				]
			]
		];
}

FReply K2Node_CastPatchToTypeCustomization::ExposeAttributesClicked()
{
	const TArray< TWeakObjectPtr<UObject> >& SelectedObjects = DetailLayout->GetSelectedObjects();

	for (int32 SelectedIndex = 0; SelectedIndex < SelectedObjects.Num(); ++SelectedIndex)
	{
		if (SelectedObjects[SelectedIndex].IsValid())
		{
			if (UDEPRECATED_K2Node_CastPatchToType* Node = Cast<UDEPRECATED_K2Node_CastPatchToType>(SelectedObjects[SelectedIndex].Get()))
			{
				Node->ExposeAttributes();
			}
		}
	}

	return FReply::Handled();
}

FReply K2Node_CastPatchToTypeCustomization::ResetAttributesClicked()
{
	if (UDEPRECATED_K2Node_CastPatchToType* Node = GetK2Node_CastPatchToType())
	{
		Node->ResetAttributes();
	}

	return FReply::Handled();
}

UDEPRECATED_K2Node_CastPatchToType* K2Node_CastPatchToTypeCustomization::GetK2Node_CastPatchToType() const
{
	const TArray< TWeakObjectPtr<UObject> >& SelectedObjects = DetailLayout->GetSelectedObjects();

	for (int32 SelectedIndex = 0; SelectedIndex < SelectedObjects.Num(); ++SelectedIndex)
	{
		if (SelectedObjects[SelectedIndex].IsValid())
		{
			return Cast<UDEPRECATED_K2Node_CastPatchToType>(SelectedObjects[SelectedIndex].Get());
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
