// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "IDetailCustomization.h"

class IDetailLayoutBuilder;
class UDEPRECATED_K2Node_CastPatchToType;

class K2Node_CastPatchToTypeCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

private:
	//~ Begin IDetailCustomization Interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailLayout) override;
	//~ Begin IDetailCustomization Interface

	FReply ExposeAttributesClicked();

	FReply ResetAttributesClicked();

	UDEPRECATED_K2Node_CastPatchToType* GetK2Node_CastPatchToType() const;

private:

	/** Cached off reference to the layout builder */
	IDetailLayoutBuilder* DetailLayout;
};
