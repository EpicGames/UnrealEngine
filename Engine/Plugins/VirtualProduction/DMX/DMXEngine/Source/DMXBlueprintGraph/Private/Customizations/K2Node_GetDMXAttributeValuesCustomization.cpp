// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/K2Node_GetDMXAttributeValuesCustomization.h"
#include "K2Node_GetDMXAttributeValues.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DetailCategoryBuilder.h"
#include "IDetailsView.h"

#define LOCTEXT_NAMESPACE "K2Node_GetDMXAttributeValuesCustomization"

TSharedRef<IDetailCustomization> FK2Node_GetDMXAttributeValuesCustomization::MakeInstance()
{
	return MakeShared<FK2Node_GetDMXAttributeValuesCustomization>();
}

void FK2Node_GetDMXAttributeValuesCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailLayout)
{
	this->DetailLayout = &InDetailLayout;
	UK2Node_GetDMXAttributeValues* Node = GetK2Node_DMXAttributeValues();

	static const FName FixtureSettingsCategoryName = TEXT("Fixture Settings");
	InDetailLayout.EditCategory(FixtureSettingsCategoryName, FText::GetEmpty(), ECategoryPriority::Important);
	IDetailCategoryBuilder& FunctionActionsCategory = InDetailLayout.EditCategory("DMXFunctionActions", LOCTEXT("FunctionActionCategoryName", "Function Actions"), ECategoryPriority::Important);

	FunctionActionsCategory.AddCustomRow(FText::GetEmpty())
		.WholeRowContent()
		.HAlign(HAlign_Left)
		[
			SNew( SBox )
			.MaxDesiredWidth(300.f)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(2.0f)
				+ SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.OnClicked(this, &FK2Node_GetDMXAttributeValuesCustomization::ExposeAttributesClicked)
					.ToolTipText(LOCTEXT("ExposeAttributesButtonTooltip", "Expose Attributes to Node Pins"))
					.IsEnabled_Lambda([Node]() -> bool { return Node && !Node->IsExposed(); })
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ExposeAttributesButton", "Expose Attributes"))
					]
				]
				+ SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.OnClicked(this, &FK2Node_GetDMXAttributeValuesCustomization::ResetAttributesClicked)
					.ToolTipText(LOCTEXT("ResetAttributesButtonTooltip", "Resets Attributes from Node Pins."))
					.IsEnabled_Lambda([Node]() -> bool { return Node && Node->IsExposed(); })
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ResetEmitterButton", "Reset Attributes"))
					]
				]
			]
		];
}

FReply FK2Node_GetDMXAttributeValuesCustomization::ExposeAttributesClicked()
{
	const TArray< TWeakObjectPtr<UObject> >& SelectedObjects = DetailLayout->GetSelectedObjects();

	for (int32 Idx = 0; Idx < SelectedObjects.Num(); ++Idx)
	{
		if (SelectedObjects[Idx].IsValid())
		{
			if (UK2Node_GetDMXAttributeValues* K2Node_GetDMXAttributeValues = Cast<UK2Node_GetDMXAttributeValues>(SelectedObjects[Idx].Get()))
			{
				K2Node_GetDMXAttributeValues->ExposeAttributes();
			}
		}
	}

	return FReply::Handled();
}

FReply FK2Node_GetDMXAttributeValuesCustomization::ResetAttributesClicked()
{
	if (UK2Node_GetDMXAttributeValues* K2Node_GetDMXAttributeValues = GetK2Node_DMXAttributeValues())
	{
		K2Node_GetDMXAttributeValues->ResetAttributes();
	}

	return FReply::Handled();
}

UK2Node_GetDMXAttributeValues* FK2Node_GetDMXAttributeValuesCustomization::GetK2Node_DMXAttributeValues() const
{
	const TArray< TWeakObjectPtr<UObject> >& SelectedObjects = DetailLayout->GetSelectedObjects();

	for (int32 Idx = 0; Idx < SelectedObjects.Num(); ++Idx)
	{
		if (SelectedObjects[Idx].IsValid())
		{
			return Cast<UK2Node_GetDMXAttributeValues>(SelectedObjects[Idx].Get());
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
