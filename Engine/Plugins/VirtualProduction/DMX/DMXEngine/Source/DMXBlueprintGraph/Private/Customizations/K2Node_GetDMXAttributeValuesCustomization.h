// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "IDetailCustomization.h"

class IDetailLayoutBuilder;
class UK2Node_GetDMXAttributeValues;

class FK2Node_GetDMXAttributeValuesCustomization
	: public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();
private:
	//~ Begin IDetailCustomization Interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailLayout) override;
	//~ Begin IDetailCustomization Interface

	/** Handles the Expose button's on click event */
	FReply ExposeAttributesClicked();

	/** Handles the Reset button's on click event */
	FReply ResetAttributesClicked();

	/** Get the pointer to GetDMXAttributeValues node */
	UK2Node_GetDMXAttributeValues* GetK2Node_DMXAttributeValues() const;

private:

	/** Cached off reference to the layout builder */
	IDetailLayoutBuilder* DetailLayout;
};
