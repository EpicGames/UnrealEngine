// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

DECLARE_LOG_CATEGORY_EXTERN(LogDMXBlueprintGraph, Log, All);

#define UE_LOG_DMXBLUEPRINTGRAPH(Verbosity, Format, ...) \
{ \
	UE_LOG(LogDMXBlueprintGraph, Verbosity, Format, ##__VA_ARGS__); \
}

#define UE_CLOG_DMXBLUEPRINTGRAPH(Conditional, Verbosity, Format, ...) \
{ \
	UE_CLOG(Conditional, LogDMXBlueprintGraph, Verbosity, Format, ##__VA_ARGS__); \
}