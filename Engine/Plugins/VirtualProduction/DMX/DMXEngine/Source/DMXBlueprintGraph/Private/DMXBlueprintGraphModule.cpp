// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXBlueprintGraphModule.h"
#include "DMXGraphPanelPinFactory.h"
#include "K2Node_GetDMXAttributeValues.h"
#include "Customizations/K2Node_GetDMXAttributeValuesCustomization.h"
#include "DMXBlueprintGraphLog.h"
#include "Library/DMXEntityFixtureType.h"

#include "AssetToolsModule.h"
#include "PropertyEditorModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "UObject/UObjectIterator.h"
#include "Customizations/K2Node_CastPatchToTypeCustomization.h"
#include "K2Node_CastPatchToType.h"

#define LOCTEXT_NAMESPACE "DMXBlueprintGraphModule"

DEFINE_LOG_CATEGORY(LogDMXBlueprintGraph);

void FDMXBlueprintGraphModule::StartupModule()
{
	DMXGraphPanelPinFactory = MakeShared<FDMXGraphPanelPinFactory>();
	FEdGraphUtilities::RegisterVisualPinFactory(DMXGraphPanelPinFactory);

	RegisterObjectCustomizations();

	UDMXEntityFixtureType::GetOnFixtureTypeChanged().AddRaw(this, &FDMXBlueprintGraphModule::OnFixtureTypeChanged);
}

void FDMXBlueprintGraphModule::ShutdownModule()
{
	FEdGraphUtilities::UnregisterVisualPinFactory(DMXGraphPanelPinFactory);

	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		// Unregister all classes customized by name
		for (const FName& ClassName : RegisteredClassNames)
		{
			PropertyModule.UnregisterCustomClassLayout(ClassName);
		}

		PropertyModule.NotifyCustomizationModuleChanged();
	}

	UDMXEntityFixtureType::GetOnFixtureTypeChanged().RemoveAll(this);
}

void FDMXBlueprintGraphModule::RegisterObjectCustomizations()
{
	RegisterCustomClassLayout(UK2Node_GetDMXAttributeValues::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FK2Node_GetDMXAttributeValuesCustomization::MakeInstance));

	RegisterCustomClassLayout(UDEPRECATED_K2Node_CastPatchToType::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&K2Node_CastPatchToTypeCustomization::MakeInstance));

}

void FDMXBlueprintGraphModule::RegisterCustomClassLayout(FName ClassName, FOnGetDetailCustomizationInstance DetailLayoutDelegate)
{
	check(ClassName != NAME_None);

	RegisteredClassNames.Add(ClassName);

	static FName PropertyEditor("PropertyEditor");
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditor);
	PropertyModule.RegisterCustomClassLayout(ClassName, DetailLayoutDelegate);
}

void FDMXBlueprintGraphModule::OnFixtureTypeChanged(const UDMXEntityFixtureType* InFixtureType)
{
	for (TObjectIterator<UK2Node_GetDMXAttributeValues> It(RF_Transient | RF_ClassDefaultObject, /** bIncludeDerivedClasses */ true, /** InternalExcludeFlags */ EInternalObjectFlags::Garbage); It; ++It)
	{
		if (It->HasValidBlueprint())
		{
			It->OnFixtureTypeChanged(InFixtureType);
		}
	}
}


IMPLEMENT_MODULE(FDMXBlueprintGraphModule, DMXBlueprintGraph)

#undef LOCTEXT_NAMESPACE
