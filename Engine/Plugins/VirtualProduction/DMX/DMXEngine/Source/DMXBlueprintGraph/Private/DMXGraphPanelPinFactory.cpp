// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXGraphPanelPinFactory.h"

#include "DMXAttribute.h"
#include "Library/DMXLibrary.h"
#include "Library/DMXEntityReference.h"
#include "Library/DMXEntity.h"
#include "Library/DMXEntityFixtureType.h"
#include "Widgets/SDMXAttributeNameGraphPin.h"
#include "Widgets/SDMXFixtureCategoryGraphPin.h"
#include "Widgets/SDMXEntityReferenceGraphPin.h"
#include "Widgets/SNullGraphPin.h"

#include "EdGraphSchema_K2.h"
#include "EdGraph/EdGraphPin.h"
#include "UObject/Class.h"


TSharedPtr<class SGraphPin> FDMXGraphPanelPinFactory::CreatePin(class UEdGraphPin* InPin) const
{
	if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
	{
		if (UScriptStruct* PinStructType = Cast<UScriptStruct>(InPin->PinType.PinSubCategoryObject.Get()))
		{
			if (PinStructType->IsChildOf(FDMXFixtureCategory::StaticStruct()))
			{
				return SNew(SDMXFixtureCategoryGraphPin, InPin);
			}
			else if (PinStructType->IsChildOf(FDMXAttributeName::StaticStruct()))
			{
				return SNew(SDMXAttributeNameGraphPin, InPin);
			}
			else if (PinStructType->IsChildOf(FDMXEntityReference::StaticStruct()))
			{
				return SNew(SDMXEntityReferenceGraphPin, InPin);
			}
		}
	}
	else if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object)
	{
		if (UClass* PinObjectType = Cast<UClass>(InPin->PinType.PinSubCategoryObject.Get()))
		{
			if (PinObjectType->IsChildOf(UDMXEntity::StaticClass()))
			{
				return SNew(SNullGraphPin, InPin);
			}
		}
	}

	return FGraphPanelPinFactory::CreatePin(InPin);
}
