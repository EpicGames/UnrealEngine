// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_CastPatchToType.h"

#include "EdGraphSchema_K2.h"
#include "EdGraph/EdGraphPin.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "DMXProtocolConstants.h"
#include "UObject/Object.h"
#include "K2Node_CallFunction.h"
#include "KismetCompiler.h"
#include "UObject/Class.h"
#include "DMXProtocolTypes.h"
#include "DMXBlueprintGraphLog.h"
#include "Library/DMXEntityFixturePatch.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "UObject/FrameworkObjectVersion.h"
#include "DMXSubsystem.h"
#include "K2Node_IfThenElse.h"
#include "K2Node_EditablePinBase.h"
#include "Serialization/Archive.h"

#define LOCTEXT_NAMESPACE "UK2Node_DMXCastToFixtureType"

const FName UDEPRECATED_K2Node_CastPatchToType::InputPinName_FixturePatch(TEXT("Input_FixturePatch"));
const FName UDEPRECATED_K2Node_CastPatchToType::InputPinName_FixtureTypeRef(TEXT("Input_FixtureTypeRef"));

const FName UDEPRECATED_K2Node_CastPatchToType::OutputPinName_AttributesMap(TEXT("Output_FixtureTypeAttributesMap"));

UDEPRECATED_K2Node_CastPatchToType::UDEPRECATED_K2Node_CastPatchToType()
{
	bIsEditable = true;
	bIsExposed = false;
}

/** 
 * Since we are overriding serialization (read comment below in ::Serialize(), and we are trying to serialize
 * FUserPinInfo data (for the user defined pins we are creating in this node), it'll not compile if we don't
 * define this operator that "explains" how to serialize the FUserPinInfos
 */ 
FArchive& operator<<(FArchive& Ar, FUserPinInfo& Info)
{
	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);

	if (Ar.CustomVer(FFrameworkObjectVersion::GUID) >= FFrameworkObjectVersion::PinsStoreFName)
	{
		Ar << Info.PinName;
	}
	else
	{
		FString PinNameStr;
		Ar << PinNameStr;
		Info.PinName = *PinNameStr;
	}

	if (Ar.UEVer() >= VER_UE4_SERIALIZE_PINTYPE_CONST)
	{
		Info.PinType.Serialize(Ar);
		Ar << Info.DesiredPinDirection;
	}
	else
	{
		check(Ar.IsLoading());

		bool bIsArray = (Info.PinType.ContainerType == EPinContainerType::Array);
		Ar << bIsArray;

		bool bIsReference = Info.PinType.bIsReference;
		Ar << bIsReference;

		Info.PinType.ContainerType = (bIsArray ? EPinContainerType::Array : EPinContainerType::None);
		Info.PinType.bIsReference = bIsReference;

		FString PinCategoryStr;
		FString PinSubCategoryStr;
		
		Ar << PinCategoryStr;
		Ar << PinSubCategoryStr;

		Info.PinType.PinCategory = *PinCategoryStr;
		Info.PinType.PinSubCategory = *PinSubCategoryStr;

		Ar << Info.PinType.PinSubCategoryObject;
	}

	Ar << Info.PinDefaultValue;

	return Ar;
}

/**
 * What we need here is to serialize this node in both the parent and grand parent ways, because one will serialize the 
 * user defined pins (since this extends from UK2Node_EditablePinBase), but by doing only the parent serialization, 
 * it'll skip the serialization of the structs we have as IN pins (UE bug?). However, the grandparent class serializes it 
 * correctly. This method basically merges both serializations so it works as expected
 */
void UDEPRECATED_K2Node_CastPatchToType::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);

	// Check if it not SavingPackage
	if (Ar.IsSaving() && !UE::IsSavingPackage())
	{
		if (Ar.IsObjectReferenceCollector() || Ar.Tell() < 0)
		{
			// When this is a reference collector/modifier, serialize some pins as structs
			FixupPinStringDataReferences(&Ar);
		}
	}

	// Do not call parent, but call grandparent
	UEdGraphNode::Serialize(Ar);

	if (Ar.IsLoading() && ((Ar.GetPortFlags() & PPF_Duplicate) == 0))
	{
		// Fix up pin default values, must be done before post load
		FixupPinDefaultValues();

		if (GIsEditor)
		{
			// We need to serialize string data references on load in editor builds so the cooker knows about them
			FixupPinStringDataReferences(nullptr);
		}
	}

	// Pins Serialization 
	TArray<FUserPinInfo> SerializedItems;

	if (Ar.IsLoading())
	{
		Ar << SerializedItems;

		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

		UserDefinedPins.Empty(SerializedItems.Num());

		for (int32 Index = 0; Index < SerializedItems.Num(); ++Index)
		{
			TSharedPtr<FUserPinInfo> PinInfo = MakeShareable(new FUserPinInfo(SerializedItems[Index]));
			
			const bool bValidateConstRefPinTypes = Ar.CustomVer(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::EditableEventsUseConstRefParameters
				&& ShouldUseConstRefParams();
			{
				if (UEdGraphPin* NodePin = FindPin(PinInfo->PinName))
				{
					{
						// NOTE: the second FindPin call here to keep us from altering a pin with the same 
						//       name but different direction (in case there is two)
						if (PinInfo->DesiredPinDirection != NodePin->Direction && FindPin(PinInfo->PinName, PinInfo->DesiredPinDirection) == nullptr)
						{
							PinInfo->DesiredPinDirection = NodePin->Direction;
						}
					}

					if (bValidateConstRefPinTypes)
					{
						// Note that we should only get here if ShouldUseConstRefParams() indicated this node represents an event function with no outputs (above).
						if (!NodePin->PinType.bIsConst
							&& NodePin->Direction == EGPD_Output
							&& !K2Schema->IsExecPin(*NodePin)
							&& !K2Schema->IsDelegateCategory(NodePin->PinType.PinCategory))
						{
							// Add 'const' to either an array pin type (always passed by reference) or a pin type that's explicitly flagged to be passed by reference.
							NodePin->PinType.bIsConst = NodePin->PinType.IsArray() || NodePin->PinType.bIsReference;

							// Also mirror the flag into the UserDefinedPins array.
							PinInfo->PinType.bIsConst = NodePin->PinType.bIsConst;
						}
					}
				}
			}

			UserDefinedPins.Add(PinInfo);
		}
	}
	else if (Ar.IsSaving())
	{
		SerializedItems.Empty(UserDefinedPins.Num());

		for (int32 PinsIndex = 0; PinsIndex < UserDefinedPins.Num(); ++PinsIndex)
		{
			SerializedItems.Add(*(UserDefinedPins[PinsIndex].Get()));
		}

		Ar << SerializedItems;
	}
	else
	{
		// We want to avoid destroying and recreating FUserPinInfo, because that will invalidate 
		// any WeakPtrs to those entries:
		for (TSharedPtr<FUserPinInfo>& PinInfo : UserDefinedPins)
		{
			Ar << *PinInfo;
		}
	}
}


void UDEPRECATED_K2Node_CastPatchToType::AllocateDefaultPins()
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, TEXT("Success"));
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, TEXT("Failure"));

	// Input pins
	UEdGraphPin* InPin_FixturePatch = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, UDMXEntityFixturePatch::StaticClass(), InputPinName_FixturePatch);
	K2Schema->ConstructBasicPinTooltip(*InPin_FixturePatch, LOCTEXT("InputDMXFixturePatchPin", "Get the fixture patch reference."), InPin_FixturePatch->PinToolTip);

	UEdGraphPin* InPin_FixtureTypeRef = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Struct, FDMXEntityFixtureTypeRef::StaticStruct(), InputPinName_FixtureTypeRef);
	K2Schema->ConstructBasicPinTooltip(*InPin_FixtureTypeRef, LOCTEXT("InputDMXFixtureTypePin", "Get the fixture Type reference."), InPin_FixtureTypeRef->PinToolTip);
	InPin_FixtureTypeRef->bNotConnectable = true;

 	// Output pins
	FCreatePinParams PinParams_OutputAttributes;
	PinParams_OutputAttributes.ContainerType = EPinContainerType::Map;
	PinParams_OutputAttributes.ValueTerminalType.TerminalCategory = UEdGraphSchema_K2::PC_Int;

	UEdGraphPin* OutputPin_AttributesMapPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Struct, FDMXAttributeName::StaticStruct(), OutputPinName_AttributesMap, PinParams_OutputAttributes);
	K2Schema->ConstructBasicPinTooltip(*OutputPin_AttributesMapPin, LOCTEXT("OutputPin_FixtureTypeAttributesMap", "FixtureTypeAttributesMap"), OutputPin_AttributesMapPin->PinToolTip);

	Super::AllocateDefaultPins();
}

FText UDEPRECATED_K2Node_CastPatchToType::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("TooltipText", "Cast Fixture Patch to Fixture Type");
}

void UDEPRECATED_K2Node_CastPatchToType::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);	

	UEdGraphPin* MeIn_FixturePatch = FindPin(InputPinName_FixturePatch);
	if (MeIn_FixturePatch == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("MissingFixturePatchPin", "FixturePatch: Pin doesn't exists. @@").ToString(), this);
		return;
	}

	UEdGraphPin* MeIn_FixtureTypeRef = FindPin(InputPinName_FixtureTypeRef);
	if (MeIn_FixtureTypeRef == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("MissingFixtureTypePin", "FixtureType: Pin doesn't exists. @@").ToString(), this);
		return;
	}

	UEdGraphPin* MeIn_Exec = GetExecPin();
	
	UEdGraphPin* MeOut_ThenSuccess = FindPin(TEXT("Success"));
	if (MeOut_ThenSuccess == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("MissingSuccessPin", "Success: Pin doesn't exists. @@").ToString(), this);
		return;
	}

	UEdGraphPin* MeOut_ThenFailure = FindPin(TEXT("Failure"));
	if (MeOut_ThenFailure == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("MissingFailurePin", "Failure: Pin doesn't exists. @@").ToString(), this);
		return;
	}

	UEdGraphPin* MeOut_AttributesMap = FindPin(OutputPinName_AttributesMap);
	if (MeOut_AttributesMap == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("AttributesMapPin", "Failure: Pin doesn't exists. @@").ToString(), this);
		return;
	}
	
	const UEdGraphSchema_K2* K2Schema = CompilerContext.GetSchema();

	// This will be moved to MeOut_ThenSuccess
	UEdGraphPin* LastThenPin = nullptr;
 	

 	// NODE 1.  GetDMXSubsystem
	FName FunName_GetDMXSubsystem = GET_FUNCTION_NAME_CHECKED(UDMXSubsystem, GetDMXSubsystem_Callable);
	UK2Node_CallFunction* DMXSubsystem_Node = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	DMXSubsystem_Node->FunctionReference.SetExternalMember(FunName_GetDMXSubsystem, UDMXSubsystem::StaticClass());
	DMXSubsystem_Node->AllocateDefaultPins();

	// Move Parent Exec to GetDMXSubsystem's Exec
	CompilerContext.MovePinLinksToIntermediate(*MeIn_Exec, *DMXSubsystem_Node->GetExecPin());

	UEdGraphPin* DMXSubsytem_ReturnValue = DMXSubsystem_Node->GetReturnValuePin();
	LastThenPin = DMXSubsystem_Node->GetThenPin();

 	// NODE 2. UDMXSubsystem::PatchIsOfSelectedType 
	FName FunName_IsPatchOfType = GET_FUNCTION_NAME_CHECKED(UDMXSubsystem, PatchIsOfSelectedType);
	const UFunction* Fun_IsPatchOfType = FindUField<UFunction>(UDMXSubsystem::StaticClass(), FunName_IsPatchOfType);
	check(nullptr != Fun_IsPatchOfType);

	UK2Node_CallFunction * PatchIsOfType_Node = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	PatchIsOfType_Node->SetFromFunction(Fun_IsPatchOfType);
	PatchIsOfType_Node->AllocateDefaultPins();

	K2Schema->TryCreateConnection(LastThenPin, PatchIsOfType_Node->GetExecPin());
	K2Schema->TryCreateConnection(PatchIsOfType_Node->FindPin(UEdGraphSchema_K2::PN_Self), DMXSubsytem_ReturnValue);

	UEdGraphPin* PatchIsOfType_IN_FixturePatch = PatchIsOfType_Node->FindPin(TEXT("InFixturePatch"));
	if (PatchIsOfType_IN_FixturePatch == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("InFixturePatchPin", "InFixturePatch: Pin doesn't exists. @@").ToString(), this);
		return;
	}

	UEdGraphPin* PatchIsOfType_IN_FixtureTypeRef = PatchIsOfType_Node->FindPin(TEXT("RefTypeValue"));
	if (PatchIsOfType_IN_FixtureTypeRef == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("InFixturePatchPin", "InFixturePatch: Pin doesn't exists. @@").ToString(), this);
		return;
	}

	CompilerContext.CopyPinLinksToIntermediate(*MeIn_FixturePatch, *PatchIsOfType_IN_FixturePatch);

	UEdGraphPin* FixturePatchPin = FindPin(InputPinName_FixtureTypeRef);
	if (FixturePatchPin == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("InFixtureTypePin", "FixtureType: Pin doesn't exists. @@").ToString(), this);
		return;
	}
	
	K2Schema->TrySetDefaultValue(*PatchIsOfType_IN_FixtureTypeRef, FixturePatchPin->DefaultValue);
	if (PatchIsOfType_IN_FixturePatch == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("InFixturePatchPin", "InFixturePatch: Pin doesn't exists. @@").ToString(), this);
		return;
	}

	UEdGraphPin* PatchIsOfType_Out_Result = PatchIsOfType_Node->GetReturnValuePin();

	LastThenPin = PatchIsOfType_Node->GetThenPin();

 	// NODE 3: Branch (if cast success)
 	UK2Node_IfThenElse* Branch_Node = CompilerContext.SpawnIntermediateNode<UK2Node_IfThenElse>(this, SourceGraph);
	Branch_Node->AllocateDefaultPins();

 	UEdGraphPin* Branch_In_Condition = Branch_Node->FindPin(UEdGraphSchema_K2::PN_Condition);
	if (Branch_In_Condition == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("ConditionPin", "Condition: Pin doesn't exists. @@").ToString(), this);
		return;
	}

 	K2Schema->TryCreateConnection(Branch_In_Condition, PatchIsOfType_Out_Result);

 	K2Schema->TryCreateConnection(Branch_Node->GetExecPin(), LastThenPin);
 
 	CompilerContext.MovePinLinksToIntermediate(*MeOut_ThenFailure, *Branch_Node->GetElsePin());
 
 	LastThenPin = Branch_Node->GetThenPin();

	// NODE 4. UDMXSubsystem::GetAttributesMap
	static const FName FuncName_GetFunctionsMap = GET_FUNCTION_NAME_CHECKED(UDMXSubsystem, GetFunctionsMap);
	UFunction* FuncPtr_GetFunctionsMap = FindUField<UFunction>(UDMXSubsystem::StaticClass(), FuncName_GetFunctionsMap);
	check(FuncPtr_GetFunctionsMap);

	UK2Node_CallFunction* GetFunctionsMap_Node = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	GetFunctionsMap_Node->SetFromFunction(FuncPtr_GetFunctionsMap);
	GetFunctionsMap_Node->AllocateDefaultPins();

	UEdGraphPin* GetAttributesMap_In_Self = GetFunctionsMap_Node->FindPin(UEdGraphSchema_K2::PN_Self);
	if (GetAttributesMap_In_Self == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("SelfPin", "Self: Pin doesn't exists. @@").ToString(), this);
		return;
	}

	UEdGraphPin* GetAttributesMap_In_Exec = GetFunctionsMap_Node->GetExecPin();
	UEdGraphPin* GetAttributesMap_In_FixturePatch = GetFunctionsMap_Node->FindPin(TEXT("InFixturePatch"));
	if (GetAttributesMap_In_FixturePatch == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("InFixturePatchPin", "InFixturePatch: Pin doesn't exists. @@").ToString(), this);
		return;
	}

	UEdGraphPin* GetAttributesMap_InOut_AttributesMap = GetFunctionsMap_Node->FindPin(TEXT("OutAttributesMap"));
	if (GetAttributesMap_InOut_AttributesMap == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("OutAttributesMapPin", "OutAttributesMap: Pin doesn't exists. @@").ToString(), this);
		return;
	}

	UEdGraphPin* GetAttributesMap_Out_Then = GetFunctionsMap_Node->GetThenPin();

	// inputs
 	K2Schema->TryCreateConnection(GetAttributesMap_In_Self, DMXSubsytem_ReturnValue);
	CompilerContext.CopyPinLinksToIntermediate(*MeIn_FixturePatch, *GetAttributesMap_In_FixturePatch);
	K2Schema->TryCreateConnection(LastThenPin, GetAttributesMap_In_Exec);

	// outputs
	CompilerContext.MovePinLinksToIntermediate(*MeOut_AttributesMap, *GetAttributesMap_InOut_AttributesMap);
	LastThenPin = GetAttributesMap_Out_Then;

	if(UserDefinedPins.Num() > 0)
	{
		TArray<UEdGraphPin*> Map_OUT_Ints;
		TArray<UEdGraphPin*> Map_IN_Names;

		// Call Attributes for dmx function values
		for (const TSharedPtr<FUserPinInfo>& PinInfo : UserDefinedPins)
		{
			UEdGraphPin* Pin = FindPin(PinInfo->PinName);
			if (Pin != nullptr)
			{
				if (Pin->Direction == EGPD_Output)
				{
					Map_OUT_Ints.Add(Pin);
				}
				else if (Pin->Direction == EGPD_Input)
				{
					Map_IN_Names.Add(Pin);
				}
			}
			else
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("UserDefinedPin", "UserDefined: Pin doesn't exists. @@").ToString(), this);
				return;
			}
		}

		check(Map_OUT_Ints.Num() == Map_IN_Names.Num());

		for (int32 PairIndex = 0; PairIndex < Map_IN_Names.Num(); ++PairIndex)
		{		
			UEdGraphPin* IN_FuncName = Map_IN_Names[PairIndex];
			UEdGraphPin* OUT_FuncInt = Map_OUT_Ints[PairIndex];

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			const FName FuncName_GetAttributesValue = GET_FUNCTION_NAME_CHECKED(UDMXSubsystem, GetFunctionsValue);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS 

			UFunction* FuncPtr_GetAttributesValue = FindUField<UFunction>(UDMXSubsystem::StaticClass(), FuncName_GetAttributesValue);
			check(FuncPtr_GetAttributesValue);

			UK2Node_CallFunction* GetAttributesValue_Node = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
			GetAttributesValue_Node->SetFromFunction(FuncPtr_GetAttributesValue);
			GetAttributesValue_Node->AllocateDefaultPins();

			UEdGraphPin* GetAttributesValue_In_Self = GetAttributesValue_Node->FindPin(UEdGraphSchema_K2::PN_Self);
			if (GetAttributesValue_In_Self == nullptr)
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("SelfPin", "Self: Pin doesn't exists. @@").ToString(), this);
				return;
			}

			UEdGraphPin* GetAttributesValue_In_Exec = GetAttributesValue_Node->GetExecPin();
			UEdGraphPin* GetAttributesValue_In_FunctionAttribute = GetAttributesValue_Node->FindPin(TEXT("FunctionAttributeName"));
			if (GetAttributesValue_In_FunctionAttribute == nullptr)
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("FunctionAttributeNamePin", "FunctionAttributeName: Pin doesn't exists. @@").ToString(), this);
				return;
			}

			UEdGraphPin* GetAttributesValue_In_InAttributesMapPin = GetAttributesValue_Node->FindPin(TEXT("InAttributesMap"));
			if (GetAttributesValue_In_InAttributesMapPin == nullptr)
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("InAttributesMapPin", "InAttributesMap: Pin doesn't exists. @@").ToString(), this);
				return;
			}

			UEdGraphPin* GetAttributesValue_Out_ReturnValue = GetAttributesValue_Node->FindPin(UEdGraphSchema_K2::PN_ReturnValue);
			if (GetAttributesValue_Out_ReturnValue == nullptr)
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("ReturnPin", "Return: Pin doesn't exists. @@").ToString(), this);
				return;
			}

			UEdGraphPin* GetAttributesValue_Out_Then= GetAttributesValue_Node->GetThenPin();

			// Input
			K2Schema->TryCreateConnection(GetAttributesValue_In_Self, DMXSubsytem_ReturnValue);
			CompilerContext.MovePinLinksToIntermediate(*IN_FuncName, *GetAttributesValue_In_FunctionAttribute);			

			// Output			
			K2Schema->TryCreateConnection(GetAttributesValue_In_InAttributesMapPin, GetAttributesMap_InOut_AttributesMap);
			CompilerContext.MovePinLinksToIntermediate(*OUT_FuncInt, *GetAttributesValue_Out_ReturnValue);

			// Execution
			K2Schema->TryCreateConnection(LastThenPin, GetAttributesValue_In_Exec);
			LastThenPin = GetAttributesValue_Out_Then;
		}

		CompilerContext.MovePinLinksToIntermediate(*MeOut_ThenSuccess, *LastThenPin);
	}
	else
	{
		CompilerContext.MovePinLinksToIntermediate(*MeOut_ThenSuccess, *LastThenPin);
	}
}

void UDEPRECATED_K2Node_CastPatchToType::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	UClass* ActionKey = GetClass();
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UDEPRECATED_K2Node_CastPatchToType::GetMenuCategory() const
{
	return FText::FromString(DMX_K2_CATEGORY_NAME);
}

void UDEPRECATED_K2Node_CastPatchToType::NotifyPinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::NotifyPinConnectionListChanged(Pin);
}


UEdGraphPin* UDEPRECATED_K2Node_CastPatchToType::CreatePinFromUserDefinition(const TSharedPtr<FUserPinInfo> NewPinInfo)
{
	UEdGraphPin* NewPin = CreatePin(NewPinInfo->DesiredPinDirection, NewPinInfo->PinType, NewPinInfo->PinName);
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	Schema->SetPinAutogeneratedDefaultValue(NewPin, NewPinInfo->PinDefaultValue);

	if(NewPinInfo->DesiredPinDirection == EEdGraphPinDirection::EGPD_Input)
	{
		NewPin->bHidden = true;
	}

	return NewPin;
}

bool UDEPRECATED_K2Node_CastPatchToType::ModifyUserDefinedPinDefaultValue(TSharedPtr<FUserPinInfo> PinInfo, const FString& NewDefaultValue)
{
	if (Super::ModifyUserDefinedPinDefaultValue(PinInfo, NewDefaultValue))
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		K2Schema->HandleParameterDefaultValueChanged(this);

		return true;
	}
	return false;
}

void UDEPRECATED_K2Node_CastPatchToType::ExposeAttributes()
{
	ResetAttributes();

	if(bIsExposed && UserDefinedPins.Num())
	{
		return;
	}	

	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	if(const UDMXEntityFixtureType* SelectedFixtureType = GetSelectedFixtureType())
	{
		for(const FDMXFixtureMode& Mode : SelectedFixtureType->Modes)
		{
			for (const FDMXFixtureFunction& Function : Mode.Functions)
			{
				FDMXAttributeName AttributeName = Function.Attribute;

				if(AttributeName.Name.IsNone())
				{
					continue;
				}

				FString EnumString = StaticEnum<EDMXFixtureSignalFormat>()->GetDisplayNameTextByIndex((int64)Function.DataType).ToString();
				FString PinFunctionName = *FString::Printf(TEXT("%s_%s"), *Function.Attribute.Name.ToString(), *EnumString);

				{
					FEdGraphPinType PinType;
					PinType.PinCategory = UEdGraphSchema_K2::PC_Int;
					UEdGraphPin* Pin = CreateUserDefinedPin(*PinFunctionName, PinType, EGPD_Output);
				}
 				
				{

					FEdGraphPinType PinType;
					PinType.PinCategory = UEdGraphSchema_K2::PC_Name;
					UEdGraphPin* Pin = CreateUserDefinedPin(*(PinFunctionName + FString("_Input")), PinType, EGPD_Input);
					Schema->TrySetDefaultValue(*Pin, Function.Attribute.Name.ToString());					
 				}			

				UBlueprint* BP = GetBlueprint();
				if (!BP->bBeingCompiled)
				{
					FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(BP);
				}
			}
		}		

		Modify();
		bIsExposed = true;
	}
	else
	{
		ResetAttributes();
	}	
}

void UDEPRECATED_K2Node_CastPatchToType::ResetAttributes()
{
	if(bIsExposed)
	{
	    // removes all the pins
		while(UserDefinedPins.Num())
		{
			TSharedPtr<FUserPinInfo> Pin = UserDefinedPins[0];
			RemoveUserDefinedPin(Pin);
		}

		bDisableOrphanPinSaving = true;

		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		K2Schema->HandleParameterDefaultValueChanged(this);
	}

	bIsExposed = false;
}

UDMXEntityFixtureType* UDEPRECATED_K2Node_CastPatchToType::GetSelectedFixtureType()
{
	UEdGraphPin* InPin_FixtureTypeRef = FindPin(InputPinName_FixtureTypeRef);
	if (InPin_FixtureTypeRef == nullptr)
	{
		UE_LOG_DMXBLUEPRINTGRAPH(Error, TEXT("No FixtureTypePin found"));
		return nullptr;
	}

	if (InPin_FixtureTypeRef->DefaultValue.Len() && InPin_FixtureTypeRef->LinkedTo.Num() == 0)
	{
		FString StringValue = InPin_FixtureTypeRef->DefaultValue;

		FDMXEntityFixtureTypeRef FixtureTypeRef;

		FDMXEntityReference::StaticStruct()
			->ImportText(*StringValue, &FixtureTypeRef, nullptr, EPropertyPortFlags::PPF_None, GLog, FDMXEntityReference::StaticStruct()->GetName());

		if (FixtureTypeRef.DMXLibrary != nullptr)
		{
			return FixtureTypeRef.GetFixtureType();
		}
	}

	return nullptr;
}

FString UDEPRECATED_K2Node_CastPatchToType::GetFixturePatchValueAsString() const
{
	UEdGraphPin* FixturePatchPin = FindPin(InputPinName_FixtureTypeRef);

	FString PatchRefString;

	if (FixturePatchPin == nullptr)
	{
		UE_LOG_DMXBLUEPRINTGRAPH(Error, TEXT("No FixturePatchPin found"));
		return PatchRefString;
	}



	// Case with default object
	if (FixturePatchPin->LinkedTo.Num() == 0)
	{
		PatchRefString = FixturePatchPin->GetDefaultAsString();
	}
	// Case with linked object
	else
	{
		PatchRefString = FixturePatchPin->LinkedTo[0]->GetDefaultAsString();
	}

	return PatchRefString;
}

#undef LOCTEXT_NAMESPACE