// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_GetDMXAttributeValues.h"

#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXLibrary.h"
#include "DMXSubsystem.h"
#include "DMXProtocolConstants.h"
#include "DMXBlueprintGraphLog.h"
#include "K2Node_GetDMXFixturePatch.h"

#include "KismetCompiler.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "ScopedTransaction.h"
#include "EdGraphUtilities.h"
#include "K2Node_CallFunction.h"

#define LOCTEXT_NAMESPACE "K2Node_GetDMXAttributeValues"

const FName UK2Node_GetDMXAttributeValues::InputDMXFixturePatchPinName(TEXT("InFixturePatch"));
const FName UK2Node_GetDMXAttributeValues::OutputAttributesMapPinName(TEXT("OutAttributesMap"));
const FName UK2Node_GetDMXAttributeValues::OutputIsSuccessPinName(TEXT("IsSuccessful"));

UK2Node_GetDMXAttributeValues::UK2Node_GetDMXAttributeValues()
{
	bIsEditable = true;
	bIsExposed = false;
}

void UK2Node_GetDMXAttributeValues::OnFixturePatchChanged()
{
	// Reset fixture path nodes if we receive a notification
	if (Pins.Num() > 0)
	{
		ResetAttributes();
	}
}

void UK2Node_GetDMXAttributeValues::RemovePinsRecursive(UEdGraphPin* PinToRemove)
{
	for (int32 SubPinIndex = PinToRemove->SubPins.Num() - 1; SubPinIndex >= 0; --SubPinIndex)
	{
		RemovePinsRecursive(PinToRemove->SubPins[SubPinIndex]);
	}

	int32 PinRemovalIndex = INDEX_NONE;
	if (Pins.Find(PinToRemove, PinRemovalIndex))
	{
		Pins.RemoveAt(PinRemovalIndex);
		PinToRemove->MarkAsGarbage();
	}
}

void UK2Node_GetDMXAttributeValues::RemoveOutputPin(UEdGraphPin* Pin)
{
	checkSlow(Pins.Contains(Pin));

	FScopedTransaction Transaction(LOCTEXT("RemovePinTx", "RemovePin"));
	Modify();

	RemovePinsRecursive(Pin);
	PinConnectionListChanged(Pin);

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
}

FText UK2Node_GetDMXAttributeValues::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("NodeTitle", "Get DMX Attribute Values");
}

void UK2Node_GetDMXAttributeValues::AllocateDefaultPins()
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Add execution pins
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);

	// Add input pin
	UEdGraphPin* InputDMXFixturePatchPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, UDMXEntityFixturePatch::StaticClass(), InputDMXFixturePatchPinName);
	K2Schema->ConstructBasicPinTooltip(*InputDMXFixturePatchPin, LOCTEXT("InputDMXFixturePatch", "Input DMX Fixture Patch"), InputDMXFixturePatchPin->PinToolTip);
	
	// Add output pin
	FCreatePinParams OutputAttributesMapPinParams;
	OutputAttributesMapPinParams.ContainerType = EPinContainerType::Map;
	OutputAttributesMapPinParams.ValueTerminalType.TerminalCategory = UEdGraphSchema_K2::PC_Int;

	UEdGraphPin* OutputAttributesMapPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Struct, FDMXAttributeName::StaticStruct(), OutputAttributesMapPinName, OutputAttributesMapPinParams);
	K2Schema->ConstructBasicPinTooltip(*OutputAttributesMapPin, LOCTEXT("OutputAttributesMap", "Output Attribute Map."), OutputAttributesMapPin->PinToolTip);

	UEdGraphPin* OutputIsSuccessPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Boolean, TEXT(""), OutputIsSuccessPinName);
	K2Schema->ConstructBasicPinTooltip(*OutputIsSuccessPin, LOCTEXT("OutputIsSuccessPin", "Is Success"), OutputIsSuccessPin->PinToolTip);

	Super::AllocateDefaultPins();
}

void UK2Node_GetDMXAttributeValues::PostPasteNode()
{
	ResetAttributes();
}

void UK2Node_GetDMXAttributeValues::PinConnectionListChanged(UEdGraphPin* Pin)
{
	if (Pin == GetInputDMXFixturePatchPin())
	{
		ResetAttributes();

		// Ask to recompile the bplueprint
		if (UBlueprint* BP = GetBlueprint())
		{
			FBlueprintEditorUtils::MarkBlueprintAsModified(BP);
		}
	}
}

void UK2Node_GetDMXAttributeValues::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

	// First node to execute. GetDMXSubsystem
	FName GetDMXSubsystemFunctionName = GET_FUNCTION_NAME_CHECKED(UDMXSubsystem, GetDMXSubsystem_Callable);
	UK2Node_CallFunction* DMXSubsystemNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	DMXSubsystemNode->FunctionReference.SetExternalMember(GetDMXSubsystemFunctionName, UDMXSubsystem::StaticClass());
	DMXSubsystemNode->AllocateDefaultPins();

	UEdGraphPin* DMXSubsystemExecPin = DMXSubsystemNode->GetExecPin();
	UEdGraphPin* DMXSubsystemThenPin = DMXSubsystemNode->GetThenPin();
	UEdGraphPin* DMXSubsystemResult = DMXSubsystemNode->GetReturnValuePin();

	// Hook up inputs
	CompilerContext.MovePinLinksToIntermediate(*GetExecPin(), *DMXSubsystemExecPin);

	// Hook up outputs
	UEdGraphPin* LastThenPin = DMXSubsystemThenPin;
	Schema->TryCreateConnection(LastThenPin, DMXSubsystemThenPin);

	// Second node to execute. GetAttributesMap
	static const FName FunctionName = GET_FUNCTION_NAME_CHECKED(UDMXSubsystem, GetFunctionsMap);
	UFunction* GetAttributesMapPointer = FindUField<UFunction>(UDMXSubsystem::StaticClass(), FunctionName);
	check(GetAttributesMapPointer);

	UK2Node_CallFunction* GetAttributesMapNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	GetAttributesMapNode->FunctionReference.SetExternalMember(FunctionName, UDMXSubsystem::StaticClass());
	GetAttributesMapNode->AllocateDefaultPins();

	// Function pins
	UEdGraphPin* GetAttributesMapNodeSelfPin = GetAttributesMapNode->FindPin(UEdGraphSchema_K2::PN_Self);
	if (GetAttributesMapNodeSelfPin == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("MissingSelfPin", "Self: Pin doesn't exists. @@").ToString(), this);
		return;
	}

	UEdGraphPin* GetAttributesMapNodeExecPin = GetAttributesMapNode->GetExecPin();
	UEdGraphPin* GetAttributesMapNodeInFixturePatchPin = GetAttributesMapNode->FindPin(TEXT("InFixturePatch"));
	if (GetAttributesMapNodeInFixturePatchPin == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("MissingInFixturePatchPin", "InFixturePatch: Pin doesn't exists. @@").ToString(), this);
		return;
	}

	UEdGraphPin* GetAttributesMapNodeOutAttributesMapPin = GetAttributesMapNode->FindPin(TEXT("OutAttributesMap"));
	if (GetAttributesMapNodeOutAttributesMapPin == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("MissingOutAttributesMapPin", "OutAttributesMap: Pin doesn't exists. @@").ToString(), this);
		return;
	}

	UEdGraphPin* GetAttributesMapNodeOutIsSuccessPin = GetAttributesMapNode->FindPin(UEdGraphSchema_K2::PN_ReturnValue);
	if (GetAttributesMapNodeOutIsSuccessPin == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("MissingReturnValuePin", "ReturnValueMap: Pin doesn't exists. @@").ToString(), this);
		return;
	}

	UEdGraphPin* GetAttributesMapNodeThenPin = GetAttributesMapNode->GetThenPin();

	// Hook up inputs
	Schema->TryCreateConnection(GetAttributesMapNodeSelfPin, DMXSubsystemResult);
	CompilerContext.MovePinLinksToIntermediate(*GetInputDMXFixturePatchPin(), *GetAttributesMapNodeInFixturePatchPin);

	// Hook up outputs
	CompilerContext.MovePinLinksToIntermediate(*GetOutputAttributesMapPin(), *GetAttributesMapNodeOutAttributesMapPin);
	CompilerContext.MovePinLinksToIntermediate(*GetOutputIsSuccessPin(), *GetAttributesMapNodeOutIsSuccessPin);
	Schema->TryCreateConnection(LastThenPin, GetAttributesMapNodeExecPin);
	LastThenPin = GetAttributesMapNodeThenPin;

	// Loop GetAttributesValueName nodes to execute. 
	if (UserDefinedPins.Num() > 0)
	{
		TArray<UEdGraphPin*> IntPairs;
		TArray<UEdGraphPin*> NamePairs;

		// Call Attributes for dmx function values
		for (const TSharedPtr<FUserPinInfo>& PinInfo : UserDefinedPins)
		{
			UEdGraphPin* Pin = FindPin(PinInfo->PinName);
			if (Pin != nullptr)
			{
				if (Pin->Direction == EGPD_Output)
				{
					IntPairs.Add(Pin);
				}
				else if (Pin->Direction == EGPD_Input)
				{
					NamePairs.Add(Pin);
				}
			}
			else
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("MissingDefinedPinPin", "DefinedPin: Pin doesn't exists. @@").ToString(), this);
				return;
			}
		}

		check(IntPairs.Num() == NamePairs.Num());

		bool bResult = false;
		for (int32 PairIndex = 0; PairIndex < NamePairs.Num(); ++PairIndex)
		{
			const FName GetAttributesValueName = GET_FUNCTION_NAME_CHECKED(UDMXSubsystem, GetFunctionsValue);
			UFunction* GetAttributesValuePointer = FindUField<UFunction>(UDMXSubsystem::StaticClass(), GetAttributesValueName);
			check(GetAttributesValuePointer);

			UK2Node_CallFunction* GetAttributesValueNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
			GetAttributesValueNode->FunctionReference.SetExternalMember(GetAttributesValueName, UDMXSubsystem::StaticClass());
			GetAttributesValueNode->AllocateDefaultPins();

			UEdGraphPin* GetAttributesValueSelfPin = GetAttributesValueNode->FindPin(UEdGraphSchema_K2::PN_Self);
			if (GetAttributesValueSelfPin == nullptr)
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("MissingSelfPin", "Self: Pin doesn't exists. @@").ToString(), this);
				return;
			}

			UEdGraphPin* GetAttributesValueExecPin = GetAttributesValueNode->GetExecPin();
			UEdGraphPin* GetAttributesValueNodeOutInAttributePin = GetAttributesValueNode->FindPin(TEXT("FunctionAttributeName"));
			if (GetAttributesValueNodeOutInAttributePin == nullptr)
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("MissingFunctionAttributeNamePin", "FunctionAttributeName: Pin doesn't exists. @@").ToString(), this);
				return;
			}

			UEdGraphPin* GetAttributesValueNodeOutAttributesMapPin = GetAttributesValueNode->FindPin(TEXT("InAttributesMap"));
			if (GetAttributesValueNodeOutAttributesMapPin == nullptr)
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("MissingInAttributesMapPin", "InAttributesMap: Pin doesn't exists. @@").ToString(), this);
				return;
			}

			UEdGraphPin* GetAttributesValueNodeOutValuePin = GetAttributesValueNode->FindPin(UEdGraphSchema_K2::PN_ReturnValue);
			if (GetAttributesValueNodeOutValuePin == nullptr)
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("MissingReturnValuePin", "ReturnValueMap: Pin doesn't exists. @@").ToString(), this);
				return;
			}
			UEdGraphPin* GetAttributesValueNodeThenPin = GetAttributesValueNode->GetThenPin();

			// Input
			Schema->TryCreateConnection(GetAttributesValueSelfPin, DMXSubsystemResult);
			CompilerContext.MovePinLinksToIntermediate(*NamePairs[PairIndex], *GetAttributesValueNodeOutInAttributePin);

			// Output
			Schema->TryCreateConnection(GetAttributesValueNodeOutAttributesMapPin, GetAttributesMapNodeOutAttributesMapPin);
			CompilerContext.MovePinLinksToIntermediate(*IntPairs[PairIndex], *GetAttributesValueNodeOutValuePin);

			// Execution
			Schema->TryCreateConnection(LastThenPin, GetAttributesValueExecPin);
			LastThenPin = GetAttributesValueNodeThenPin;
		}

		CompilerContext.MovePinLinksToIntermediate(*GetThenPin(), *LastThenPin);
	}
	else
	{
		CompilerContext.MovePinLinksToIntermediate(*GetThenPin(), *LastThenPin);
	}
}

void UK2Node_GetDMXAttributeValues::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	UClass* ActionKey = GetClass();

	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_GetDMXAttributeValues::GetMenuCategory() const
{
	return FText::FromString(DMX_K2_CATEGORY_NAME);
}

UEdGraphPin* UK2Node_GetDMXAttributeValues::CreatePinFromUserDefinition(const TSharedPtr<FUserPinInfo> NewPinInfo)
{
	UEdGraphPin* NewPin = CreatePin(NewPinInfo->DesiredPinDirection, NewPinInfo->PinType, NewPinInfo->PinName);
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	K2Schema->SetPinAutogeneratedDefaultValue(NewPin, NewPinInfo->PinDefaultValue);

	if (NewPinInfo->DesiredPinDirection == EEdGraphPinDirection::EGPD_Input)
	{
		NewPin->bHidden = true;
	}

	return NewPin;
}

bool UK2Node_GetDMXAttributeValues::CanCreateUserDefinedPin(const FEdGraphPinType& InPinType, EEdGraphPinDirection InDesiredDirection, FText& OutErrorMessage)
{
	if (!IsEditable())
	{
		return false;
	}

	// Make sure that if this is an exec node we are allowed one.
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Exec && !CanModifyExecutionWires())
	{
		OutErrorMessage = LOCTEXT("MultipleExecPinError", "Cannot support more exec pins!");
		return false;
	}
	else
	{
		TArray<TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>> TypeTree;
		Schema->GetVariableTypeTree(TypeTree, ETypeTreeFilter::RootTypesOnly);

		bool bIsValid = false;
		for (TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>& TypeInfo : TypeTree)
		{
			FEdGraphPinType CurrentType = TypeInfo->GetPinType(false);
			// only concerned with the list of categories
			if (CurrentType.PinCategory == InPinType.PinCategory)
			{
				bIsValid = true;
				break;
			}
		}

		if (!bIsValid)
		{
			OutErrorMessage = LOCTEXT("AddInputPinError", "Cannot add pins of this type to custom event node!");
			return false;
		}
	}

	return true;
}

bool UK2Node_GetDMXAttributeValues::ModifyUserDefinedPinDefaultValue(TSharedPtr<FUserPinInfo> PinInfo, const FString& NewDefaultValue)
{
	if (Super::ModifyUserDefinedPinDefaultValue(PinInfo, NewDefaultValue))
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		K2Schema->HandleParameterDefaultValueChanged(this);

		return true;
	}
	return false;
}

UEdGraphPin* UK2Node_GetDMXAttributeValues::GetInputDMXFixturePatchPin() const
{
	UEdGraphPin* Pin = FindPin(InputDMXFixturePatchPinName);
	if (Pin == nullptr)
	{
		UE_LOG_DMXBLUEPRINTGRAPH(Error, TEXT("No InputDMXFixturePatchPin found"));
		return nullptr;
	}

	check(Pin->Direction == EGPD_Input);

	return Pin;
}

UEdGraphPin* UK2Node_GetDMXAttributeValues::GetOutputAttributesMapPin() const
{
	UEdGraphPin* Pin = FindPin(OutputAttributesMapPinName);
	if (Pin == nullptr)
	{
		UE_LOG_DMXBLUEPRINTGRAPH(Error, TEXT("No OutputAttributesMapPin found"));
		return nullptr;
	}

	check(Pin->Direction == EGPD_Output);

	return Pin;
}

UEdGraphPin* UK2Node_GetDMXAttributeValues::GetOutputIsSuccessPin() const
{
	UEdGraphPin* Pin = FindPin(OutputIsSuccessPinName);
	if (Pin == nullptr)
	{
		UE_LOG_DMXBLUEPRINTGRAPH(Error, TEXT("No OutputIsSuccessPin found"));
		return nullptr;
	}

	check(Pin->Direction == EGPD_Output);

	return Pin;
}

void UK2Node_GetDMXAttributeValues::ExposeAttributes()
{
	if (bIsExposed == true && UserDefinedPins.Num())
	{
		return;
	}

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	if (const FDMXFixtureMode* ActiveFixtureMode = GetActiveFixtureMode())
	{
		for (const FDMXFixtureFunction& Function : ActiveFixtureMode->Functions)
		{
			if (!Function.Attribute.IsValid())
			{
				continue;
			}

			{
				FEdGraphPinType PinType;
				PinType.PinCategory = UEdGraphSchema_K2::PC_Int;
				UEdGraphPin* Pin = CreateUserDefinedPin(GetPinName(Function), PinType, EGPD_Output);
			}

			{
				FEdGraphPinType PinType;
				PinType.PinCategory = UEdGraphSchema_K2::PC_Name;
				UEdGraphPin* Pin = CreateUserDefinedPin(*(GetPinName(Function).ToString() + FString("_Input")), PinType, EGPD_Input);
				K2Schema->TrySetDefaultValue(*Pin, Function.Attribute.Name.ToString());
			}

			const bool bIsCompiling = GetBlueprint()->bBeingCompiled;
			if (!bIsCompiling)
			{
				FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
			}
		}
	}
	else
	{
		UE_LOG_DMXBLUEPRINTGRAPH(Verbose, TEXT("No active mode found"));
		return;
	}


	Modify();
	bIsExposed = true;
}

void UK2Node_GetDMXAttributeValues::ResetAttributes()
{
	if (bIsExposed)
	{
		while (UserDefinedPins.Num())
		{
			TSharedPtr<FUserPinInfo> Pin = UserDefinedPins[0];
			RemoveUserDefinedPin(Pin);
		}

		// Reconstruct the entry/exit definition and recompile the blueprint to make sure the signature has changed before any fixups
		bDisableOrphanPinSaving = true;

		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		K2Schema->HandleParameterDefaultValueChanged(this);
	}


	bIsExposed = false;
}

UDMXEntityFixturePatch* UK2Node_GetDMXAttributeValues::GetFixturePatchFromPin() const
{
	UEdGraphPin* FixturePatchPin = GetInputDMXFixturePatchPin();
	if (FixturePatchPin == nullptr)
	{
		return nullptr;
	}

	// Case with default object
	if (FixturePatchPin->DefaultObject != nullptr && FixturePatchPin->LinkedTo.Num() == 0)
	{
		return Cast<UDMXEntityFixturePatch>(FixturePatchPin->DefaultObject);
	}

	// Case with linked object
	if (FixturePatchPin->LinkedTo.Num() > 0)
	{
		if (UK2Node_GetDMXFixturePatch* NodeGetFixturePatch = Cast<UK2Node_GetDMXFixturePatch>(FixturePatchPin->LinkedTo[0]->GetOwningNode()))
		{
			return NodeGetFixturePatch->GetFixturePatchRefFromPin().GetFixturePatch();
		}
	}

	return nullptr;
}

FName UK2Node_GetDMXAttributeValues::GetPinName(const FDMXFixtureFunction& Function)
{
	FString EnumString;
	EnumString = StaticEnum<EDMXFixtureSignalFormat>()->GetDisplayNameTextByIndex((int64)Function.DataType).ToString();

	return *FString::Printf(TEXT("%s_%s"), *Function.Attribute.Name.ToString(), *EnumString);
}

const FDMXFixtureMode* UK2Node_GetDMXAttributeValues::GetActiveFixtureMode() const
{
	if (UDMXEntityFixturePatch* FixturePatch = GetFixturePatchFromPin())
	{
		if (const FDMXFixtureMode* FixtureMode = FixturePatch->GetActiveMode())
		{
			return FixtureMode;
		}
	}

	return nullptr;
}

UDMXEntityFixtureType* UK2Node_GetDMXAttributeValues::GetParentFixtureType() const
{
	if (UDMXEntityFixturePatch* FixturePatch = GetFixturePatchFromPin())
	{
		return FixturePatch->GetFixtureType();
	}

	return nullptr;
}

void UK2Node_GetDMXAttributeValues::OnFixtureTypeChanged(const UDMXEntityFixtureType* InFixtureType)
{
	// Check if there are any pins exists
	if (Pins.Num() == 0)
	{
		return;
	}

	if (const UDMXEntityFixtureType* ParentFixtureType = GetParentFixtureType())
	{
		if (const FDMXFixtureMode* ActiveFixtureMode = GetActiveFixtureMode())
		{
			// Reset Attributes if there is a match in fixture types objects
			if (InFixtureType && (InFixtureType == ParentFixtureType))
			{
				ResetAttributes();

				// Ask to recompile the bplueprint
				if (UBlueprint* BP = GetBlueprint())
				{
					FBlueprintEditorUtils::MarkBlueprintAsModified(BP);
				}
			}
		}
	}
}

void UK2Node_GetDMXAttributeValues::OnDataTypeChanged(const UDMXEntityFixtureType* InFixtureType, const FDMXFixtureMode& InMode)
{
	// DEPRECATED 5.0
	OnFixtureTypeChanged(InFixtureType);
}

#undef LOCTEXT_NAMESPACE
