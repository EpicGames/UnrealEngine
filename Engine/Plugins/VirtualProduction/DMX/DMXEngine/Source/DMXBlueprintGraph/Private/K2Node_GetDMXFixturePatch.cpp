// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_GetDMXFixturePatch.h"
#include "K2Node_GetDMXAttributeValues.h"
#include "Library/DMXEntityFixturePatch.h"
#include "DMXSubsystem.h"
#include "DMXProtocolConstants.h"
#include "DMXBlueprintGraphLog.h"

#include "KismetCompilerMisc.h"
#include "KismetCompiler.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "K2Node_CallFunction.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "UObject/FrameworkObjectVersion.h"

#define LOCTEXT_NAMESPACE "UK2Node_GetDMXFixturePatch"

const FName UK2Node_GetDMXFixturePatch::InputDMXFixturePatchPinName(TEXT("InFixturePatch"));
const FName UK2Node_GetDMXFixturePatch::OutputDMXFixturePatchPinName(TEXT("OutFixturePatch"));

void UK2Node_GetDMXFixturePatch::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);

	// Check if it not SavingPackage
	if (Ar.IsSaving() && !UE::IsSavingPackage())
	{
		if (Ar.IsObjectReferenceCollector() || Ar.Tell() < 0)
		{
			// When this is a reference collector/modifier, serialize some pins as structs
			FixupPinStringDataReferences(&Ar);
		}
	}

	// Do not call parent, but call grandparent
	UEdGraphNode::Serialize(Ar);

	if (Ar.IsLoading() && ((Ar.GetPortFlags() & PPF_Duplicate) == 0))
	{
		// Fix up pin default values, must be done before post load
		FixupPinDefaultValues();

		if (GIsEditor)
		{
			// We need to serialize string data references on load in editor builds so the cooker knows about them
			FixupPinStringDataReferences(nullptr);
		}
	}
}

void UK2Node_GetDMXFixturePatch::AllocateDefaultPins()
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Input pins
	UEdGraphPin* InputDMXFixturePatchPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Struct, FDMXEntityFixturePatchRef::StaticStruct(), InputDMXFixturePatchPinName);
	K2Schema->ConstructBasicPinTooltip(*InputDMXFixturePatchPin, LOCTEXT("InputDMXFixtureTypePin", "Get the fixture patch reference."), InputDMXFixturePatchPin->PinToolTip);
	InputDMXFixturePatchPin->bNotConnectable = true;



	// Output pins
	UEdGraphPin* OutputDMXFixturePatchPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Object, UDMXEntityFixturePatch::StaticClass(), OutputDMXFixturePatchPinName);
	K2Schema->ConstructBasicPinTooltip(*OutputDMXFixturePatchPin, LOCTEXT("OutputDMXFixturePatch", "Fixture patch."), OutputDMXFixturePatchPin->PinToolTip);
	OutputDMXFixturePatchPin->PinType.bIsReference = true;

	Super::AllocateDefaultPins();
}

FText UK2Node_GetDMXFixturePatch::GetTooltipText() const
{
	return LOCTEXT("TooltipText", "Get selected Fixture Patch");
}

FText UK2Node_GetDMXFixturePatch::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("NodeTitle", "Get DMX Fixture Patch");
}

void UK2Node_GetDMXFixturePatch::PinDefaultValueChanged(UEdGraphPin* FromPin)
{
	Super::PinDefaultValueChanged(FromPin);

	if (!FromPin)
	{
		return; 
	}

	if (FromPin->PinName == InputDMXFixturePatchPinName)
	{
		NotifyInputChanged();
	}
}

void UK2Node_GetDMXFixturePatch::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	const UEdGraphSchema_K2* K2Schema = CompilerContext.GetSchema();

	// First node to execute. GetDMXSubsystem
	FName GetDMXSubsystemFunctionName = GET_FUNCTION_NAME_CHECKED(UDMXSubsystem, GetDMXSubsystem_Pure);
	const UFunction* GetDMXSubsystemFunction = UDMXSubsystem::StaticClass()->FindFunctionByName(GetDMXSubsystemFunctionName);
	check(nullptr != GetDMXSubsystemFunction);
	UK2Node_CallFunction* DMXSubsystemNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	DMXSubsystemNode->SetFromFunction(GetDMXSubsystemFunction);
	DMXSubsystemNode->AllocateDefaultPins();

	UEdGraphPin* DMXSubsystemResult = DMXSubsystemNode->GetReturnValuePin();

	// Second node to execute. GetFixturePatch
	const FName GetFixturePatchName = GET_FUNCTION_NAME_CHECKED(UDMXSubsystem, GetFixturePatch);
	const UFunction* GetFixturePatchFunction = UDMXSubsystem::StaticClass()->FindFunctionByName(GetFixturePatchName);
	check(nullptr != GetFixturePatchFunction);

	// Spawn call function node
	UK2Node_CallFunction* SendDataGetFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	SendDataGetFunction->FunctionReference.SetExternalMember(GetFixturePatchName, UK2Node_CallFunction::StaticClass());

	// Set function node pins
	SendDataGetFunction->SetFromFunction(GetFixturePatchFunction);
	SendDataGetFunction->AllocateDefaultPins();

	// Hook up function node inputs
	UEdGraphPin* FunctionInFixturePatchPin = SendDataGetFunction->FindPin(TEXT("InFixturePatch"));
	if (FunctionInFixturePatchPin == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("MissingInFixturePatchPin", "InFixturePatch: Pin doesn't exists. @@").ToString(), this);
		return;
	}

	UEdGraphPin* FunctionOutFixturePatchPin = SendDataGetFunction->FindPin(UEdGraphSchema_K2::PN_ReturnValue);
	if (FunctionOutFixturePatchPin == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("MissingReturnPin", "Return: Pin doesn't exists. @@").ToString(), this);
		return;
	}

	UEdGraphPin* FunctionSelfPin = SendDataGetFunction->FindPin(UEdGraphSchema_K2::PN_Self);
	if (FunctionSelfPin == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("MissingSelfPin", "Self: Pin doesn't exists. @@").ToString(), this);
		return;
	}

	const FString&& FixturePatchStr = GetFixturePatchValueAsString();

	// Hook up input
	K2Schema->TryCreateConnection(FunctionSelfPin, DMXSubsystemResult);
	K2Schema->TrySetDefaultValue(*FunctionInFixturePatchPin, FixturePatchStr);
	check(FunctionInFixturePatchPin->GetDefaultAsString().Equals(FixturePatchStr));

	// Hook up outputs
	CompilerContext.MovePinLinksToIntermediate(*GetOutputDMXFixturePatchPin(), *FunctionOutFixturePatchPin);
}

void UK2Node_GetDMXFixturePatch::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	UClass* ActionKey = GetClass();
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_GetDMXFixturePatch::GetMenuCategory() const
{
	return FText::FromString(DMX_K2_CATEGORY_NAME);
}

UEdGraphPin* UK2Node_GetDMXFixturePatch::GetInputDMXFixturePatchPin() const
{
	UEdGraphPin* Pin = FindPin(InputDMXFixturePatchPinName);
	if (Pin == nullptr)
	{
		UE_LOG_DMXBLUEPRINTGRAPH(Error, TEXT("No DMXFixturePatchPin found"));
		return nullptr;
	}
	check(Pin->Direction == EGPD_Input);

	return Pin;
}

UEdGraphPin* UK2Node_GetDMXFixturePatch::GetOutputDMXFixturePatchPin() const
{
	UEdGraphPin* Pin = FindPin(OutputDMXFixturePatchPinName);
	if (Pin == nullptr)
	{
		UE_LOG_DMXBLUEPRINTGRAPH(Error, TEXT("No DMXFixturePatchPin found"));
		return nullptr;
	}
	check(Pin->Direction == EGPD_Output);

	return Pin;
}

FString UK2Node_GetDMXFixturePatch::GetFixturePatchValueAsString() const
{
	UEdGraphPin* FixturePatchPin = GetInputDMXFixturePatchPin();

	FString PatchRefString;

	// Case with default object
	if (FixturePatchPin->LinkedTo.Num() == 0)
	{
		PatchRefString = FixturePatchPin->GetDefaultAsString();
	}
	// Case with linked object
	else
	{
		PatchRefString = FixturePatchPin->LinkedTo[0]->GetDefaultAsString();
	}

	return PatchRefString;
}

FDMXEntityFixturePatchRef UK2Node_GetDMXFixturePatch::GetFixturePatchRefFromPin() const
{
	FDMXEntityFixturePatchRef PatchRef;

	const FString&& PatchRefString = GetFixturePatchValueAsString();
	if (!PatchRefString.IsEmpty())
	{
		FDMXEntityReference::StaticStruct()->ImportText(*PatchRefString, &PatchRef, nullptr, EPropertyPortFlags::PPF_None, GLog, FDMXEntityReference::StaticStruct()->GetName());
	}

	return PatchRef;
}

void UK2Node_GetDMXFixturePatch::SetInFixturePatchPinValue(const FDMXEntityFixturePatchRef& InPatchRef) const
{
	FString ValueString;
	FDMXEntityReference::StaticStruct()->ExportText(ValueString, &InPatchRef, nullptr, nullptr, EPropertyPortFlags::PPF_None, nullptr);
	UEdGraphPin* FixturePatchPin = GetInputDMXFixturePatchPin();
	FixturePatchPin->GetSchema()->TrySetDefaultValue(*FixturePatchPin, ValueString);
}

void UK2Node_GetDMXFixturePatch::NotifyInputChanged()
{
	// Notify all GetDMXActiveModeFunctionValues nodes connected to our output pin
	const TArray<UEdGraphPin*>& OutputConnections = GetOutputDMXFixturePatchPin()->LinkedTo;
	for (const UEdGraphPin* ConnectedPin : OutputConnections)
	{
		if (UK2Node_GetDMXAttributeValues* K2Node_GetDMXAttributeValues = Cast<UK2Node_GetDMXAttributeValues>(ConnectedPin->GetOwningNode()))
		{
			K2Node_GetDMXAttributeValues->OnFixturePatchChanged();
		}
	}

	if (UBlueprint* BP = GetBlueprint())
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(BP);
	}

	UEdGraph* Graph = GetGraph();
	Graph->NotifyNodeChanged(this);
}

#undef LOCTEXT_NAMESPACE
