// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_GetDMXFixtureType.h"
#include "K2Node_GetDMXAttributeValues.h"
#include "Library/DMXEntityFixtureType.h"
#include "DMXSubsystem.h"
#include "DMXProtocolConstants.h"
#include "DMXBlueprintGraphLog.h"

#include "KismetCompilerMisc.h"
#include "KismetCompiler.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "K2Node_CallFunction.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "UObject/FrameworkObjectVersion.h"

#define LOCTEXT_NAMESPACE "UK2Node_GetDMXFixtureType"

const FName UK2Node_GetDMXFixtureType::InputDMXFixtureTypePinName(TEXT("InFixtureType"));
const FName UK2Node_GetDMXFixtureType::OutputDMXFixtureTypePinName(TEXT("OutFixtureType"));

void UK2Node_GetDMXFixtureType::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);

	// Check if it not SavingPackage
	if (Ar.IsSaving() && !UE::IsSavingPackage())
	{
		if (Ar.IsObjectReferenceCollector() || Ar.Tell() < 0)
		{
			// When this is a reference collector/modifier, serialize some pins as structs
			FixupPinStringDataReferences(&Ar);
		}
	}

	// Do not call parent, but call grandparent
	UEdGraphNode::Serialize(Ar);

	if (Ar.IsLoading() && ((Ar.GetPortFlags() & PPF_Duplicate) == 0))
	{
		// Fix up pin default values, must be done before post load
		FixupPinDefaultValues();

		if (GIsEditor)
		{
			// We need to serialize string data references on load in editor builds so the cooker knows about them
			FixupPinStringDataReferences(nullptr);
		}
	}
}

void UK2Node_GetDMXFixtureType::AllocateDefaultPins()
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Input pins
	UEdGraphPin* InputDMXFixtureTypePin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Struct, FDMXEntityFixtureTypeRef::StaticStruct(), InputDMXFixtureTypePinName);
	K2Schema->ConstructBasicPinTooltip(*InputDMXFixtureTypePin, LOCTEXT("InputDMXFixtureTypePin", "Get the fixture Type reference."), InputDMXFixtureTypePin->PinToolTip);
	InputDMXFixtureTypePin->bNotConnectable = true;

	// Output pins
	UEdGraphPin* OutputDMXFixtureTypePin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Object, UDMXEntityFixtureType::StaticClass(), OutputDMXFixtureTypePinName);
	K2Schema->ConstructBasicPinTooltip(*OutputDMXFixtureTypePin, LOCTEXT("OutputDMXFixtureType", "Fixture Type."), OutputDMXFixtureTypePin->PinToolTip);
	OutputDMXFixtureTypePin->PinType.bIsReference = true;

	Super::AllocateDefaultPins();
}

FText UK2Node_GetDMXFixtureType::GetTooltipText() const
{
	return LOCTEXT("TooltipText", "Get selected Fixture Type");
}

FText UK2Node_GetDMXFixtureType::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("NodeTitle", "Get DMX Fixture Type");
}

void UK2Node_GetDMXFixtureType::PinDefaultValueChanged(UEdGraphPin* FromPin)
{
	Super::PinDefaultValueChanged(FromPin);

	UEdGraph* Graph = GetGraph();
	Graph->NotifyNodeChanged(this);
}

void UK2Node_GetDMXFixtureType::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	const UEdGraphSchema_K2* K2Schema = CompilerContext.GetSchema();

	// First node to execute. GetDMXSubsystem
	FName GetDMXSubsystemFunctionName = GET_FUNCTION_NAME_CHECKED(UDMXSubsystem, GetDMXSubsystem_Pure);
	const UFunction* GetDMXSubsystemFunction = UDMXSubsystem::StaticClass()->FindFunctionByName(GetDMXSubsystemFunctionName);
	check(nullptr != GetDMXSubsystemFunction);
	UK2Node_CallFunction* DMXSubsystemNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	DMXSubsystemNode->SetFromFunction(GetDMXSubsystemFunction);
	DMXSubsystemNode->AllocateDefaultPins();

	UEdGraphPin* DMXSubsystemResult = DMXSubsystemNode->GetReturnValuePin();

	// Second node to execute. GetFixtureType
	const FName GetFixtureTypeName = GET_FUNCTION_NAME_CHECKED(UDMXSubsystem, GetFixtureType);
	const UFunction* GetFixtureTypeFunction = UDMXSubsystem::StaticClass()->FindFunctionByName(GetFixtureTypeName);
	check(nullptr != GetFixtureTypeFunction);

	// Spawn call function node
	UK2Node_CallFunction* SendDataGetFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	SendDataGetFunction->FunctionReference.SetExternalMember(GetFixtureTypeName, UK2Node_CallFunction::StaticClass());

	// Set function node pins
	SendDataGetFunction->SetFromFunction(GetFixtureTypeFunction);
	SendDataGetFunction->AllocateDefaultPins();

	// Hook up function node inputs
	UEdGraphPin* FunctionInFixtureTypePin = SendDataGetFunction->FindPin(TEXT("InFixtureType"));
	if (FunctionInFixtureTypePin == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("MissingInFixtureTypePin", "InFixtureType: Pin doesn't exists. @@").ToString(), this);
		return;
	}

	UEdGraphPin* FunctionOutFixtureTypePin = SendDataGetFunction->FindPin(UEdGraphSchema_K2::PN_ReturnValue);
	if (FunctionOutFixtureTypePin == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("MissingReturnPin", "Return: Pin doesn't exists. @@").ToString(), this);
		return;
	}

	UEdGraphPin* FunctionSelfPin = SendDataGetFunction->FindPin(UEdGraphSchema_K2::PN_Self);
	if (FunctionSelfPin == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("MissingSelfPin", "Self: Pin doesn't exists. @@").ToString(), this);
		return;
	}

	const FString&& FixtureTypeStr = GetFixtureTypeValueAsString();

	// Hook up input
	K2Schema->TryCreateConnection(FunctionSelfPin, DMXSubsystemResult);
	K2Schema->TrySetDefaultValue(*FunctionInFixtureTypePin, FixtureTypeStr);
	check(FunctionInFixtureTypePin->GetDefaultAsString().Equals(FixtureTypeStr));

	// Hook up outputs
	CompilerContext.MovePinLinksToIntermediate(*GetOutputDMXFixtureTypePin(), *FunctionOutFixtureTypePin);
}

void UK2Node_GetDMXFixtureType::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	UClass* ActionKey = GetClass();
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_GetDMXFixtureType::GetMenuCategory() const
{
	return FText::FromString(DMX_K2_CATEGORY_NAME);
}

UEdGraphPin* UK2Node_GetDMXFixtureType::GetInputDMXFixtureTypePin() const
{
	UEdGraphPin* Pin = FindPin(InputDMXFixtureTypePinName);
	if (Pin == nullptr)
	{
		UE_LOG_DMXBLUEPRINTGRAPH(Error, TEXT("No DMXFixtureTypePin found"));
		return nullptr;
	}
	check(Pin->Direction == EGPD_Input);

	return Pin;
}

UEdGraphPin* UK2Node_GetDMXFixtureType::GetOutputDMXFixtureTypePin() const
{
	UEdGraphPin* Pin = FindPin(OutputDMXFixtureTypePinName);
	if (Pin == nullptr)
	{
		UE_LOG_DMXBLUEPRINTGRAPH(Error, TEXT("No DMXFixtureTypePin found"));
		return nullptr;
	}
	check(Pin->Direction == EGPD_Output);

	return Pin;
}

FString UK2Node_GetDMXFixtureType::GetFixtureTypeValueAsString() const
{
	UEdGraphPin* FixtureTypePin = GetInputDMXFixtureTypePin();

	FString TypeRefString;

	// Case with default object
	if (FixtureTypePin->LinkedTo.Num() == 0)
	{
		TypeRefString = FixtureTypePin->GetDefaultAsString();
	}
	// Case with linked object
	else
	{
		TypeRefString = FixtureTypePin->LinkedTo[0]->GetDefaultAsString();
	}

	return TypeRefString;
}

FDMXEntityFixtureTypeRef UK2Node_GetDMXFixtureType::GetFixtureTypeRefFromPin() const
{
	FDMXEntityFixtureTypeRef TypeRef;

	const FString&& TypeRefString = GetFixtureTypeValueAsString();
	if (!TypeRefString.IsEmpty())
	{
		FDMXEntityReference::StaticStruct()->ImportText(*TypeRefString, &TypeRef, nullptr, EPropertyPortFlags::PPF_None, GLog, FDMXEntityReference::StaticStruct()->GetName());
	}

	return TypeRef;
}

void UK2Node_GetDMXFixtureType::SetInFixtureTypePinValue(const FDMXEntityFixtureTypeRef& InTypeRef) const
{
	FString ValueString;
	FDMXEntityReference::StaticStruct()->ExportText(ValueString, &InTypeRef, nullptr, nullptr, EPropertyPortFlags::PPF_None, nullptr);
	UEdGraphPin* FixtureTypePin = GetInputDMXFixtureTypePin();
	FixtureTypePin->GetSchema()->TrySetDefaultValue(*FixtureTypePin, ValueString);
}

#undef LOCTEXT_NAMESPACE
