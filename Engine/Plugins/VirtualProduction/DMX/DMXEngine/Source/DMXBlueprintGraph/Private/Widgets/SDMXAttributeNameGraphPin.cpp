// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXAttributeNameGraphPin.h"

#include "DMXAttribute.h"
#include "Widgets/SNameListPicker.h"


void SDMXAttributeNameGraphPin::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget>	SDMXAttributeNameGraphPin::GetDefaultValueWidget()
{
	return SNew(SNameListPicker)
		.HasMultipleValues(false)
		.Value(this, &SDMXAttributeNameGraphPin::GetValue)
		.OnValueChanged(this, &SDMXAttributeNameGraphPin::SetValue)
		.OptionsSource(MakeAttributeLambda(&FDMXAttributeName::GetPredefinedValues))
		.bDisplayWarningIcon(true)
		.Visibility(this, &SGraphPin::GetDefaultValueVisibility);
}

FName SDMXAttributeNameGraphPin::GetValue() const
{
	FDMXAttributeName NameItem;

	if (!GraphPinObj->GetDefaultAsString().IsEmpty())
	{
		FDMXAttributeName::StaticStruct()->ImportText(*GraphPinObj->GetDefaultAsString(), &NameItem, nullptr, EPropertyPortFlags::PPF_None, GLog, FDMXAttributeName::StaticStruct()->GetName());
	}

	return NameItem.Name;
}

void SDMXAttributeNameGraphPin::SetValue(FName NewValue)
{
	FString ValueString;
	FDMXAttributeName NewNameItem(NewValue);
	FDMXAttributeName::StaticStruct()->ExportText(ValueString, &NewNameItem, nullptr, nullptr, EPropertyPortFlags::PPF_None, nullptr);
	GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, ValueString);
}
