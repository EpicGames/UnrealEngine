// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphPin.h"

#include "CoreMinimal.h"


class SDMXAttributeNameGraphPin
	: public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SDMXAttributeNameGraphPin)
	{}
	SLATE_END_ARGS()

	/**  Constructs this widet */
	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

	//~ Begin SGraphPin Interface
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface

private:
	/** Gets the currently selected value */
	FName GetValue() const;

	/** Sets the currently selected value */
	void SetValue(FName NewValue);
};
