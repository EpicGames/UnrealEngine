// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDMXEntityReferenceGraphPin.h"

#include "Library/DMXEntityReference.h"
#include "Library/DMXEntity.h"
#include "Widgets/SDMXEntityDropdownMenu.h"
#include "Widgets/SAssetPickerButton.h"

#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "SDMXEntityReferenceGraphPin"

void SDMXEntityReferenceGraphPin::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget> SDMXEntityReferenceGraphPin::GetDefaultValueWidget()
{
	// Create entity picker button
	TSharedRef<SDMXEntityPickerButton<UDMXEntity>> EntityComboButton = SNew(SDMXEntityPickerButton<UDMXEntity>)
		.CurrentEntity(this, &SDMXEntityReferenceGraphPin::GetEntity)
		.OnEntitySelected(this, &SDMXEntityReferenceGraphPin::OnEntitySelected)
		.EntityTypeFilter(this, &SDMXEntityReferenceGraphPin::GetEntityType)
		.DMXLibrary(this, &SDMXEntityReferenceGraphPin::GetDMXLibrary)
		.ForegroundColor(this, &SDMXEntityReferenceGraphPin::GetComboForeground)
		.Visibility(this, &SGraphPin::GetDefaultValueVisibility);

	if (GetPinValue().bDisplayLibraryPicker)
	{
		// Create the asset picker button

		// Check with the node to see if there is any "AllowClasses" or "DisallowedClasses" metadata for the pin
		TArray<FName> AllowedClasses;
		FString AllowedClassesFilterString = GraphPinObj->GetOwningNode()->GetPinMetaData(GraphPinObj->PinName, FName(TEXT("AllowedClasses")));
		if (!AllowedClassesFilterString.IsEmpty())
		{
			// Parse and add the classes from the metadata
			TArray<FString> AllowedClassesFilterNames;
			AllowedClassesFilterString.ParseIntoArray(AllowedClassesFilterNames, TEXT(","), true);
			for (const FString& AllowedClassesFilterName : AllowedClassesFilterNames)
			{
				AllowedClasses.Add(FName(*AllowedClassesFilterName));
			}
		}

		TArray<FName> DisallowedClasses;
		FString DisallowedClassesFilterString = GraphPinObj->GetOwningNode()->GetPinMetaData(GraphPinObj->PinName, FName(TEXT("DisallowedClasses")));
		if (!DisallowedClassesFilterString.IsEmpty())
		{
			TArray<FString> DisallowedClassesFilterNames;
			DisallowedClassesFilterString.ParseIntoArray(DisallowedClassesFilterNames, TEXT(","), true);
			for (const FString& DisallowedClassesFilterName : DisallowedClassesFilterNames)
			{
				DisallowedClasses.Add(FName(*DisallowedClassesFilterName));
			}
		}

		TSharedRef<SAssetPickerButton> AssetPickerButton = SNew(SAssetPickerButton)
			.AssetClass(UDMXLibrary::StaticClass())
			.OnAssetSelected(this, &SDMXEntityReferenceGraphPin::OnDMXLibrarySelected)
			.CurrentAssetValue(this, &SDMXEntityReferenceGraphPin::GetCurrentDMXLibrary)
			.OnParentIsHovered(FOnParentIsHovered::CreateLambda([this]() { return IsHovered(); }))
			.Visibility(this, &SGraphPin::GetDefaultValueVisibility);

		return SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			[
				AssetPickerButton
			]
			
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.Padding(0.0f, 5.0f, 0.0f, 0.0f)
			[
				EntityComboButton
			];
	}
	else // Don't display the DMX Library picker
	{
		return EntityComboButton;
	}
}

FDMXEntityReference SDMXEntityReferenceGraphPin::GetPinValue() const
{
	FDMXEntityReference EntityReference;

	const FString&& EntityRefStr = GraphPinObj->GetDefaultAsString();
	if (!EntityRefStr.IsEmpty())
	{
		FDMXEntityReference::StaticStruct()->ImportText(*EntityRefStr, &EntityReference, nullptr, EPropertyPortFlags::PPF_None, GLog, FDMXEntityReference::StaticStruct()->GetName());
	}
	else if (!GraphPinObj->IsPendingKill() && IsValid(GraphPinObj->GetSchema()) && GraphPinObj->GetDefaultAsString().IsEmpty())
	{
		// Defaults are empty and, no matter the type of Entity Ref, we would return UDMXEntity as type because
		// we are creating a FDMXEntityReference to return.
		// So we fix it by returning correct default values from the actual pin type
		UScriptStruct* StructClass = CastChecked<UScriptStruct>(GraphPinObj->PinType.PinSubCategoryObject.Get());
		StructClass->InitializeDefaultValue((uint8*)&EntityReference);
		// We don't save these defaults to the pin because depending on when this gets called,
		// setting the pin value creates a duplicated node and the user has to delete it twice.
	}

	return EntityReference;
}

void SDMXEntityReferenceGraphPin::SetPinValue(const FDMXEntityReference& NewEntityRef) const
{
	FString ValueString;
	FDMXEntityReference::StaticStruct()->ExportText(ValueString, &NewEntityRef, nullptr, nullptr, EPropertyPortFlags::PPF_None, nullptr);
	GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, ValueString);
}

TWeakObjectPtr<UDMXEntity> SDMXEntityReferenceGraphPin::GetEntity() const
{
	return GetPinValue().GetEntity();
}

void SDMXEntityReferenceGraphPin::OnEntitySelected(UDMXEntity* NewEntity) const
{
	SetPinValue(NewEntity);
}

TSubclassOf<UDMXEntity> SDMXEntityReferenceGraphPin::GetEntityType() const
{
	return GetPinValue().GetEntityType();
}

TWeakObjectPtr<UDMXLibrary> SDMXEntityReferenceGraphPin::GetDMXLibrary() const
{
	return GetPinValue().DMXLibrary;
}

FLinearColor SDMXEntityReferenceGraphPin::GetComboForeground() const
{
	float Alpha = (IsHovered() || bOnlyShowDefaultValue) ? 1.f : 0.6f;
	return FLinearColor(1.f, 1.f, 1.f, Alpha);
}

TWeakObjectPtr<UObject> SDMXEntityReferenceGraphPin::GetCurrentDMXLibrary() const
{
	return GetPinValue().DMXLibrary;
}

void SDMXEntityReferenceGraphPin::OnDMXLibrarySelected(const FAssetData& InAssetData) const
{
	const FScopedTransaction Transaction(LOCTEXT("ChangeObjectPinValue", "Select DMX Library"));
	GraphPinObj->Modify();

	FDMXEntityReference&& EntityRef = GetPinValue();
	EntityRef.DMXLibrary = Cast<UDMXLibrary>(InAssetData.GetAsset());

	SetPinValue(EntityRef);
}

#undef LOCTEXT_NAMESPACE
