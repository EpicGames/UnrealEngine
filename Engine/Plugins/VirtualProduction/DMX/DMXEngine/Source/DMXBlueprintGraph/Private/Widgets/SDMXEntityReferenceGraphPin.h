// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SGraphPin.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Templates/SubclassOf.h"
#include "AssetRegistry/AssetData.h"

class UDMXLibrary;
class UDMXEntity;
struct FDMXEntityReference;

DECLARE_DELEGATE_OneParam(FOnEntitySelected, UDMXEntity*);

class DMXBLUEPRINTGRAPH_API SDMXEntityReferenceGraphPin
	: public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SDMXEntityReferenceGraphPin) {}
	SLATE_END_ARGS()

	/**  Slate widget construction method */
	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

	//~ Begin SGraphPin Interface
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface

private:
	FDMXEntityReference GetPinValue() const;
	void SetPinValue(const FDMXEntityReference& NewEntityRef) const;

	TWeakObjectPtr<UDMXEntity> GetEntity() const;
	void OnEntitySelected(UDMXEntity* NewEntity) const;
	TSubclassOf<UDMXEntity> GetEntityType() const;
	TWeakObjectPtr<UDMXLibrary> GetDMXLibrary() const;

	FLinearColor GetComboForeground() const;

	//~ Asset picker functions
	TWeakObjectPtr<UObject> GetCurrentDMXLibrary() const;
	void OnDMXLibrarySelected(const FAssetData& InAssetData) const;
};
