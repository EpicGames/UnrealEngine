// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXFixtureCategoryGraphPin.h"

#include "DMXProtocolTypes.h"
#include "Widgets/SNameListPicker.h"
#include "Widgets/SNullGraphPin.h"


void SDMXFixtureCategoryGraphPin::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget>	SDMXFixtureCategoryGraphPin::GetDefaultValueWidget()
{
	return SNew(SNameListPicker)
		.HasMultipleValues(false)
		.Value(this, &SDMXFixtureCategoryGraphPin::GetValue)
		.OnValueChanged(this, &SDMXFixtureCategoryGraphPin::SetValue)
		.OptionsSource(MakeAttributeLambda(&FDMXFixtureCategory::GetPredefinedValues))
		.bDisplayWarningIcon(true)
		.Visibility(this, &SGraphPin::GetDefaultValueVisibility);
}

FName SDMXFixtureCategoryGraphPin::GetValue() const
{
	FDMXFixtureCategory NameItem;

	if (!GraphPinObj->GetDefaultAsString().IsEmpty())
	{
		FDMXFixtureCategory::StaticStruct()->ImportText(*GraphPinObj->GetDefaultAsString(), &NameItem, nullptr, EPropertyPortFlags::PPF_None, GLog, FDMXFixtureCategory::StaticStruct()->GetName());
	}

	return NameItem.Name;
}

void SDMXFixtureCategoryGraphPin::SetValue(FName NewValue)
{
	FString ValueString;
	FDMXFixtureCategory NewNameItem(NewValue);
	FDMXFixtureCategory::StaticStruct()->ExportText(ValueString, &NewNameItem, nullptr, nullptr, EPropertyPortFlags::PPF_None, nullptr);
	GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, ValueString);
}
