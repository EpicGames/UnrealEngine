// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SNullGraphPin.h"

#include "Widgets/SNullWidget.h"

void SNullGraphPin::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget> SNullGraphPin::GetDefaultValueWidget()
{
	//Create widget
	return SNullWidget::NullWidget;
}