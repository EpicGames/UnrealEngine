// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SGraphPin.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

/**
 * Creates an empty widget for the pin, so the user can't edit the pin's default value.
 * Useful for UObject pointer pins that would otherwise display an asset picker.
 */
class DMXBLUEPRINTGRAPH_API SNullGraphPin
	: public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SNullGraphPin) {}
	SLATE_END_ARGS()

	/**  Slate widget construction method */
	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

	//~ Begin SGraphPin Interface
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface
};
