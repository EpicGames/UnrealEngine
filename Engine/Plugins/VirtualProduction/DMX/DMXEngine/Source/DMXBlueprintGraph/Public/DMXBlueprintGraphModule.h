// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"

class FDMXGraphPanelPinFactory;
class FDMXFixturePatchPinFactory;
struct FDMXFixtureMode;
class UDMXEntityFixtureType;

/**  The public interface to this module */
class FDMXBlueprintGraphModule : public IModuleInterface
{

public:

	//~ Begin IModuleInterface implementation
	virtual void StartupModule() override;

	virtual void ShutdownModule() override;

	//~ End IModuleInterface implementation

	static FDMXBlueprintGraphModule& Get()
	{
		return FModuleManager::LoadModuleChecked< FDMXBlueprintGraphModule >("DMXBlueprintGraphModule");
	}

	static bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "IDMXBlueprintGraphModule" );
	}

private:
	void RegisterObjectCustomizations();

	/**
	 * Registers a custom class
	 *
	 * @param ClassName				The class name to register for property customization
	 * @param DetailLayoutDelegate	The delegate to call to get the custom detail layout instance
	 */
	void RegisterCustomClassLayout(FName ClassName, FOnGetDetailCustomizationInstance DetailLayoutDelegate);


	/**
	 * Called when a fixture type changed
	 *
	 * @param	InFixtureType	Fixture type UObject
	 * @param	InMode			Changed mode
	 *
	 */
	void OnFixtureTypeChanged(const UDMXEntityFixtureType* InFixtureType);

	/** FDMXProtocolName and Custom nodes Graph Pin customizations */
	TSharedPtr<FDMXGraphPanelPinFactory> DMXGraphPanelPinFactory;

	/** List of registered class that we must unregister when the module shuts down */
	TSet< FName > RegisteredClassNames;
};

