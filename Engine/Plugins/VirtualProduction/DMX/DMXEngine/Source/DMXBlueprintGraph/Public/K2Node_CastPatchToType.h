// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "K2Node_EditablePinBase.h"
#include "K2Node_AddPinInterface.h"

#include "K2Node_CastPatchToType.generated.h"

class FBlueprintActionDatabaseRegistrar;
class UBlueprint;
class UEdGraph;
class UEdGraphPin;
class UDMXEntityFixturePatch;
class UDMXEntityFixtureType;
struct FDMXFixtureMode;
struct FDMXFixtureFunction;
struct FDMXEntityFixtureTypeRef;

/**
* k2Node that checks if a FixturePatch is of a given FixtureType, and if it succeed, lets you
* grab the function values from the patch
*/
UCLASS(Deprecated)
class DMXBLUEPRINTGRAPH_API UDEPRECATED_K2Node_CastPatchToType
	: public UK2Node_EditablePinBase
{
	GENERATED_BODY()

public:

	static const FName InputPinName_FixturePatch;
	static const FName InputPinName_FixtureTypeRef;

	static const FName OutputPinName_AttributesMap;

public:

	UPROPERTY()
	bool bIsExposed;

public:

	UDEPRECATED_K2Node_CastPatchToType();

	//~ Begin UObject interface
	virtual void Serialize(FArchive& Ar) override;
	//~ End UObject interface

	friend FArchive& operator <<(FArchive& Ar, FUserPinInfo& Info);

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	//~ End UEdGraphNode Interface

	//~ Begin K2Node Interface
	virtual bool IsNodePure() const override { return false; }
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual void NotifyPinConnectionListChanged(UEdGraphPin* Pin) override;
	//~ End K2Node Interface

	//~ Begin UK2Node_EditablePinBase Interface
	virtual UEdGraphPin* CreatePinFromUserDefinition(const TSharedPtr<FUserPinInfo> NewPinInfo) override;
	virtual bool ModifyUserDefinedPinDefaultValue(TSharedPtr<FUserPinInfo> PinInfo, const FString& NewDefaultValue) override;

	// We only override this to return true, otherwise we can't create the user defined pins for the Attributes
	virtual bool CanCreateUserDefinedPin(const FEdGraphPinType& InPinType, EEdGraphPinDirection InDesiredDirection, FText& OutErrorMessage) override { return true; }	
	//~ End UK2Node_EditablePinBase Interface

public:

	UFUNCTION()
	void ExposeAttributes();

	UFUNCTION()
	void ResetAttributes();

	UDMXEntityFixtureType* GetSelectedFixtureType();

	FString GetFixturePatchValueAsString() const;

	bool IsExposed() const { return bIsExposed; }
};
