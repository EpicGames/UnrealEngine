// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "K2Node_EditablePinBase.h"
#include "K2Node_AddPinInterface.h"

#include "K2Node_GetDMXAttributeValues.generated.h"

class FBlueprintActionDatabaseRegistrar;
class UBlueprint;
class UEdGraph;
class UEdGraphPin;
class UDMXEntityFixturePatch;
class UDMXEntityFixtureType;
struct FDMXFixtureMode;
struct FDMXFixtureFunction;

UCLASS()
class DMXBLUEPRINTGRAPH_API UK2Node_GetDMXAttributeValues
	: public UK2Node_EditablePinBase
{
	GENERATED_BODY()

public:
	UK2Node_GetDMXAttributeValues();

	//~ Begin UEdGraphNode Interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void AllocateDefaultPins() override;
	virtual void PostPasteNode() override;
	virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual bool ShouldShowNodeProperties() const override { return true; }
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	//~ End UK2Node Interface

	//~ Begin UK2Node_EditablePinBase Interface
	virtual bool ShouldUseConstRefParams() const override { return true; }
	virtual UEdGraphPin* CreatePinFromUserDefinition(const TSharedPtr<FUserPinInfo> NewPinInfo) override;
	virtual bool CanCreateUserDefinedPin(const FEdGraphPinType& InPinType, EEdGraphPinDirection InDesiredDirection, FText& OutErrorMessage) override;
	virtual bool ModifyUserDefinedPinDefaultValue(TSharedPtr<FUserPinInfo> PinInfo, const FString& NewDefaultValue) override;
	//~ End UK2Node_EditablePinBase Interface

	/** Triggers when FixturePatch from input pin has been changed */
	void OnFixturePatchChanged();

	void RemovePinsRecursive(UEdGraphPin* Pin);
	void RemoveOutputPin(UEdGraphPin* Pin);

	/**
	 * Called when the Fixture Type changed 
	 *
	 * @param	InFixtureType		The changed Fixture Type
	 *
	 */
	void OnFixtureTypeChanged(const UDMXEntityFixtureType* InFixtureType);

	/** DEPRECATED 5.0 */
	UE_DEPRECATED(5.0, "Replaced in favor of the more generic OnFixtureTypeChanged.")
	void OnDataTypeChanged(const UDMXEntityFixtureType* InFixtureType, const FDMXFixtureMode& InMode);

public:
	bool IsExposed() const { return bIsExposed; }

	UDMXEntityFixturePatch* GetFixturePatchFromPin() const;

	UEdGraphPin* GetInputDMXFixturePatchPin() const;
	UEdGraphPin* GetOutputAttributesMapPin() const;
	UEdGraphPin* GetOutputIsSuccessPin() const;

	/** Expose DMX function pins from fixture patch active mode */
	void ExposeAttributes();

	/** Removes DMX function node pins */
	void ResetAttributes();

private:
	/** Get pin name based on FixtureFunction input */
	FName GetPinName(const FDMXFixtureFunction& Function);

	/**  Get pinter to active mode, never cache this pointer and keep it dynamic */
	const FDMXFixtureMode* GetActiveFixtureMode() const;

	/** Get the fixture type from active fixture mode */
	UDMXEntityFixtureType* GetParentFixtureType() const;

public:
	static const FName InputDMXFixturePatchPinName;
	static const FName OutputAttributesMapPinName;
	static const FName OutputIsSuccessPinName;

public:
	UPROPERTY()
	bool bIsExposed;
};
