// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "K2Node.h"

#include "Library/DMXEntityReference.h"

#include "K2Node_GetDMXFixtureType.generated.h"

class FNodeHandlingFunctor;
class FKismetCompilerContext;
class UDMXEntityFixtureType;

UCLASS()
class DMXBLUEPRINTGRAPH_API UK2Node_GetDMXFixtureType
	: public UK2Node
{
	GENERATED_BODY()

public:
	//~ Begin UObject interface
	virtual void Serialize(FArchive& Ar) override;
	//~ End UObject interface

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	//~ End UEdGraphNode Interface

	//~ Begin K2Node Interface
	virtual bool IsNodePure() const override { return true; }
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	//~ End K2Node Interface

public:
	UEdGraphPin* GetInputDMXFixtureTypePin() const;
	UEdGraphPin* GetOutputDMXFixtureTypePin() const;

	/** Gets the DMX Fixture Type pin value as String */
	FString GetFixtureTypeValueAsString() const;
	/** Gets the DMX Fixture Type Reference from input pin */
	FDMXEntityFixtureTypeRef GetFixtureTypeRefFromPin() const;
	/** Updates values in the Fixture Type input pin struct */
	void SetInFixtureTypePinValue(const FDMXEntityFixtureTypeRef& InTypeRef) const;

public:
	static const FName InputDMXFixtureTypePinName;
	static const FName OutputDMXFixtureTypePinName;
};
