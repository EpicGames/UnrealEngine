// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DMXEditor : ModuleRules
{
	public DMXEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"DMXProtocol",
				"DMXProtocolEditor",
				"DMXRuntime",
				"Sequencer",
				"ToolMenus"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"AssetDefinition",
				"AssetRegistry",
				"AssetTools",
				"ContentBrowser",
				"CoreUObject",
				"DesktopPlatform",
				"DMXGDTF",
				"DMXZip",
				"EditorFramework",
				"EditorStyle",
				"EditorWidgets",
				"Engine",
				"InputCore",
				"Json",
				"JsonUtilities",
				"Kismet",
				"KismetWidgets",
				"Projects",
				"MainFrame",
				"MovieScene",
				"PropertyEditor",
				"SequencerCore",
				"Slate",
				"SlateCore",
				"TakesCore",
				"TakeRecorder",
				"TakeTrackRecorders",
				"ToolMenus",
				"ToolWidgets",
				"UnrealEd",
				"XmlParser",
			}
		);
	}
}
