// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EngineAnalytics.h"
#include "Misc/DateTime.h"
#include "UObject/NameTypes.h"


namespace UE::DMX
{
	/** Can be implemented in tools that want to provide engine analytics */
	class DMXEDITOR_API FDMXEditorToolAnalyticsProvider
	{
	public:
		FDMXEditorToolAnalyticsProvider(const FName& InToolName);
		virtual ~FDMXEditorToolAnalyticsProvider();

		/**
		 * Records a telemetry event.
		 *
		 * @param Name						The name of the specific event. Will be appended to the tool specific event name.
		 * @param Attributes				The attributes recorded with this event.
		 */
		void RecordEvent(const FName& Name, const TArray<FAnalyticsEventAttribute>& InAttributes);

	private:
		/** Records analytics for when the tool started. */
		void RecordToolStarted();

		/** Records analytics for when the tool ended. */
		void RecordToolEnded();

		/** The time when the tool was started */
		FDateTime ToolStartTimestamp;

		/** True after RecordToolEnded was called, either on engine pre exit or on destruction */
		bool bEnded = false;

		/** Name of the tool we are providing analytics for */
		const FName ToolName;

		/** Event name for DMX telemetry events */
		static const FString DMXToolEventName;
	};
}
