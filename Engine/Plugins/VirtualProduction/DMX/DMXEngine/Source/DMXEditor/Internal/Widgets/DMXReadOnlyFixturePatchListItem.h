// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "UObject/GCObject.h"

struct FDMXEntityFixturePatchRef;
class UDMXEntityFixturePatch;
class UDMXLibrary;


/** An item in the read only fixture patch list */
class FDMXReadOnlyFixturePatchListItem
	: public FGCObject
	, public TSharedFromThis<FDMXReadOnlyFixturePatchListItem>
{
public:
	FDMXReadOnlyFixturePatchListItem(const FDMXEntityFixturePatchRef& FixturePatchReference);

	/** Returns the fixture patch this item uses */
	UDMXEntityFixturePatch* GetFixturePatch() const { return FixturePatch; }

	/** Returns the dmx library this item uses */
	UDMXLibrary* GetDMXLibrary() const;

	/** Returns the name of the patch this item uses, as text */
	FText GetNameText() const;

	/** Returns universe and channel of the patch this item uses, as text */
	FText GetUniverseChannelText() const;

	/** Returns fixture ID of the patch this item uses, as text */
	FText GetFixtureIDText() const;

	/** Returns fixture type name of the patch this item uses, as text */
	FText GetFixtureTypeText() const;

	/** Returns mode name of the patch this item uses, as text */
	FText GetModeText() const;

	/** Returns the editor color of the patch this item uses */
	FSlateColor GetEditorColor() const;

protected:
	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;
	// End of FGCObject interface

private:
	/** The fixture patch this item uses */
	TObjectPtr<UDMXEntityFixturePatch> FixturePatch;
};
