// Copyright Epic Games, Inc. All Rights Reserved.

#include "Analytics/DMXEditorToolAnalyticsProvider.h"

#include "DMXEditorLog.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Timespan.h"


namespace UE::DMX
{
	const FString FDMXEditorToolAnalyticsProvider::DMXToolEventName = TEXT("Usage.DMX.ToolEvent");

	FDMXEditorToolAnalyticsProvider::FDMXEditorToolAnalyticsProvider(const FName& InToolName)
		: ToolName(InToolName)
	{
		RecordToolStarted();

		FCoreDelegates::OnEnginePreExit.AddRaw(this, &FDMXEditorToolAnalyticsProvider::RecordToolEnded);
	}

	FDMXEditorToolAnalyticsProvider::~FDMXEditorToolAnalyticsProvider()
	{
		RecordToolEnded();

		FCoreDelegates::OnEnginePreExit.RemoveAll(this);
	}

	void FDMXEditorToolAnalyticsProvider::RecordEvent(const FName& Name, const TArray<FAnalyticsEventAttribute>& InAttributes)
	{
		if (!FEngineAnalytics::IsAvailable())
		{
			return;
		}

		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Add(FAnalyticsEventAttribute(TEXT("ToolName"), ToolName));
		Attributes.Append(InAttributes);

		FEngineAnalytics::GetProvider().RecordEvent(DMXToolEventName, Attributes);
	}

	void FDMXEditorToolAnalyticsProvider::RecordToolStarted()
	{
		if (!FEngineAnalytics::IsAvailable())
		{
			return;
		}

		ToolStartTimestamp = FDateTime::UtcNow();

		const TArray<FAnalyticsEventAttribute> Attributes =
		{
			FAnalyticsEventAttribute(TEXT("ToolName"), ToolName)
		};

		FEngineAnalytics::GetProvider().RecordEvent(DMXToolEventName, Attributes);
	}

	void FDMXEditorToolAnalyticsProvider::RecordToolEnded()
	{
		if (!FEngineAnalytics::IsAvailable() || bEnded)
		{
			return;
		}

		const FDateTime Now = FDateTime::UtcNow();
		const FTimespan ToolUsageDuration = Now - ToolStartTimestamp;

		const TArray<FAnalyticsEventAttribute> Attributes =
		{
			FAnalyticsEventAttribute(TEXT("ToolName"), ToolName),
			FAnalyticsEventAttribute(TEXT("DurationSeconds"), static_cast<float>(ToolUsageDuration.GetTotalSeconds()))
		};

		FEngineAnalytics::GetProvider().RecordEvent(DMXToolEventName, Attributes);

		bEnded = true;
	}
}
