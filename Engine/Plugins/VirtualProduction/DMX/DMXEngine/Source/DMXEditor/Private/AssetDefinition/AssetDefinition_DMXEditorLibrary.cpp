// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_DMXEditorLibrary.h"

#include "DMXEditorModule.h"
#include "Library/DMXLibrary.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_DMXEditorLibrary"

FText UAssetDefinition_DMXEditorLibrary::GetAssetDisplayName() const
{
	return LOCTEXT("DMXEditorLibrary_AssetName", "DMX Library");
}

TSoftClassPtr<UObject> UAssetDefinition_DMXEditorLibrary::GetAssetClass() const
{
	return UDMXLibrary::StaticClass();
}

FLinearColor UAssetDefinition_DMXEditorLibrary::GetAssetColor() const
{
	return FColor(62, 140, 35);
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_DMXEditorLibrary::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] =
	{
		FAssetCategoryPath(FAssetCategoryPath(LOCTEXT("DMXEditorLibrary_AssetCategory", "Virtual Production")), LOCTEXT("DMXEditorLibrary_CategorySection", "DMX"), ECategoryMenuType::Section)
	};

	return Categories;
}

EAssetCommandResult UAssetDefinition_DMXEditorLibrary::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	TArray<UObject*> Objects = OpenArgs.LoadObjects<UObject>();
	TArray<UDMXLibrary*> LibrariesToOpen;
	for (UObject* Obj : Objects)
	{
		UDMXLibrary* Library = Cast<UDMXLibrary>(Obj);
		if (Library)
		{
			LibrariesToOpen.Add(Library);
		}
	}

	FDMXEditorModule& DMXEditorModule = FDMXEditorModule::Get();
	for (UDMXLibrary* Library : LibrariesToOpen)
	{
		DMXEditorModule.CreateEditor(EToolkitMode::Standalone, OpenArgs.ToolkitHost, Library);
	}
	
	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
