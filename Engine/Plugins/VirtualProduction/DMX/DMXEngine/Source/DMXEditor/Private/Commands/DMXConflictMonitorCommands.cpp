// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DMXConflictMonitorCommands.h"

#define LOCTEXT_NAMESPACE "DMXConflictMonitorCommands"

FDMXConflictMonitorCommands::FDMXConflictMonitorCommands()
	: TCommands<FDMXConflictMonitorCommands>(TEXT("DMXConflictMonitor"), LOCTEXT("DMXConflictMonitor", "DMX Conflict Monitor"), NAME_None, FAppStyle::GetAppStyleSetName())
{}

void FDMXConflictMonitorCommands::RegisterCommands()
{
	UI_COMMAND(StartScan, "Start Scan", "Scans DMX Output Ports for conflicts. Does not scan inputs.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PauseScan, "Pause Scan", "Pauses scanning for DMX conflicts.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ResumeScan, "Resume Scan", "Resumes scanning for DMX conflicts. ", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(StopScan, "Stop Scan", "Stops scanning for DMX conflicts.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ToggleAutoPause, "Auto Pause", "When active, the monitor pauses when a conflict was detected..", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(TogglePrintToLog, "Print to Log", "When active, prints conflicts to log. Only available when Auto Pause is enabled.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleRunWhenOpened, "Run when opened", "When enabled, the monitor automatically starts when opened..", EUserInterfaceActionType::ToggleButton, FInputChord());
}

#undef LOCTEXT_NAMESPACE
