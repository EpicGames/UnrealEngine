// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

class DMXEDITOR_API FDMXConflictMonitorCommands
	: public TCommands<FDMXConflictMonitorCommands>
{
public:
	FDMXConflictMonitorCommands();

	//~ Begin TCommands implementation
	virtual void RegisterCommands() override;
	//~ End TCommands implementation

	TSharedPtr<FUICommandInfo> StartScan;
	TSharedPtr<FUICommandInfo> PauseScan;
	TSharedPtr<FUICommandInfo> ResumeScan;
	TSharedPtr<FUICommandInfo> StopScan;

	TSharedPtr<FUICommandInfo> ToggleAutoPause;
	TSharedPtr<FUICommandInfo> TogglePrintToLog;
	TSharedPtr<FUICommandInfo> ToggleRunWhenOpened;
};
