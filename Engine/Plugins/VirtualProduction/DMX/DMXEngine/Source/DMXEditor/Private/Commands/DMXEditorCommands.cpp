// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DMXEditorCommands.h"

#define LOCTEXT_NAMESPACE "DMXEditorCommands"

FDMXEditorCommands::FDMXEditorCommands()
	: TCommands<FDMXEditorCommands>(TEXT("DMXEditor"), LOCTEXT("DMXEditor", "DMX Editor"), NAME_None, FAppStyle::GetAppStyleSetName())
{}

void FDMXEditorCommands::RegisterCommands()
{
	UI_COMMAND(GoToDocumentation, "View Documentation", "View documentation about DMX editor", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ImportDMXLibrary, "Import", "Import the DMX Library from an MVR File", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ExportDMXLibrary, "Export", "Exports the DMX Library to an MVR File", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(AddNewEntityFixtureType, "New Fixture Type", "Creates a new Fixture Type in this library", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddNewEntityFixturePatch, "Add Fixture", "Creates a new Fixture Patch in this library", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(AddNewFixtureTypeMode, "Add Mode", "Creates a new Mode in the Fixture Type", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddNewModeFunction, "Add Function", "Creates a new Function in the Mode", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(OpenChannelsMonitor, "Channel Monitor", "Open the Monitor for all DMX Channels in a Universe", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(OpenActivityMonitor, "Activity Monitor", "Open the Monitor for all DMX activity in a range of Universes", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(OpenConflictMonitor, "Conflict Monitor", "Open the Monitor to identify conflicting DMX being sent by the engine", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(OpenPatchTool, "Patch Tool", "Open the patch tool - Useful to patch many fixtures at once.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleReceiveDMX, "Receive DMX", "Sets whether DMX is received in editor", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Alt, EKeys::M));
	UI_COMMAND(ToggleSendDMX, "Send DMX", "Sets whether DMX is sent from editor", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Alt, EKeys::N));

	UI_COMMAND(Align, "Align", "Aligns selected patches", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(Stack, "Stack", "Stacks selected patches", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AutoAssignSelectedUniverse, "Auto-Assign in selected Universe", "Auto-assigns selected patches to first consecutive range of free channels, starting from the selected Universe.", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
