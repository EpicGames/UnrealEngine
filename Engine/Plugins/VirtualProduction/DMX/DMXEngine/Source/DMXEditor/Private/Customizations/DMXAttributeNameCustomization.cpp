// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXAttributeNameCustomization.h"

#include "DMXAttribute.h"
#include "DMXProtocolSettings.h"

#include "DetailWidgetRow.h"
#include "Editor.h"
#include "IPropertyTypeCustomization.h"
#include "IPropertyUtilities.h"
#include "ScopedTransaction.h"
#include "Widgets/SNameListPicker.h"


TSharedRef<IPropertyTypeCustomization> FDMXAttributeNameCustomization::MakeInstance()
{
	return MakeShared<FDMXAttributeNameCustomization>();
}

void FDMXAttributeNameCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	StructPropertyHandle = InPropertyHandle;
	check(CastFieldChecked<FStructProperty>(StructPropertyHandle->GetProperty())->Struct == FDMXAttributeName::StaticStruct());

	PropertyUtilities = CustomizationUtils.GetPropertyUtilities();

	NameHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDMXAttributeName, Name));

	UDMXProtocolSettings* ProtocolSettings = GetMutableDefault<UDMXProtocolSettings>();
	if (!ProtocolSettings)
	{
		return;
	}

	TArray<FName> Attributes;
	for (const FDMXAttribute& Attribute : ProtocolSettings->Attributes)
	{
		Attributes.Add(Attribute.Name);
	}

	InHeaderRow
		.IsEnabled(MakeAttributeLambda([this, InPropertyHandle] 
			{ 
				return !InPropertyHandle->IsEditConst() && PropertyUtilities->IsPropertyEditingEnabled();
			}))
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(125.0f)
		.MaxDesiredWidth(0.0f)
		[
			SNew(SNameListPicker)
			.Font(CustomizationUtils.GetRegularFont())
			.HasMultipleValues(this, &FDMXAttributeNameCustomization::HasMultipleValues)
			.OptionsSource(Attributes)
			.Value(this, &FDMXAttributeNameCustomization::GetValue)
			.bDisplayWarningIcon(true)
			.OnValueChanged(this, &FDMXAttributeNameCustomization::SetValue)
		];

	ProtocolSettings->GetOnDefaultAttributesChanged().AddSP(this, &FDMXAttributeNameCustomization::RequestRefresh);

	NameHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FDMXAttributeNameCustomization::RequestRefresh));
}

void FDMXAttributeNameCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{

}

FName FDMXAttributeNameCustomization::GetValue() const
{
	FName Name;
	NameHandle->GetValue(Name);
	
	return Name;
}

void FDMXAttributeNameCustomization::SetValue(FName NewValue)
{
	const FScopedTransaction SetAttributeValueTransaction(NSLOCTEXT("DMXAttributeNameCustomization", "SetAttributeValueTransaction", "Set Fixture Function Attribute Name"));

	NameHandle->SetValue(NewValue);
}

bool FDMXAttributeNameCustomization::HasMultipleValues() const
{
	TArray<const void*> RawData;
	StructPropertyHandle->AccessRawData(RawData);
	if (RawData.Num() == 1)
	{
		return false;
	}

	TOptional<FDMXAttributeName> CompareAgainst;
	for (const void* RawPtr : RawData)
	{
		if (RawPtr == nullptr)
		{
			if (CompareAgainst.IsSet())
			{
				return false;
			}
		}
		else
		{
			const FDMXAttributeName* ThisValue = reinterpret_cast<const FDMXAttributeName*>(RawPtr);

			if (!CompareAgainst.IsSet())
			{
				CompareAgainst = *ThisValue;
			}
			else if (!(*ThisValue == CompareAgainst.GetValue()))
			{
				return true;
			}
		}
	}

	return false;
}

void FDMXAttributeNameCustomization::RequestRefresh()
{
	if (!RefreshHandle.IsValid())
	{
		RefreshHandle = GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateSP(this, &FDMXAttributeNameCustomization::Refresh));
	}
}

void FDMXAttributeNameCustomization::Refresh()
{
	RefreshHandle.Invalidate();
	PropertyUtilities->ForceRefresh();
}
