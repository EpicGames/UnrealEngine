// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolTypes.h"

#include "CoreMinimal.h"
#include "DetailWidgetRow.h"
#include "Engine/EngineTypes.h"
#include "IPropertyTypeCustomization.h"
#include "IPropertyUtilities.h"
#include "Widgets/SNameListPicker.h"

class IPropertyUtilities;


/**  
 * Customization for FDMXAttributeName struct
 */
class FDMXAttributeNameCustomization
	: public IPropertyTypeCustomization
{
public:
	/** Creates an instance of this property type customization */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	//~ IPropertyTypeCustomization interface begin
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ IPropertyTypeCustomization interface end

private:
	/** Returns the value as FName */
	FName GetValue() const;

	/** Sets the value */
	void SetValue(FName NewValue);

	/** True if the customization displays multiple values */
	bool HasMultipleValues() const;

	/** Requests a refresh of the customization */
	void RequestRefresh();

	/** Refreshes instantly */
	void Refresh();

	/** Property handle for the Name property */
	TSharedPtr<IPropertyHandle> NameHandle;

	/** Handle for the FDMXAttributeName struct self */
	TSharedPtr<IPropertyHandle> StructPropertyHandle;

	/** Timer handle set when refresh is requested but not carried out yet */
	FTimerHandle RefreshHandle;

	/** Property utlities for this customization */
	TSharedPtr<IPropertyUtilities> PropertyUtilities;
};
