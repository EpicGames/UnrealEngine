// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXAutoExpandedStructCustomization.h"

#include "DetailWidgetRow.h"
#include "IDetailPropertyRow.h"
#include "IDetailChildrenBuilder.h"


TSharedRef<IPropertyTypeCustomization> FDMXAutoExpandedStructCustomization::MakeInstance()
{
	return MakeShared<FDMXAutoExpandedStructCustomization>();
}

void FDMXAutoExpandedStructCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	constexpr TCHAR ShowOnlyInnerPropertiesMetaDataName[] = TEXT("ShowOnlyInnerProperties");
	bool bShowHeader = !InPropertyHandle->HasMetaData(ShowOnlyInnerPropertiesMetaDataName);
	if (bShowHeader)
	{
		InHeaderRow
			.ShouldAutoExpand(true)
			.NameContent()
			[
				InPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				InPropertyHandle->CreatePropertyValueWidget()
			];
	}
}

void FDMXAutoExpandedStructCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	uint32 NumChildren;
	if (InPropertyHandle->GetNumChildren(NumChildren) != FPropertyAccess::Success)
	{
		return;
	}

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		const TSharedRef<IPropertyHandle> ChildHandle = InPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		InChildBuilder.AddProperty(ChildHandle).ShouldAutoExpand(true);
	}
}
