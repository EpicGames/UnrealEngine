// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

/**
 * Customization for DMX auto expanded structs
 */
class FDMXAutoExpandedStructCustomization
	: public IPropertyTypeCustomization
{
public:
	/** Creates an instance of the property type customization */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	//~ IPropertyTypeCustomization interface begin
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ IPropertyTypeCustomization interface end
};
