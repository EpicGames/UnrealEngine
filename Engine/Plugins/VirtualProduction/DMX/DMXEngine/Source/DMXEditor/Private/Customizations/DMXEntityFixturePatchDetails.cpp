// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/DMXEntityFixturePatchDetails.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DMXEditorUtils.h"
#include "DMXFixturePatchSharedData.h"
#include "IPropertyUtilities.h"
#include "Library/DMXEntityFixturePatch.h"
#include "ScopedTransaction.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SDMXEntityDropdownMenu.h"

#define LOCTEXT_NAMESPACE "DMXEntityFixturePatchFixtureSettingsDetails"

FDMXEntityFixturePatchDetails::FDMXEntityFixturePatchDetails(TWeakPtr<FDMXEditor> InDMXEditorPtr)
	: DMXEditorPtr(InDMXEditorPtr)
{}

TSharedRef<IDetailCustomization> FDMXEntityFixturePatchDetails::MakeInstance(TWeakPtr<FDMXEditor> InDMXEditorPtr)
{
	return MakeShared<FDMXEntityFixturePatchDetails>(InDMXEditorPtr);
}

void FDMXEntityFixturePatchDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	PropertyUtilities = DetailBuilder.GetPropertyUtilities();

	ParentFixtureTypeHandle = DetailBuilder.GetProperty(UDMXEntityFixturePatch::GetParentFixtureTypeTemplatePropertyNameChecked());
	ActiveModeHandle = DetailBuilder.GetProperty(UDMXEntityFixturePatch::GetActiveModePropertyNameChecked());

	// Handle Property changes
	UniverseIDHandle = DetailBuilder.GetProperty(UDMXEntityFixturePatch::GetUniverseIDPropertyNameChecked());
	UniverseIDHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FDMXEntityFixturePatchDetails::OnUniverseIDChanged));

	// Handle Fixture Type changes
	UDMXEntityFixtureType::GetOnFixtureTypeChanged().AddSP(this, &FDMXEntityFixturePatchDetails::OnFixtureTypeChanged);

	// Make a Fixture Types dropdown for the Fixture Type template property
	DetailBuilder.EditDefaultProperty(ParentFixtureTypeHandle)->CustomWidget(false)
		.NameContent()
		[
			ParentFixtureTypeHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(200.0f)
		.MaxDesiredWidth(400.0f)
		[
			SNew(SDMXEntityPickerButton<UDMXEntityFixtureType>)
			.DMXEditor(DMXEditorPtr)
			.CurrentEntity(this, &FDMXEntityFixturePatchDetails::GetParentFixtureType)
			.OnEntitySelected(this, &FDMXEntityFixturePatchDetails::OnParentFixtureTypeChanged)
			.HasMultipleValues(this, &FDMXEntityFixturePatchDetails::IsParentFixtureTypeMultipleValues)
		];

	// Make a modes dropdown to select the active Fixture Type Mode, if a valid Fixture Type is selected
	TSharedPtr<uint32> DefaultSelectedActiveMode = nullptr;
	GenerateActiveModesSource();

	DetailBuilder.EditDefaultProperty(ActiveModeHandle)->CustomWidget(false)
		.NameContent()
		[
			ActiveModeHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SAssignNew(ActiveModeComboBox, SComboBox<TSharedPtr<uint32>>)
			.IsEnabled(this, &FDMXEntityFixturePatchDetails::IsActiveModeEditable)
			.OptionsSource(&ActiveModesSource)
			.OnGenerateWidget(this, &FDMXEntityFixturePatchDetails::GenerateActiveModeWidget)
			.OnSelectionChanged(this, &FDMXEntityFixturePatchDetails::OnActiveModeChanged)
			.InitiallySelectedItem(DefaultSelectedActiveMode)
			[
				SNew(STextBlock)
				.MinDesiredWidth(50.0f)
				.Text(this, &FDMXEntityFixturePatchDetails::GetCurrentActiveModeLabel)
				.Font(DetailBuilder.GetDetailFont())
			]
		];
}

TSharedRef<SWidget> FDMXEntityFixturePatchDetails::GenerateActiveModeWidget(const TSharedPtr<uint32> InMode) const
{
	UObject* Object = nullptr;

	if (ParentFixtureTypeHandle->GetValue(Object) == FPropertyAccess::Success && Object != nullptr)
	{
		if (UDMXEntityFixtureType* Patch = Cast<UDMXEntityFixtureType>(Object))
		{
			if (InMode.IsValid() && 
				Patch->Modes.IsValidIndex(*InMode))
			{
				return 
					SNew(STextBlock)
					.Text(FText::FromString(Patch->Modes[*InMode].ModeName));
			}
		}
	}

	return SNullWidget::NullWidget;
}

void FDMXEntityFixturePatchDetails::OnParentFixtureTypeChanged(UDMXEntity* NewTemplate) const
{
	ParentFixtureTypeHandle->SetValue(Cast<UDMXEntityFixtureType>(NewTemplate));
	PropertyUtilities->RequestRefresh();
}

void FDMXEntityFixturePatchDetails::OnFixtureTypeChanged(const UDMXEntityFixtureType* FixtureType)
{
	GenerateActiveModesSource();
	ActiveModeComboBox->RefreshOptions();

	PropertyUtilities->RequestRefresh();
}

void FDMXEntityFixturePatchDetails::OnUniverseIDChanged()
{
	const TSharedPtr<FDMXEditor> DMXEditor = DMXEditorPtr.Pin();
	if (!DMXEditor.IsValid())
	{
		return;
	}

	int32 UniverseID;
	if (UniverseIDHandle->GetValue(UniverseID) == FPropertyAccess::Success)
	{
		DMXEditor->GetFixturePatchSharedData()->SelectUniverse(UniverseID);
	}
}

void FDMXEntityFixturePatchDetails::OnActiveModeChanged(const TSharedPtr<uint32> InSelectedMode, ESelectInfo::Type SelectInfo)
{
	if (InSelectedMode.IsValid())
	{
		ActiveModeHandle->SetValue(*InSelectedMode);
	}
}

void FDMXEntityFixturePatchDetails::GenerateActiveModesSource()
{
	ActiveModesSource.Reset();

	UObject* Object = nullptr;
	if (ParentFixtureTypeHandle->GetValue(Object) == FPropertyAccess::Success)
	{
		if (UDMXEntityFixtureType* Fixture = Cast<UDMXEntityFixtureType>(Object))
		{
			const uint32 NumModes = Fixture->Modes.Num();
			for (uint32 ModeIndex = 0; ModeIndex < NumModes; ++ModeIndex)
			{
				ActiveModesSource.Add(MakeShared<uint32>(ModeIndex));
			}
		}
	}
}

TWeakObjectPtr<UDMXEntityFixtureType> FDMXEntityFixturePatchDetails::GetParentFixtureType() const
{
	UObject* Object;
	if (ParentFixtureTypeHandle->GetValue(Object) == FPropertyAccess::Success)
	{
		return Cast<UDMXEntityFixtureType>(Object);
	}
	return nullptr;
}

bool FDMXEntityFixturePatchDetails::IsParentFixtureTypeMultipleValues() const
{
	UObject* Object;
	return ParentFixtureTypeHandle->GetValue(Object) == FPropertyAccess::MultipleValues;
}

bool FDMXEntityFixturePatchDetails::IsActiveModeEditable() const
{
	UObject* Object = nullptr;
	if (ParentFixtureTypeHandle->GetValue(Object) == FPropertyAccess::Success && Object)
	{
		if (UDMXEntityFixtureType* Fixture = Cast<UDMXEntityFixtureType>(Object))
		{
			return Fixture->Modes.Num() > 0;
		}

		return false;
	}

	return false;
}

FText FDMXEntityFixturePatchDetails::GetCurrentActiveModeLabel() const
{
	static const FText MultipleValuesLabel = LOCTEXT("MultipleValuesLabel", "Multiple Values");
	static const FText NullTypeLabel = LOCTEXT("NullFixtureTypeLabel", "No Fixture Type selected");
	static const FText MultipleTypesLabel = LOCTEXT("MultipleFixtureTypesLabel", "Multiple Fixture Types Selected");
	static const FText NoModesLabel = LOCTEXT("NoModesLabel", "No Modes in Fixture Type");

	UObject* Object = nullptr;
	const FPropertyAccess::Result FixtureTemplateAccessResult = ParentFixtureTypeHandle->GetValue(Object);
	UDMXEntityFixtureType* FixtureTemplate = Cast<UDMXEntityFixtureType>(Object);

	// Is only one type of Fixture Type selected?
	if (FixtureTemplateAccessResult == FPropertyAccess::Success)
	{
		// Is this type valid?
		if (FixtureTemplate != nullptr)
		{
			// We can try to get the mode, although it could be a different one for each of the templates
			int32 ModeValue = 0;
			if (ActiveModeHandle->GetValue(ModeValue) == FPropertyAccess::Success)
			{
				const TArray<FDMXFixtureMode>& CurrentModes = FixtureTemplate->Modes;
				if (CurrentModes.Num() > 0 && CurrentModes.IsValidIndex(ModeValue))
				{
					return FText::FromString(
						CurrentModes[ModeValue].ModeName
					);
				}
				else
				{
					return NoModesLabel;
				}
			}

			return MultipleValuesLabel;
		}

		return NullTypeLabel;
	}

	return MultipleTypesLabel;
}

void FDMXEntityFixturePatchDetails::SetActiveMode(int32 ModeIndex)
{
	TArray<UObject*> OuterObjects;
	ActiveModeHandle->GetOuterObjects(OuterObjects);

	if (OuterObjects.Num() > 0)
	{
		for (UObject* Object : OuterObjects)
		{
			UDMXEntityFixturePatch* Patch = CastChecked<UDMXEntityFixturePatch>(Object);
			if (Patch->GetActiveModeIndex() == ModeIndex)
			{
				continue;
			}
			
			Patch->Modify();
			Patch->PreEditChange(UDMXEntityFixturePatch::StaticClass()->FindPropertyByName(UDMXEntityFixturePatch::GetActiveModePropertyNameChecked()));

			Patch->SetActiveModeIndex(ModeIndex);

			Patch->PostEditChange();
		}
	}
}

#undef LOCTEXT_NAMESPACE
