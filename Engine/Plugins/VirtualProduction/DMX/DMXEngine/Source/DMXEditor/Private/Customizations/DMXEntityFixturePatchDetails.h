// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "Types/SlateEnums.h"

struct FDMXFixtureMode;
class FDMXEditor;
class UDMXEntity;
class UDMXEntityFixtureType;

class IPropertyHandle;
class IPropertyUtilities;
template<typename OptionType> class SComboBox;
class SWidget;


/** Details customization for the DMMXFixturePatch class */
class FDMXEntityFixturePatchDetails
	: public IDetailCustomization
{
public:
	/** Constructor */
	FDMXEntityFixturePatchDetails(TWeakPtr<FDMXEditor> InDMXEditorPtr);
	
	/** Creates a detail customization instance */
	static TSharedRef<IDetailCustomization> MakeInstance(TWeakPtr<FDMXEditor> InDMXEditorPtr);

protected:
	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization interface

private:
	/** Generates the customized ActiveMode property widget */
	TSharedRef<SWidget> GenerateActiveModeWidget(const TSharedPtr<uint32> InMode) const;

	/** Called when the ParentFixtureType property changed */
	void OnParentFixtureTypeChanged(UDMXEntity* NewTemplate) const;

	/** Called when a Mode in the Modes Array property changed */
	void OnFixtureTypeChanged(const UDMXEntityFixtureType* FixtureType);

	/** Called when the UniverseID property changed */
	void OnUniverseIDChanged();

	/** Called when the ActiveMode property changed */
	void OnActiveModeChanged(const TSharedPtr<uint32> InSelectedMode, ESelectInfo::Type SelectInfo);

	/** Generates the Array of available mode in the ActiveModeSource member array */
	void GenerateActiveModesSource();

	/** Returns the Fixture Type the patch uses */
	TWeakObjectPtr<UDMXEntityFixtureType> GetParentFixtureType() const;
	
	/** Returns true if the ParentFixtureType property has multiple values */
	bool IsParentFixtureTypeMultipleValues() const;

	/** Returns true if the ActiveMode property is editable */
	bool IsActiveModeEditable() const;

	/** Returns a label for the currently active mode */
	FText GetCurrentActiveModeLabel() const;

	/** Sets the ActiveMode to specified value */
	void SetActiveMode(int32 ModeIndex);

	/** Handle for the ParentFixtureType property */
	TSharedPtr<IPropertyHandle> ParentFixtureTypeHandle;

	/** Handle for the UniverseID property */
	TSharedPtr<IPropertyHandle> UniverseIDHandle;

	/** Handle for the ActiveMode property */
	TSharedPtr<IPropertyHandle> ActiveModeHandle;

	/** ComboBox that displays the ActiveMode property */
	TSharedPtr<SComboBox<TSharedPtr<uint32>>> ActiveModeComboBox;

	/** Array of available mode indicies */
	TArray<TSharedPtr<uint32>> ActiveModesSource;

	/** The property utilities for the customization */
	TSharedPtr<IPropertyUtilities> PropertyUtilities;

	/** Weak reference to the DMX editor */
	TWeakPtr<FDMXEditor> DMXEditorPtr;
};
