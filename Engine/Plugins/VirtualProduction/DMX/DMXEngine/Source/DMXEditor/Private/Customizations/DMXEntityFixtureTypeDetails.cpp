// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/DMXEntityFixtureTypeDetails.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Factories/DMXGDTFToFixtureTypeConverter.h"
#include "IPropertyUtilities.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/DMXImportGDTF.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "DMXEntityFixtureTypeDetails"

TSharedRef<IDetailCustomization> FDMXEntityFixtureTypeDetails::MakeInstance()
{
	return MakeShared<FDMXEntityFixtureTypeDetails>();
}

void FDMXEntityFixtureTypeDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	PropertyUtilities = DetailBuilder.GetPropertyUtilities();

	// Hide the Modes property, it has its own view in the editor
	{
		DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UDMXEntityFixtureType, Modes));
	}

	// Handle property changes for the GDTFSource property
	{
		GDTFSourceHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDMXEntityFixtureType, GDTFSource));
		GDTFSourceHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FDMXEntityFixtureTypeDetails::OnGDTFSourceChanged));
	}

	// Customize the bExportGeneratedGDTF property
	{
		const TSharedRef<IPropertyHandle> ExportGeneratedGDTFHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDMXEntityFixtureType, bExportGeneratedGDTF));
		const FName GDTFCategory = ExportGeneratedGDTFHandle->GetDefaultCategoryName();
		DetailBuilder.EditCategory(GDTFCategory)
			.AddProperty(ExportGeneratedGDTFHandle, EPropertyLocation::Advanced)
			.Visibility(TAttribute<EVisibility>::CreateSP(this, &FDMXEntityFixtureTypeDetails::GetGDTFOptionsVisibility));
	}

	// Add the "Reset to GDTF" button
	{
		const FName GDTFCategoryName = GDTFSourceHandle->GetDefaultCategoryName();
		const FText EmptyFilterText = FText::GetEmpty();

		IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(GDTFCategoryName);

		TArray<TSharedRef<IPropertyHandle>> DefaultProperties;
		CategoryBuilder.GetDefaultProperties(DefaultProperties);

		for (const TSharedRef<IPropertyHandle>& DefaultProperty : DefaultProperties)
		{
			CategoryBuilder.AddProperty(DefaultProperty);
		}

		CategoryBuilder
			.AddCustomRow(EmptyFilterText)
			.WholeRowContent()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Text(LOCTEXT("ResetToGDTFLabel", "Reset To GDTF"))
					.ToolTipText(LOCTEXT("ResetToGDTFTooltip", "Resets the Fixture Type to the state defined in the GDTF"))
					.OnClicked(this, &FDMXEntityFixtureTypeDetails::OnResetToGDTFClicked)
				]
			]
		.Visibility(TAttribute<EVisibility>::CreateSP(this, &FDMXEntityFixtureTypeDetails::GetGDTFOptionsVisibility));
	}
}

void FDMXEntityFixtureTypeDetails::OnGDTFSourceChanged()
{
	using namespace UE::DMX::GDTF;

	TArray<void*> RawDataArray;
	GDTFSourceHandle->AccessRawData(RawDataArray);

	TSoftObjectPtr<UDMXImportGDTF>* GDTFAssetPtr = nullptr;
	if (!RawDataArray.IsEmpty())
	{
		GDTFAssetPtr = reinterpret_cast<TSoftObjectPtr<UDMXImportGDTF>*>(RawDataArray[0]);
	}

	const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = PropertyUtilities->GetSelectedObjects();
	if (GDTFAssetPtr)
	{
		if (UDMXImportGDTF* GDTF = GDTFAssetPtr->LoadSynchronous())
		{
			for (const TWeakObjectPtr<UObject>& WeakFixtureTypeObject : SelectedObjects)
			{
				if (UDMXEntityFixtureType* FixtureType = Cast<UDMXEntityFixtureType>(WeakFixtureTypeObject.Get()))
				{
					FixtureType->PreEditChange(nullptr);

					// Generate GDTF
					constexpr bool bUpdateFixtureTypeName = true;
					FDMXGDTFToFixtureTypeConverter::ConvertGDTF(*FixtureType, *GDTF, bUpdateFixtureTypeName);

					// Set Actor Class to Spawn
					FixtureType->ActorClassToSpawn = GDTF->GetActorClass();

					FixtureType->PostEditChange();
				}
			}
		}
		else
		{
			for (const TWeakObjectPtr<UObject>& WeakFixtureTypeObject : SelectedObjects)
			{
				if (UDMXEntityFixtureType* FixtureType = Cast<UDMXEntityFixtureType>(WeakFixtureTypeObject.Get()))
				{
					FixtureType->PreEditChange(nullptr);

					// Reset GDTF
					FixtureType->Modes.Reset();

					// Reset Actor Class to Spawn
					FixtureType->ActorClassToSpawn.Reset();

					FixtureType->PostEditChange();
				}
			}
		}
	}
}

FReply FDMXEntityFixtureTypeDetails::OnResetToGDTFClicked()
{
	const FScopedTransaction ResetToGDTFTransaction(LOCTEXT("ResetToGDTFTransaction", "Reset To GDTF"));

	// Run as if the source changed
	OnGDTFSourceChanged();

	return FReply::Handled();
}

EVisibility FDMXEntityFixtureTypeDetails::GetGDTFOptionsVisibility() const
{
	TArray<const void*> RawDataArray;
	GDTFSourceHandle->AccessRawData(RawDataArray);

	if (!RawDataArray.IsEmpty())
	{
		const TSoftObjectPtr<UDMXImportGDTF>* GDTFAssetPtr = reinterpret_cast<const TSoftObjectPtr<UDMXImportGDTF>*>(RawDataArray[0]);

		const bool bHasGDTF = GDTFAssetPtr && GDTFAssetPtr->IsValid();
		return bHasGDTF ? EVisibility::Visible : EVisibility::Collapsed;
	}

	return EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE
