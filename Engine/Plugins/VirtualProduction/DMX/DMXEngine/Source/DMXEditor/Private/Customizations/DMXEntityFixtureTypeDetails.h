// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"

class IPropertyHandle;
class IPropertyUtilities;


/** Details customization for the 'FixtureType FunctionProperties' details view */
class FDMXEntityFixtureTypeDetails
	: public IDetailCustomization
{
public:
	/** Creates an instance of this details customization */
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization interface

private:
	/** Called when the GDTFSource property changed */
	void OnGDTFSourceChanged();

	/** Called when the Reset To GDTF button was clicked */
	FReply OnResetToGDTFClicked();

	/** Returns the visibility of the GDTF options, they're only visible if there is a GDTF */
	EVisibility GetGDTFOptionsVisibility() const;

	/** Handle to the GDTFSource property */
	TSharedPtr<IPropertyHandle> GDTFSourceHandle;

	/** Property utilities for this customization */
	TSharedPtr<IPropertyUtilities> PropertyUtilities;
};
