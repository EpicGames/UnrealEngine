// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/DMXEntityReferenceCustomization.h"

#include "Library/DMXEntity.h"
#include "Library/DMXEntityReference.h"
#include "Library/DMXLibrary.h"
#include "Widgets/SDMXEntityDropdownMenu.h"

#include "DetailWidgetRow.h"
#include "IDetailPropertyRow.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "DMXEntityReferenceCustomization"

TSharedRef<IPropertyTypeCustomization> FDMXEntityReferenceCustomization::MakeInstance()
{
	return MakeShared<FDMXEntityReferenceCustomization>();
}

void FDMXEntityReferenceCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	StructHandle = InPropertyHandle;

	constexpr TCHAR ShowOnlyInnerPropertiesMetaDataName[] = TEXT("ShowOnlyInnerProperties");
	bool bShowHeader = !InPropertyHandle->HasMetaData(ShowOnlyInnerPropertiesMetaDataName);
	if (bShowHeader)
	{
		// If we should display any children, leave the standard header. Otherwise, create custom picker
		TSharedRef<SWidget> ValueContent = !ShouldDisplayLibrary()
			? CreateEntityPickerWidget(InPropertyHandle)
			: InPropertyHandle->CreatePropertyValueWidget(false);

		InHeaderRow
			.ShouldAutoExpand(true)
			.NameContent()
			[
				InPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MinDesiredWidth(200.0f)
			.MaxDesiredWidth(400.0f)
			[
				ValueContent
			];
	}
}

void FDMXEntityReferenceCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	if (!ShouldDisplayLibrary())
	{
		// Don't add any child properties
		return;
	}

	// Retrieve structure's child properties
	uint32 NumChildren;
	InPropertyHandle->GetNumChildren(NumChildren);
	TMap<FName, TSharedPtr< IPropertyHandle > > PropertyHandles;
	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = InPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		const FName PropertyName = ChildHandle->GetProperty()->GetFName();
		PropertyHandles.Add(PropertyName, ChildHandle);
	}

	// Add the properties
	for (const TPair<FName, TSharedPtr<IPropertyHandle>>& PropertyHandlePair : PropertyHandles)
	{
		if ((PropertyHandlePair.Key == GET_MEMBER_NAME_CHECKED(FDMXEntityReference, DMXLibrary) && ShouldDisplayLibrary())
			|| (PropertyHandlePair.Key != GET_MEMBER_NAME_CHECKED(FDMXEntityReference, DMXLibrary) && PropertyHandlePair.Key != "EntityId")) // EntityId is private, so GET_MEMBER... won't work
		{
			InChildBuilder.AddProperty(PropertyHandlePair.Value.ToSharedRef());
		}
	}

	// Add the picker
	InChildBuilder.AddCustomRow(LOCTEXT("EntityReferencePickerSearchText", "Entity"))
		.NameContent()
		[
			SNew(STextBlock)
			.Font(CustomizationUtils.GetRegularFont())
			.Text(this, &FDMXEntityReferenceCustomization::GetPickerPropertyLabel)
		]
		.ValueContent()
		.MinDesiredWidth(200.0f)
		.MaxDesiredWidth(400.0f)
		[
			CreateEntityPickerWidget(InPropertyHandle)
		];
}

bool FDMXEntityReferenceCustomization::ShouldDisplayLibrary() const
{
	TArray<const void*> RawDataArr;
	StructHandle->AccessRawData(RawDataArr);

	for (const void* RawData : RawDataArr)
	{
		const FDMXEntityReference* EntityRefPtr = reinterpret_cast<const FDMXEntityReference*>(RawData);
		if (!EntityRefPtr->bDisplayLibraryPicker)
		{
			return false;
		}
	}

	return true;
}

TSharedRef<SWidget> FDMXEntityReferenceCustomization::CreateEntityPickerWidget(TSharedPtr<IPropertyHandle> InPropertyHandle) const
{
	return SNew(SDMXEntityPickerButton<UDMXEntity>)
		.CurrentEntity(this, &FDMXEntityReferenceCustomization::GetCurrentEntity)
		.HasMultipleValues(this, &FDMXEntityReferenceCustomization::GetEntityIsMultipleValues)
		.OnEntitySelected(this, &FDMXEntityReferenceCustomization::OnEntitySelected)
		.EntityTypeFilter(this, &FDMXEntityReferenceCustomization::GetEntityType)
		.DMXLibrary(this, &FDMXEntityReferenceCustomization::GetDMXLibrary)
		.IsEnabled(this, &FDMXEntityReferenceCustomization::GetPickerEnabled);
}

FText FDMXEntityReferenceCustomization::GetPickerPropertyLabel() const
{
	if (TSubclassOf<UDMXEntity> EntityType = GetEntityType())
	{
		return FDMXEditorUtils::GetEntityTypeNameText(EntityType, false);
	}

	return LOCTEXT("GenericTypeEntityLabel", "Entity");
}

bool FDMXEntityReferenceCustomization::GetPickerEnabled() const
{
	return  
		StructHandle->IsEditable() && 
		GetEntityType() != nullptr;
}

TWeakObjectPtr<UDMXEntity> FDMXEntityReferenceCustomization::GetCurrentEntity() const
{
	if (GetEntityIsMultipleValues()) return nullptr;

	void* StructPtr = nullptr;
	if (StructHandle->GetValueData(StructPtr) == FPropertyAccess::Success && StructPtr != nullptr)
	{
		const FDMXEntityReference* EntityRef = reinterpret_cast<FDMXEntityReference*>(StructPtr);
		return EntityRef->GetEntity();
	}

	return nullptr;
}

bool FDMXEntityReferenceCustomization::GetEntityIsMultipleValues() const
{
	TArray<void*> RawData;
	StructHandle->AccessRawData(RawData);
	if (RawData.IsEmpty() || RawData[0] == nullptr)
	{
		return true;
	}

	bool bFirstEntitySet = false;
	UDMXEntity* FirstEntityPtr = nullptr;

	for (const void* StructPtr : RawData)
	{
		const FDMXEntityReference* EntityRefPtr = reinterpret_cast<const FDMXEntityReference*>(StructPtr);

		if (bFirstEntitySet)
		{
			if (EntityRefPtr->GetEntity() != FirstEntityPtr)
			{
				return true;
			}
		}
		else
		{
			FirstEntityPtr = EntityRefPtr->GetEntity();
			bFirstEntitySet = true;
		}
	}

	return false;
}

void FDMXEntityReferenceCustomization::OnEntitySelected(UDMXEntity* NewEntity) const
{
	FDMXEntityReference NewStructValues;
	NewStructValues.SetEntity(NewEntity);

	// Export new values to text format that can be imported later into the actual struct properties
	FString TextValue;
	FDMXEntityReference::StaticStruct()->ExportText(TextValue, &NewStructValues, nullptr, nullptr, EPropertyPortFlags::PPF_None, nullptr);

	// Set values on edited property handle from exported text
	ensure(StructHandle->SetValueFromFormattedString(TextValue, EPropertyValueSetFlags::DefaultFlags) == FPropertyAccess::Success);
}

TSubclassOf<UDMXEntity> FDMXEntityReferenceCustomization::GetEntityType() const
{
	TArray<void*> RawData;
	StructHandle->AccessRawData(RawData);
	if (RawData[0] == nullptr)
	{
		return nullptr;
	}

	const TSubclassOf<UDMXEntity> FirstEntityType = reinterpret_cast<FDMXEntityReference*>(RawData[0])->GetEntityType();

	for (const void* StructPtr : RawData)
	{
		const FDMXEntityReference* EntityRefPtr = reinterpret_cast<const FDMXEntityReference*>(StructPtr);
		if (EntityRefPtr->GetEntityType() != FirstEntityType)
		{
			// Different types are selected
			return nullptr;
		}
	}

	return FirstEntityType;
}

TWeakObjectPtr<UDMXLibrary> FDMXEntityReferenceCustomization::GetDMXLibrary() const
{
	TSharedPtr<IPropertyHandle> LibraryHandle = StructHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDMXEntityReference, DMXLibrary));
	UObject* Object = nullptr;
	if (LibraryHandle.IsValid() && LibraryHandle->GetValue(Object) == FPropertyAccess::Success)
	{
		return Cast<UDMXLibrary>(Object);
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
