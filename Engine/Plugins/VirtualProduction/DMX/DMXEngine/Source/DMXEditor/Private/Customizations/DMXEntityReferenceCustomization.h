// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "CoreMinimal.h"

#include "DMXProtocolTypes.h"

class SWidget;
class UDMXEntity;
class UDMXLibrary;

/**
 * Customization for Entity Reference structs
 */
class FDMXEntityReferenceCustomization
	: public IPropertyTypeCustomization
{
public:
	/** Creates an instance of the property type customization */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	//~ IPropertyTypeCustomization interface begin
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ IPropertyTypeCustomization interface end

private:
	/** Returns whether the library field should be displayed */
	bool ShouldDisplayLibrary() const;

	/** Creates the widget where the entity can be picked */
	TSharedRef<SWidget> CreateEntityPickerWidget(TSharedPtr<IPropertyHandle> InPropertyHandle) const;

	/** Returns the label of the entity property */
	FText GetPickerPropertyLabel() const;

	/** Returns true if the picker is enabled */
	bool GetPickerEnabled() const;
	
	/** Returns the currently selected entity */
	TWeakObjectPtr<UDMXEntity> GetCurrentEntity() const;

	/** Returns true if multiple values are customized */
	bool GetEntityIsMultipleValues() const;

	/** Called when an entity was selected */
	void OnEntitySelected(UDMXEntity* NewEntity) const;

	/** Returns the type of entities that can be selected */
	TSubclassOf<UDMXEntity> GetEntityType() const;

	/** Returns the DMX library where the entity resides */
	TWeakObjectPtr<UDMXLibrary> GetDMXLibrary() const;

private:
	/** Handle for the entity struct that is customized */
	TSharedPtr<IPropertyHandle> StructHandle;
};
