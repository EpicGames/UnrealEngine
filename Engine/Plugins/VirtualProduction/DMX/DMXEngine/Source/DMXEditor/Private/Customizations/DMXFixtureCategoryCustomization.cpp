// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXFixtureCategoryCustomization.h"

#include "DMXProtocolTypes.h"
#include "DMXProtocolSettings.h"

#include "DetailWidgetRow.h"
#include "IPropertyTypeCustomization.h"
#include "IPropertyUtilities.h"
#include "Widgets/SNameListPicker.h"


TSharedRef<IPropertyTypeCustomization> FDMXFixtureCategoryCustomization::MakeInstance()
{
	return MakeShared<FDMXFixtureCategoryCustomization>();
}

void FDMXFixtureCategoryCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	StructPropertyHandle = InPropertyHandle;
	check(CastFieldChecked<FStructProperty>(StructPropertyHandle->GetProperty())->Struct == FDMXFixtureCategory::StaticStruct());

	TSharedPtr<IPropertyUtilities> PropertyUtilities = CustomizationUtils.GetPropertyUtilities();

	UDMXProtocolSettings* ProtocolSettings = GetMutableDefault<UDMXProtocolSettings>();
	if (!ProtocolSettings)
	{
		return;
	}

	InHeaderRow
		.IsEnabled(MakeAttributeLambda([=] 
			{ 
				return !InPropertyHandle->IsEditConst() && PropertyUtilities->IsPropertyEditingEnabled();
			}))
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(125.0f)
		.MaxDesiredWidth(0.0f)
		[
			SNew(SNameListPicker)
			.Font(CustomizationUtils.GetRegularFont())
			.HasMultipleValues(this, &FDMXFixtureCategoryCustomization::HasMultipleValues)
			.OptionsSource(ProtocolSettings->FixtureCategories.Array())
			.Value(this, &FDMXFixtureCategoryCustomization::GetValue)
			.bDisplayWarningIcon(true)
			.OnValueChanged(this, &FDMXFixtureCategoryCustomization::SetValue)
		];
}

void FDMXFixtureCategoryCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{

}

FName FDMXFixtureCategoryCustomization::GetValue() const
{
	TArray<const void*> RawData;
	StructPropertyHandle->AccessRawData(RawData);

	for (const void* RawPtr : RawData)
	{
		if (RawPtr != nullptr)
		{
			// The types we use with this customization must have a cast constructor to FName
			return reinterpret_cast<const FDMXFixtureCategory*>(RawPtr)->Name;
		}
	}

	return FName();
}

void FDMXFixtureCategoryCustomization::SetValue(FName NewValue)
{
	const TSharedPtr<IPropertyHandle> NameHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDMXFixtureCategory, Name));

	// Avoid recursion
	FName OldValue;
	if (NameHandle->GetValue(OldValue) == FPropertyAccess::Success &&
		OldValue != NewValue)
	{
		NameHandle->NotifyPreChange();
		NameHandle->SetValue(NewValue);
		NameHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	}
}

bool FDMXFixtureCategoryCustomization::HasMultipleValues() const
{
	TArray<const void*> RawData;
	StructPropertyHandle->AccessRawData(RawData);
	if (RawData.Num() == 1)
	{
		return false;
	}

	TOptional<FDMXFixtureCategory> CompareAgainst;
	for (const void* RawPtr : RawData)
	{
		if (RawPtr == nullptr)
		{
			if (CompareAgainst.IsSet())
			{
				return false;
			}
		}
		else
		{
			const FDMXFixtureCategory* ThisValue = reinterpret_cast<const FDMXFixtureCategory*>(RawPtr);

			if (!CompareAgainst.IsSet())
			{
				CompareAgainst = *ThisValue;
			}
			else if (!(*ThisValue == CompareAgainst.GetValue()))
			{
				return true;
			}
		}
	}

	return false;
}
