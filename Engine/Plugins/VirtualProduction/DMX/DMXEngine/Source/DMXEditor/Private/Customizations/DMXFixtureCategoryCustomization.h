// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolTypes.h"

#include "CoreMinimal.h"
#include "DetailWidgetRow.h"
#include "IPropertyTypeCustomization.h"
#include "IPropertyUtilities.h"
#include "Widgets/SNameListPicker.h"


/**  
 * Customization for the FDMXFixtureCategory struct 
 */
class FDMXFixtureCategoryCustomization
	: public IPropertyTypeCustomization
{
public:
	/** Creates an instance of this property type customization */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	//~ IPropertyTypeCustomization interface begin
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ IPropertyTypeCustomization interface end

private:
	/** Returns the value as FName */
	FName GetValue() const;

	/** Sets the value */
	void SetValue(FName NewValue);

	/** True if the customization displays multiple values */
	bool HasMultipleValues() const;

private:
	/** Handle for the FDMXFixtureCategory struct self */
	TSharedPtr<IPropertyHandle> StructPropertyHandle;
};
