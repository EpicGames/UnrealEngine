// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXFixtureFunctionCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "Library/DMXEntityFixtureType.h"
#include "PropertyHandle.h"

namespace UE::DMX
{
	TSharedRef<IPropertyTypeCustomization> FDMXFixtureFunctionCustomization::MakeInstance()
	{
		return MakeShared<FDMXFixtureFunctionCustomization>();
	}

	void FDMXFixtureFunctionCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
	{
		uint32 NumChildren;
		if (InPropertyHandle->GetNumChildren(NumChildren) != FPropertyAccess::Success)
		{
			return;
		}

		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ChildIndex++)
		{
			const TSharedPtr<IPropertyHandle> ChildHandle = InPropertyHandle->GetChildHandle(ChildIndex);

			// Hide physical properties, they're editor only and confusing in blueprints
			if (!ChildHandle.IsValid() ||
				!ChildHandle->IsValidHandle() ||
				ChildHandle->GetProperty()->GetFName() == FDMXFixtureFunction::GetPhysicalDefaultValuePropertyName() ||
				ChildHandle->GetProperty()->GetFName() == FDMXFixtureFunction::GetPhysicalUnitPropertyName() ||
				ChildHandle->GetProperty()->GetFName() == FDMXFixtureFunction::GetPhysicalFromPropertyName() ||
				ChildHandle->GetProperty()->GetFName() == FDMXFixtureFunction::GetPhysicalToPropertyName())
			{
				continue;
			}

			InChildBuilder.AddProperty(ChildHandle.ToSharedRef());
		}
	}
}
