// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

namespace UE::DMX
{
	/**
	 * Customization for the FDMXFixtureFunction struct
	 */
	class FDMXFixtureFunctionCustomization
		: public IPropertyTypeCustomization
	{
	public:
		/** Creates an instance of this property type customization */
		static TSharedRef<IPropertyTypeCustomization> MakeInstance();

		//~ IPropertyTypeCustomization interface begin
		virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override {};
		virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
		//~ IPropertyTypeCustomization interface end
	};
}
