// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/DMXFixtureSignalFormatCustomization.h"

#include "DMXProtocolTypes.h"
#include "Widgets/SDMXSignalFormatSelector.h"

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "DMXFixtureSignalFormatCustomization"

TSharedRef<IPropertyTypeCustomization> FDMXFixtureSignalFormatCustomization::MakeInstance()
{
	return MakeShared<FDMXFixtureSignalFormatCustomization>();
}

void FDMXFixtureSignalFormatCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	PropertyHandle = StructPropertyHandle;

	TArray<EDMXFixtureSignalFormat> SignalFormats = GetSignalFormats();
	const bool bDisplay32BitOption = SignalFormats.Contains(EDMXFixtureSignalFormat::E32Bit);
	const EDMXFixtureSignalFormat InitialSelection = SignalFormats.Num() == 1 ? SignalFormats[0] : EDMXFixtureSignalFormat::E8Bit;

	HeaderRow
		.NameContent()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
			.Text(LOCTEXT("SignalFormatLabel", "Data Type"))
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SAssignNew(SignalFormatSelector, SDMXSignalFormatSelector)
			.HasMultipleValues_Lambda([this]()
				{
					return GetSignalFormats().Num() > 1;
				})
			.InitialSelection(InitialSelection)
			.OnSignalFormatSelected(this, &FDMXFixtureSignalFormatCustomization::SetSignalFormats)
		];
}

TArray<EDMXFixtureSignalFormat> FDMXFixtureSignalFormatCustomization::GetSignalFormats() const
{
	if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
	{
		TArray<void*> RawDatas;
		PropertyHandle->AccessRawData(RawDatas);

		TArray<EDMXFixtureSignalFormat> SignalFormats;
		for (void* RawData : RawDatas)
		{
			const EDMXFixtureSignalFormat* SignalFormatPtr = reinterpret_cast<EDMXFixtureSignalFormat*>(RawData);
			if (SignalFormatPtr)
			{
				SignalFormats.AddUnique(*SignalFormatPtr);
			}
		}

		return SignalFormats;
	}

	return {};
}

void FDMXFixtureSignalFormatCustomization::SetSignalFormats(EDMXFixtureSignalFormat NewSignalFormat) const
{
	PropertyHandle->NotifyPreChange();
	
	TArray<void*> RawDatas;
	PropertyHandle->AccessRawData(RawDatas);

	for (void* RawData : RawDatas)
	{
		EDMXFixtureSignalFormat* SignalFormatPtr = reinterpret_cast<EDMXFixtureSignalFormat*>(RawData);
		if (SignalFormatPtr)
		{
			EDMXFixtureSignalFormat& SignalFormatRef = *SignalFormatPtr;
			SignalFormatRef = NewSignalFormat;
		}
	}

	PropertyHandle->NotifyPostChange(EPropertyValueSetFlags::DefaultFlags);
}

#undef LOCTEXT_NAMESPACE
