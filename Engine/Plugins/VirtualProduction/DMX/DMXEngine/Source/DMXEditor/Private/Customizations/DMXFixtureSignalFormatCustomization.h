// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"

enum class EDMXFixtureSignalFormat : uint8;
class SDMXSignalFormatSelector;

class IPropertyHandle;


/** Property type customization for the EDMXFixtureSignalFromat enum */
class FDMXFixtureSignalFormatCustomization
	: public IPropertyTypeCustomization
{
public:
	/** Creates an instance of the property type customization */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	// ~Begin IPropertyTypeCustomization Interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InStructBuilder, IPropertyTypeCustomizationUtils& InStructCustomizationUtils) override {}
	// ~End IPropertyTypeCustomization Interface

private:
	/** Returns the Signal Formats currently being edited */
	TArray<EDMXFixtureSignalFormat> GetSignalFormats() const;

	/** Sets the Signal Format */
	void SetSignalFormats(EDMXFixtureSignalFormat NewSignalFormat) const;

	/** The widget to select a Signal Format from */
	TSharedPtr<SDMXSignalFormatSelector> SignalFormatSelector;

	/** Property handle for the enum property */
	TSharedPtr<IPropertyHandle> PropertyHandle;
};
