// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/DMXLibraryDetails.h"

#include "DMXEditorStyle.h"
#include "DMXSubsystem.h"
#include "Library/DMXLibrary.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "ISettingsModule.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "DMXLibraryDetails"

TSharedRef<IDetailCustomization> FDMXLibraryDetails::MakeInstance()
{
	return MakeShared<FDMXLibraryDetails>();
}

void FDMXLibraryDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DetailBuilder.HideCategory("DMX");

	IDetailCategoryBuilder& DetailCategoryBuilder = DetailBuilder.EditCategory("DMX Ports");

	// Add a button to open DMX Project Settings
	DetailCategoryBuilder.AddCustomRow(LOCTEXT("OpenSettingsFilterString", "Settings"))
		.WholeRowContent()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Top)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
			.ForegroundColor(FLinearColor::White)
			.ToolTipText(LOCTEXT("OpenSettingsTooltip", "Opens the DMX Project Settings"))
			.ContentPadding(FMargin(4.0f, 1.0f))
			.OnClicked_Lambda([]() 
				{ 
					FModuleManager::GetModulePtr<ISettingsModule>("Settings")->ShowViewer("Project", "Plugins", "DMX"); return FReply::Handled(); 
				})
			.Content()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(0.f, 0.f, 12.f, 0.f))
				[
					SNew(SImage)
					.Image(FDMXEditorStyle::Get().GetBrush("Icons.DMXLibrarySettings"))
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(FMargin(2.f, 0.f, 2.f, 0.f))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("OpenSettingsLabel", "Open DMX Project Settings"))
				]
			]
		];


	// Add the Port References
	const TSharedPtr<IPropertyHandle> PortReferencesHandle = DetailBuilder.GetProperty(UDMXLibrary::GetPortReferencesPropertyName());
	DetailCategoryBuilder
		.AddProperty(PortReferencesHandle.ToSharedRef())
		.ShouldAutoExpand(true);

	PortReferencesHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FDMXLibraryDetails::OnPortReferencesChanged));
}

void FDMXLibraryDetails::OnPortReferencesChanged()
{
	UDMXSubsystem::ClearDMXBuffers();
}

#undef LOCTEXT_NAMESPACE
