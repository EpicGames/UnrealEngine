// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"

class IPropertyHandle;
class IPropertyUtilities;


/** Details customization for the 'FixtureType FunctionProperties' details view */
class FDMXLibraryDetails
	: public IDetailCustomization
{
public:
	/** Creates an instance of this details customization */
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization interface

private:
	/** Called when port references changed */
	void OnPortReferencesChanged();
};
