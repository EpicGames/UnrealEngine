// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/DMXLibraryFromMVRImportOptionsDetails.h"

#include "DMXProtocolSettings.h"
#include "Factories/DMXLibraryFromMVRImporter.h"
#include "Factories/DMXLibraryFromMVRImportOptions.h"
#include "IO/DMXPortManager.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXLibrary.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Misc/Paths.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "DMXLibraryFromMVROptionsDetails"

TSharedRef<IDetailCustomization> FDMXLibraryFromMVRImportOptionsDetails::MakeInstance()
{
	return MakeShared<FDMXLibraryFromMVRImportOptionsDetails>();
}

void FDMXLibraryFromMVRImportOptionsDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	if (!ensureAlwaysMsgf(ObjectsBeingCustomized.Num() == 1 || !ObjectsBeingCustomized[0].IsValid(), TEXT("Multi-Editing MVR Import Options is not supported, or invalid customized Object.")))
	{
		return;
	}

	ImportOptions = CastChecked<UDMXLibraryFromMVRImportOptions>(ObjectsBeingCustomized[0]);
	if (!ensureAlwaysMsgf(FPaths::FileExists(ImportOptions->Filename), TEXT("Invalid MVR File '%s'."), *ImportOptions->Filename))
	{
		return;
	}

	IDetailCategoryBuilder& PortsCategoryBuilder = DetailBuilder.EditCategory("Ports");
	if (!HasAnyInputPorts())
	{
		const TSharedRef<IPropertyHandle> InputPortToUpdateHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDMXLibraryFromMVRImportOptions, InputPortToUpdate));
		InputPortToUpdateHandle->MarkHiddenByCustomization();

		// Show 'Generate Input Port' instead of the default property name 'Update Input Port' if there's no port available. 
		const TSharedRef<IPropertyHandle> UpdateInputPortHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDMXLibraryFromMVRImportOptions, bUpdateInputPort));
		UpdateInputPortHandle->MarkHiddenByCustomization();

		PortsCategoryBuilder.AddCustomRow(FText::GetEmpty())
			.NameContent()
			[
				SNew(STextBlock)
				.Font(DetailBuilder.GetDetailFont())
				.Text(LOCTEXT("GenerateInputPortDisplayName", "Generate Input Port"))
			]
			.ValueContent()
			[
				UpdateInputPortHandle->CreatePropertyValueWidget()
			];
	}

	if (!HasAnyOutputPorts())
	{
		const TSharedRef<IPropertyHandle> OutputPortToUpdateHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDMXLibraryFromMVRImportOptions, OutputPortToUpdate));
		OutputPortToUpdateHandle->MarkHiddenByCustomization();

		// Show 'Generate Output Port' instead of the default property name 'Update Output Port' if there's no port available. 
		const TSharedRef<IPropertyHandle> UpdateOutputPortHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDMXLibraryFromMVRImportOptions, bUpdateOutputPort));
		UpdateOutputPortHandle->MarkHiddenByCustomization();

		PortsCategoryBuilder.AddCustomRow(FText::GetEmpty())
			.NameContent()
			[
				SNew(STextBlock)
				.Font(DetailBuilder.GetDetailFont())
				.Text(LOCTEXT("GenerateOutputPortDisplayName", "Generate Output Port"))
			]
			.ValueContent()
			[
				UpdateOutputPortHandle->CreatePropertyValueWidget()
			];
	}
}

bool FDMXLibraryFromMVRImportOptionsDetails::HasAnyInputPorts() const
{
	return !FDMXPortManager::Get().GetInputPorts().IsEmpty();
}

bool FDMXLibraryFromMVRImportOptionsDetails::HasAnyOutputPorts() const
{
	return !FDMXPortManager::Get().GetOutputPorts().IsEmpty();
}

#undef LOCTEXT_NAMESPACE
