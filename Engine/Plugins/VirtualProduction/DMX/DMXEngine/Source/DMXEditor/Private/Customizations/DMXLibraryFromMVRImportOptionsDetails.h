// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"

class UDMXLibrary;
class UDMXLibraryFromMVRImportOptions;

class IPropertyUtilities;


/** Details customization fro the MVR Scene Actor Factory Import Options */
class FDMXLibraryFromMVRImportOptionsDetails final
	: public IDetailCustomization
{
public:
	/** Creates an instance of this details customization */
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization interface

private:
	/** Returns true if the project has any input ports defined */
	bool HasAnyInputPorts() const;

	/** Returns true if the project has any output ports defined */
	bool HasAnyOutputPorts() const;

	/** For comparison when merging, create a temporary DMX Library from the imported MVR before actually merging changes when reimporting */
	UDMXLibrary* TempDMXLibrary = nullptr;

	/** Import Options Object */
	UDMXLibraryFromMVRImportOptions* ImportOptions = nullptr;
};
