// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/DMXLibraryPortReferencesCustomization.h"

#include "IO/DMXPortManager.h"
#include "IO/DMXInputPort.h"
#include "IO/DMXInputPortReference.h"
#include "IO/DMXOutputPort.h"
#include "IO/DMXOutputPortReference.h"
#include "IO/DMXPortManager.h"
#include "Interfaces/IDMXProtocol.h"
#include "Library/DMXLibrary.h"

#include "DetailWidgetRow.h"
#include "Styling/AppStyle.h"
#include "DetailCategoryBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "IPropertyUtilities.h"
#include "Styling/CoreStyle.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h" 
#include "Widgets/Layout/SWrapBox.h"


#define LOCTEXT_NAMESPACE "DMXLibraryPortReferencesCustomization"

namespace
{
	/** Widget that displays a single port reference */
	template <typename PortReferenceType>
	class SDMXLibraryPortReferenceCustomization
		: public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SDMXLibraryPortReferenceCustomization<PortReferenceType>)
		{}
		
			SLATE_ARGUMENT(TSharedPtr<IPropertyHandle>, PortReferenceHandle)

		SLATE_END_ARGS()

		/** Constructs the widget */
		void Construct(const FArguments& InArgs, const TSharedRef<SWidget>& PortInfoWidget)
		{
			check(InArgs._PortReferenceHandle.IsValid() && InArgs._PortReferenceHandle->IsValidHandle());

			PortReferenceHandle = InArgs._PortReferenceHandle;

			const PortReferenceType PortReference = GetPortReferenceChecked();

			const FDMXPortSharedRef& Port = FDMXPortManager::Get().FindPortByGuidChecked(PortReference.GetPortGuid());
			bool bEnabledFlagSet = PortReference.IsEnabledFlagSet();

			ChildSlot
			[
				SNew(SVerticalBox)

				// Port name and Port infos
				+ SVerticalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Top)
				.Padding(FMargin(4.f, 4.f, 4.f, 4.f))
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SBox)
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Top)
						.MinDesiredWidth(180.f)
						[
							SNew(STextBlock)
							.ToolTipText(LOCTEXT("PortReferenceEnabledTextTooltip", "Enables or disables the port for the Library"))
							.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
							.Text(FText::FromString(Port->GetPortName()))
						]
					]

					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					.Padding(FMargin(16.f, 4.f, 4.f, 4.f))
					.FillWidth(1.f)
					[
						PortInfoWidget
					]
				]

				// Enabled checkbox
				+ SVerticalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Top)
				.Padding(FMargin(4.f, 4.f, 4.f, 4.f))
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					// Enabled checkbox
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Top)
					.AutoWidth()
					.Padding(FMargin(16.f, 0.f, 4.f, 0.f))
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
						.Text(LOCTEXT("PortReferenceEnabledLabel", "Enabled"))
					]

					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Top)
					[
						SNew(SCheckBox)
						.ToolTipText(LOCTEXT("PortReferenceEnabledCheckboxTooltip", "Enables or disables the port for the Library"))
						.IsChecked(bEnabledFlagSet)
						.OnCheckStateChanged(this, &SDMXLibraryPortReferenceCustomization<PortReferenceType>::OnPortReferenceChangedEnabledState)
					]
				]
			];
		}
		

	private:
		/** Called when the checkbox state changed */
		void OnPortReferenceChangedEnabledState(ECheckBoxState CheckBoxState)
		{
			check(PortReferenceHandle.IsValid());

			TSharedPtr<IPropertyHandle> PortReferenceEnabledFlagHandle = PortReferenceHandle->GetChildHandle(PortReferenceType::GetEnabledFlagPropertyName());
			check(PortReferenceEnabledFlagHandle.IsValid());

			if (CheckBoxState == ECheckBoxState::Checked)
			{
				PortReferenceEnabledFlagHandle->SetValue(true);
			}
			else
			{
				PortReferenceEnabledFlagHandle->SetValue(false);
			}
		}

		/** Returns the port reference from the handle */
		const PortReferenceType& GetPortReferenceChecked()
		{
			check(PortReferenceHandle.IsValid());

			TArray<void*> RawData;
			PortReferenceHandle->AccessRawData(RawData);

			// Multi editing isn't possible in arrays, so there should be always exactly one element
			check(RawData.Num() == 1);

			return *static_cast<PortReferenceType*>(RawData[0]);
		}

		/** Handle to the port reference this widget draws */
		TSharedPtr<IPropertyHandle> PortReferenceHandle;
	};
}

TSharedRef<IPropertyTypeCustomization> FDMXLibraryPortReferencesCustomization::MakeInstance()
{
	return MakeShared<FDMXLibraryPortReferencesCustomization>();
}

void FDMXLibraryPortReferencesCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	const bool bDisplayResetToDefault = false;
	const FText DisplayNameOverride = LOCTEXT("HeaderDisplayName", "Ports");
	const FText DisplayToolTipOverride = LOCTEXT("HeaderToolTip", "The ports to be used with this Library.");

	HeaderRow
		.ShouldAutoExpand(true)
		.NameContent()
		[
			SNullWidget::NullWidget
		];
}

void FDMXLibraryPortReferencesCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	PropertyUtilities = StructCustomizationUtils.GetPropertyUtilities();
	LibraryPortReferencesHandle = StructPropertyHandle;

	// Expand the category
	ChildBuilder.GetParentCategory().InitiallyCollapsed(false);

	// Hide the 'reset to default' option
	LibraryPortReferencesHandle->MarkResetToDefaultCustomized();


	// Customize input port refrences view in the ports struct
	ChildBuilder
		.AddCustomRow(LOCTEXT("InputPortReferenceTitleRowSearchString", "InputPorts"))
		.NameContent()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Top)
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
			.Text(LOCTEXT("InputPortsLabel", "Input Ports"))
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		[
			SAssignNew(InputPortReferenceContentBorder, SBorder)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
		];

	ChildBuilder
		.AddCustomRow(LOCTEXT("SeparatorSearchString", "Separator"))
		.WholeRowContent()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		[
			SNew(SSeparator)
			.Orientation(EOrientation::Orient_Horizontal)
		];

	
	// Customize output port refrences view in the ports struct
	ChildBuilder
		.AddCustomRow(LOCTEXT("OutputPortReferenceTitleRowSearchString", "OutputPorts"))
		.NameContent()
		.HAlign(HAlign_Left)
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
			.Text(LOCTEXT("OutputPortsLabel", "Output Ports"))
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		[
			SAssignNew(OutputPortReferenceContentBorder, SBorder)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
		];

	// Bind to port reference array changes
	const TSharedPtr<IPropertyHandle> InputPortReferencesHandle = LibraryPortReferencesHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDMXLibraryPortReferences, InputPortReferences));
	const TSharedPtr<IPropertyHandleArray> InputPortReferencesHandleArray = InputPortReferencesHandle->AsArray();
	check(InputPortReferencesHandleArray.IsValid());

	FSimpleDelegate OnInputPortArrayChangedDelegate = FSimpleDelegate::CreateSP(this, &FDMXLibraryPortReferencesCustomization::RefreshPortReferenceWidgets);
	InputPortReferencesHandleArray->SetOnNumElementsChanged(OnInputPortArrayChangedDelegate);

	const TSharedPtr<IPropertyHandle> OutputPortReferencesHandle = LibraryPortReferencesHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDMXLibraryPortReferences, OutputPortReferences));
	const TSharedPtr<IPropertyHandleArray> OutputPortReferencesHandleArray = OutputPortReferencesHandle->AsArray();
	check(OutputPortReferencesHandleArray.IsValid());

	FSimpleDelegate OnOutputPortArrayChangedDelegate = FSimpleDelegate::CreateSP(this, &FDMXLibraryPortReferencesCustomization::RefreshPortReferenceWidgets);
	OutputPortReferencesHandleArray->SetOnNumElementsChanged(OnOutputPortArrayChangedDelegate);

	// Create content widgets
	RefreshPortReferenceWidgets();

	// Handle port changes
	FDMXPortManager::Get().OnPortsChanged.AddSP(this, &FDMXLibraryPortReferencesCustomization::OnPortsChanged);
}

void FDMXLibraryPortReferencesCustomization::OnPortsChanged()
{
	if (PropertyUtilities.IsValid())
	{
		PropertyUtilities->ForceRefresh();
	}
}

void FDMXLibraryPortReferencesCustomization::RefreshPortReferenceWidgets()
{
	check(InputPortReferenceContentBorder.IsValid());
	check(OutputPortReferenceContentBorder.IsValid());

	// Add the Input Port Reference widgets
	const TSharedRef<SVerticalBox> InputPortReferencesWidget = SNew(SVerticalBox);
	
	const TSharedPtr<IPropertyHandle> InputPortReferencesHandle = LibraryPortReferencesHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDMXLibraryPortReferences, InputPortReferences));
	const TSharedPtr<IPropertyHandleArray> InputPortReferencesHandleArray = InputPortReferencesHandle->AsArray();
	check(InputPortReferencesHandleArray.IsValid());

	uint32 NumInputPortRefElements = 0;
	InputPortReferencesHandleArray->GetNumElements(NumInputPortRefElements);
	for (uint32 IndexElement = 0; IndexElement < NumInputPortRefElements; IndexElement++)
	{
		TSharedRef<IPropertyHandle> InputPortReferenceHandle = InputPortReferencesHandleArray->GetElement(IndexElement);
		FDMXInputPortSharedPtr InputPort = GetInputPort(InputPortReferenceHandle);

		// May not be valid while array interactions are ongoing
		if(InputPort.IsValid())
		{
			InputPortReferencesWidget->AddSlot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.AutoHeight()
			[
				SNew(SBox)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Top)
				[
					SNew(SDMXLibraryPortReferenceCustomization<FDMXInputPortReference>, GeneratePortInfoWidget(InputPort))
					.PortReferenceHandle(InputPortReferenceHandle)
				]
			];
		}
	}

	InputPortReferenceContentBorder->SetContent(InputPortReferencesWidget);

	// Add the Output Port Reference widgets
	const TSharedRef<SVerticalBox> OutputPortReferencesWidget = SNew(SVerticalBox);

	const TSharedPtr<IPropertyHandle> OutputPortReferencesHandle = LibraryPortReferencesHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDMXLibraryPortReferences, OutputPortReferences));
	const TSharedPtr<IPropertyHandleArray> OutputPortReferencesHandleArray = OutputPortReferencesHandle->AsArray();
	check(OutputPortReferencesHandleArray.IsValid());

	uint32 NumOutputPortRefElements = 0;
	OutputPortReferencesHandleArray->GetNumElements(NumOutputPortRefElements);
	for (uint32 IndexElement = 0; IndexElement < NumOutputPortRefElements; IndexElement++)
	{
		TSharedRef<IPropertyHandle> OutputPortReferenceHandle = OutputPortReferencesHandleArray->GetElement(IndexElement);
		FDMXOutputPortSharedPtr OutputPort = GetOutputPort(OutputPortReferenceHandle);

		// May not be valid while array interactions are ongoing
		if(OutputPort.IsValid())
		{
			OutputPortReferencesWidget->AddSlot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.AutoHeight()
			[
				SNew(SBox)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Top)
				[
					SNew(SDMXLibraryPortReferenceCustomization<FDMXOutputPortReference>, GeneratePortInfoWidget(OutputPort))
					.PortReferenceHandle(OutputPortReferenceHandle)
				]
			];
		}
	}

	OutputPortReferenceContentBorder->SetContent(OutputPortReferencesWidget);
}

TSharedRef<SWidget> FDMXLibraryPortReferencesCustomization::GeneratePortInfoWidget(const FDMXPortSharedPtr& Port) const
{
	if(Port.IsValid())
	{
		IDMXProtocolPtr Protocol = Port->GetProtocol();
		if(Protocol.IsValid())
		{
			static const FSlateFontInfo PropertyWindowNormalFont = FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont"));
			static const FLinearColor FontColor = FLinearColor(0.6f, 0.6f, 0.6f);

			const int32 LocalUniverseStart = Port->GetLocalUniverseStart();
			const int32 LocalUniverseEnd = Port->GetLocalUniverseEnd();
			const int32 ExternUniverseStart = Port->GetExternUniverseStart();
			const int32 ExternUniverseEnd = Port->GetExternUniverseEnd();

			const FText PortName = FText::FromString(Port->GetPortName() + TEXT(":"));
			const FText ProtocolName = FText::FromName(Protocol->GetProtocolName());
			const FText LocalUniverseStartText = FText::FromString(FString::FromInt(LocalUniverseStart));
			const FText LocalUniverseEndText = FText::FromString(FString::FromInt(LocalUniverseEnd));
			const FText ExternUniverseStartText = FText::FromString(FString::FromInt(ExternUniverseStart));
			const FText ExternUniverseEndText = FText::FromString(FString::FromInt(ExternUniverseEnd));
	
			return
				SNew(SWrapBox)
				.InnerSlotPadding(FVector2D(4.f, 4.f))
				.UseAllottedSize(true)

				// Protocol Name
				+ SWrapBox::Slot()
				[
					SNew(STextBlock)
					.ColorAndOpacity(FontColor)
					.Font(PropertyWindowNormalFont)
					.Text(ProtocolName)
				]
		
				// Local Universe Label
				+ SWrapBox::Slot()
				[
					SNew(STextBlock)
					.ColorAndOpacity(FontColor)
					.Font(PropertyWindowNormalFont)
					.Text(LOCTEXT("LocalUniverseStartLabel", "Local Universe:"))
				]

				// Local Universe Start
				+ SWrapBox::Slot()
				[
					SNew(STextBlock)
					.ColorAndOpacity(FontColor)
					.Font(PropertyWindowNormalFont)
					.Text(LocalUniverseStartText)
				]

				// Local Universe 'to' Label
				+ SWrapBox::Slot()
				[
					SNew(STextBlock)
					.ColorAndOpacity(FontColor)
					.Font(PropertyWindowNormalFont)
					.Text(LOCTEXT("LocalUniverseToLabel", "-"))
				]

				// Local Universe End
				+ SWrapBox::Slot()
				[
					SNew(STextBlock)
					.ColorAndOpacity(FontColor)
					.Font(PropertyWindowNormalFont)
					.Text(LocalUniverseEndText)
				]

				// Extern Universe Label
				+ SWrapBox::Slot()
				[
					SNew(STextBlock)
					.ColorAndOpacity(FontColor)
					.Font(PropertyWindowNormalFont)
					.Text(LOCTEXT("ExternUniverseStartLabel", "mapped to extern Universe:"))
				]

				// Extern Universe Start
				+ SWrapBox::Slot()
				[
					SNew(STextBlock)
					.ColorAndOpacity(FontColor)
					.Font(PropertyWindowNormalFont)
					.Text(ExternUniverseStartText)
				]

				// Extern Universe 'to' Label
				+ SWrapBox::Slot()
				[
					SNew(STextBlock)
					.ColorAndOpacity(FontColor)
					.Font(PropertyWindowNormalFont)
					.Text(LOCTEXT("ExternUniverseToLabel", "-"))
				]

				// Extern Universe End
				+ SWrapBox::Slot()
				[
					SNew(STextBlock)
					.ColorAndOpacity(FontColor)
					.Font(PropertyWindowNormalFont)
					.Text(ExternUniverseEndText)
				];
		}
	}

	return SNullWidget::NullWidget;
}

FDMXInputPortSharedPtr FDMXLibraryPortReferencesCustomization::GetInputPort(const TSharedPtr<IPropertyHandle>& InputPortReferenceHandle)
{
	check(InputPortReferenceHandle.IsValid());
	const TSharedPtr<IPropertyHandle> PortGuidHandle = InputPortReferenceHandle->GetChildHandle(FDMXInputPortReference::GetPortGuidPropertyName());
	
	check(PortGuidHandle.IsValid());

	TArray<void*> RawData;
	PortGuidHandle->AccessRawData(RawData);

	if (ensureMsgf(RawData.Num() == 1, TEXT("FDMXLibraryPortReference struct is only ment to be used in DMX library and does not support multi editing.")))
	{
		const FGuid* PortGuidPtr = reinterpret_cast<FGuid*>(RawData[0]);
		if (PortGuidPtr && PortGuidPtr->IsValid())
		{
			const FDMXInputPortSharedPtr InputPort = FDMXPortManager::Get().FindInputPortByGuid(*PortGuidPtr);
			if (InputPort.IsValid())
			{
				return InputPort;
			}
		}
	}

	return nullptr;
}

FDMXOutputPortSharedPtr FDMXLibraryPortReferencesCustomization::GetOutputPort(const TSharedPtr<IPropertyHandle>& OutputPortReferenceHandle)
{
	check(OutputPortReferenceHandle.IsValid());
	const TSharedPtr<IPropertyHandle> PortGuidHandle = OutputPortReferenceHandle->GetChildHandle(FDMXOutputPortReference::GetPortGuidPropertyName());

	check(PortGuidHandle.IsValid());

	TArray<void*> RawData;
	PortGuidHandle->AccessRawData(RawData);

	if (ensureMsgf(RawData.Num() == 1, TEXT("FDMXLibraryPortReference struct is only ment to be used in DMX library and does not support multi editing.")))
	{
		const FGuid* PortGuidPtr = reinterpret_cast<FGuid*>(RawData[0]);
		if (PortGuidPtr && PortGuidPtr->IsValid())
		{
			const FDMXOutputPortSharedPtr OutputPort = FDMXPortManager::Get().FindOutputPortByGuid(*PortGuidPtr);
			if (OutputPort.IsValid())
			{
				return OutputPort;
			}
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
