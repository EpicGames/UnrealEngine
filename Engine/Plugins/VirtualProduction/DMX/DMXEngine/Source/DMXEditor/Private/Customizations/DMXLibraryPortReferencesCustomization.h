// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolCommon.h"

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"

class FDetailWidgetRow;
class IPropertyHandle;
class IPropertyUtilities;
class SBorder;
class SWidget;

/** Property type customization for the Library Port References struct*/
class FDMXLibraryPortReferencesCustomization
	: public IPropertyTypeCustomization
{
public:
	/** Creates an instance of the property type customization */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	// ~Begin IPropertyTypeCustomization Interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	// ~End IPropertyTypeCustomization Interface

protected:
	/** Called when ports changed */
	void OnPortsChanged();

	/** Generates an port row given the PortReferenceHandles */
	void RefreshPortReferenceWidgets();

	/** Generates infos about the port */
	TSharedRef<SWidget> GeneratePortInfoWidget(const FDMXPortSharedPtr& Port) const;

	/** Returns the input port from the port refernce handle */
	FDMXInputPortSharedPtr GetInputPort(const TSharedPtr<IPropertyHandle>& InputPortReferenceHandle);

	/** Returns the output port from the port refernce handle */
	FDMXOutputPortSharedPtr GetOutputPort(const TSharedPtr<IPropertyHandle>& OutputPortReferenceHandle);

	/** The PortReferences property handle (the outermost StructPropertyHandle that is being customized) */
	TSharedPtr<IPropertyHandle> LibraryPortReferencesHandle;

	/** Input port reference content border */
	TSharedPtr<SBorder> InputPortReferenceContentBorder;

	/** Output port reference content border */
	TSharedPtr<SBorder> OutputPortReferenceContentBorder;

	/** Property utilities for this customization */
	TSharedPtr<IPropertyUtilities> PropertyUtilities;
};
