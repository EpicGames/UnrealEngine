// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/DMXMVRSceneActorDetails.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Game/DMXComponent.h"
#include "IPropertyUtilities.h"
#include "ISceneOutliner.h"
#include "LevelEditor.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/DMXLibrary.h"
#include "Modules/ModuleManager.h"
#include "MVR/DMXMVRSceneActor.h"
#include "ScopedTransaction.h"
#include "Subsystems/EditorActorSubsystem.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SWrapBox.h"

#define LOCTEXT_NAMESPACE "DMXMVRSceneActorDetails"

TSharedRef<IDetailCustomization> FDMXMVRSceneActorDetails::MakeInstance()
{
	return MakeShared<FDMXMVRSceneActorDetails>();
}

void FDMXMVRSceneActorDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	PropertyUtilities = DetailBuilder.GetPropertyUtilities();

	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	for (TWeakObjectPtr<UObject> Object : ObjectsBeingCustomized)
	{
		if (ADMXMVRSceneActor* MVRSceneActor = Cast<ADMXMVRSceneActor>(Object))
		{
			OuterSceneActors.Add(MVRSceneActor);
		}
	}

	CreateDMXLibrarySection(DetailBuilder);
	CreateFixtureTypeToActorClassSection(DetailBuilder);

	// Listen to map and actor changes
	FEditorDelegates::MapChange.AddSP(this, &FDMXMVRSceneActorDetails::OnMapChange);

	if (GEngine)
	{
		GEngine->OnLevelActorDeleted().AddSP(this, &FDMXMVRSceneActorDetails::OnActorDeleted);
	}
}

void FDMXMVRSceneActorDetails::CreateDMXLibrarySection(IDetailLayoutBuilder& DetailBuilder)
{
	IDetailCategoryBuilder& MVRCategory = DetailBuilder.EditCategory("MVR");
	
	// DMX Library
	MVRCategory.AddProperty(DetailBuilder.GetProperty(ADMXMVRSceneActor::GetDMXLibraryPropertyNameChecked()));

	// Write Transforms to DMX Library button
	MVRCategory.AddCustomRow(LOCTEXT("WriteTransformsFilterText", "Write Transforms to DMX Library"))
		[
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.Padding(8.f, 1.f, 0.f, 1.f)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			[
				SNew(SButton)
				.Text(LOCTEXT("WriteTransformsToDMXLibraryCaption", "Write Transforms to DMX Library"))
				.ToolTipText(LOCTEXT("WriteTransformsToDMXLibraryTooltip", "Sets the transform of the Fixture Actors as Default Transforms for the Fixture Patches.\n\nThe transforms will be used when the DMX Library is spawned in another level.\nThe transforms will be used when the DMX Library is exported as MVR."))
				.OnClicked(this, &FDMXMVRSceneActorDetails::OnWriteTransformsToDMXLibraryClicked)
			]
		];

	// Refresh from DMX Library button
	MVRCategory.AddCustomRow(LOCTEXT("RefreshSceneFilterText", "Refresh from DMX Library"))
		.WholeRowContent()
		[			
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.Padding(8.f, 1.f, 0.f, 1.f)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			[
				SNew(SButton)
				.Text(LOCTEXT("RefreshActorsFromDMXLibraryCaption", "Refresh Actors from DMX Library"))
				.ToolTipText(LOCTEXT("RefreshActorsFromDMXLibraryTooltip", "Updates the MVR Scene to reflect the DMX Library, possibly respwaning deleted actors and reseting to default transforms according to options."))
				.OnClicked(this, &FDMXMVRSceneActorDetails::OnRefreshActorsFromDMXLibraryClicked)
			]
		];

	// Respawn Deleted Actors option
	const TSharedRef<IPropertyHandle> RespawnDeletedActorHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ADMXMVRSceneActor, bRespawnDeletedActorsOnRefresh));
	RespawnDeletedActorHandle->MarkHiddenByCustomization();
	MVRCategory.AddCustomRow(LOCTEXT("RespawnDeletedActorsFilterText", "Respawn Deleted Actors"))
		.NameContent()
		[
			SNew(SBorder)
			.Padding(8.f, 0.f, 0.f, 0.f)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			[
				RespawnDeletedActorHandle->CreatePropertyNameWidget()
			]
		]
		.ValueContent()
		[
			RespawnDeletedActorHandle->CreatePropertyValueWidget()
		];
	
	// Reset Transforms option
	const TSharedRef<IPropertyHandle> UpdateTransformHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ADMXMVRSceneActor, bUpdateTransformsOnRefresh));
	UpdateTransformHandle->MarkHiddenByCustomization();
	MVRCategory.AddCustomRow(LOCTEXT("ResetTransformsFilterText", "Reset Transforms"))
		.NameContent()
		[
			SNew(SBorder)
			.Padding(8.f, 0.f, 0.f, 0.f)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			[
				UpdateTransformHandle->CreatePropertyNameWidget()
			]
		]
		.ValueContent()
		[
			UpdateTransformHandle->CreatePropertyValueWidget()
		];
}

void FDMXMVRSceneActorDetails::CreateFixtureTypeToActorClassSection(IDetailLayoutBuilder& DetailBuilder)
{
	IDetailCategoryBuilder& ActorTypeForFixtureTypeCategory = DetailBuilder.EditCategory("Fixture Type to Spawned Actor");
	ActorTypeForFixtureTypeCategory.InitiallyCollapsed(false);

	const TSharedRef<IPropertyHandle> FixtureTypeToActorClassesHandle = DetailBuilder.GetProperty(ADMXMVRSceneActor::GetFixtureTypeToActorClassesPropertyNameChecked());
	FixtureTypeToActorClassesHandle->MarkHiddenByCustomization();

	const TSharedPtr<IPropertyHandleArray> FixtureTypeToActorClassesHandleArray = FixtureTypeToActorClassesHandle->AsArray();
	FixtureTypeToActorClassesHandleArray->SetOnNumElementsChanged(FSimpleDelegate::CreateSP(this, &FDMXMVRSceneActorDetails::RequestRefresh));

	uint32 NumFixtureTypeToActorClassElements;
	if (!ensure(FixtureTypeToActorClassesHandleArray->GetNumElements(NumFixtureTypeToActorClassElements) == FPropertyAccess::Success))
	{
		return;
	}

	for (uint32 FixtureTypeToActorClassElementIndex = 0; FixtureTypeToActorClassElementIndex < NumFixtureTypeToActorClassElements; FixtureTypeToActorClassElementIndex++)
	{
		const TSharedPtr<IPropertyHandle> FixtureTypeToActorClassHandle = FixtureTypeToActorClassesHandleArray->GetElement(FixtureTypeToActorClassElementIndex);
		const TSharedPtr<IPropertyHandle> FixtureTypeHandle = FixtureTypeToActorClassHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDMXMVRSceneFixtureTypeToActorClassPair, FixtureType));
		TSharedPtr<IPropertyHandle> ActorClassHandle = FixtureTypeToActorClassHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDMXMVRSceneFixtureTypeToActorClassPair, ActorClass));
		ActorClassHandle->SetOnChildPropertyValuePreChange(FSimpleDelegate::CreateSP(this, &FDMXMVRSceneActorDetails::OnPreEditChangeActorClassInFixtureTypeToActorClasses));
		ActorClassHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FDMXMVRSceneActorDetails::OnPostEditChangeActorClassInFixtureTypeToActorClasses));

		UObject* FixtureTypeObject = nullptr;
		if (!FixtureTypeHandle->GetValue(FixtureTypeObject))
		{
			return;
		}

		UDMXEntityFixtureType* FixtureType = Cast<UDMXEntityFixtureType>(FixtureTypeObject);
		if (!FixtureType)
		{
			continue;
		}

		ActorTypeForFixtureTypeCategory.AddCustomRow(LOCTEXT("FixtureTypeToActorClassFilter", "Fixture Type to Spawned Actor"))
		.NameContent()
		[
			SNew(STextBlock)
			.Font(DetailBuilder.GetDetailFont())
			.Text_Lambda([WeakFixtureType = TWeakObjectPtr<UDMXEntityFixtureType>(FixtureType)]()
				{
					if (WeakFixtureType.IsValid())
					{
						return FText::FromString(WeakFixtureType.Get()->Name);
					}
					
					return LOCTEXT("InvalidFixtureTypeName", "Invalid Fixture Type");
				})
		]
		.ValueContent()
		[
			SNew(SWrapBox)

			+ SWrapBox::Slot()
			[
				ActorClassHandle->CreatePropertyValueWidget()
			]

			+ SWrapBox::Slot()
			[
				SNew(SButton)
				.OnClicked(this, &FDMXMVRSceneActorDetails::OnFixtureTypeToActorClassGroupSelected, FixtureTypeObject)
				.Text(LOCTEXT("SelectFixtureTypeGroupButtonCaption", "Select"))
			]
		];
	}
}

FReply FDMXMVRSceneActorDetails::OnRefreshActorsFromDMXLibraryClicked()
{
	const FScopedTransaction RefreshActorsFromDMXLibraryTransaction(LOCTEXT("RefreshActorsFromDMXLibraryTransaction", "Update MVR Scene form DMX Library"));

	const TArray<TWeakObjectPtr<UObject>> SelectedObjects = PropertyUtilities->GetSelectedObjects();
	
	AActor* FirstActor = SelectedObjects.IsEmpty() ? nullptr : Cast<AActor>(SelectedObjects[0]);
	UWorld* World = FirstActor ? FirstActor->GetWorld() : nullptr;
	if (World)
	{
		World->PreEditChange(nullptr);
	}
	
	for (const TWeakObjectPtr<UObject>& SelectedObject : SelectedObjects)
	{
		if (ADMXMVRSceneActor* MVRSceneActor = Cast<ADMXMVRSceneActor>(SelectedObject.Get()))
		{
			MVRSceneActor->PreEditChange(ADMXMVRSceneActor::StaticClass()->FindPropertyByName(ADMXMVRSceneActor::GetRelatedAcctorsPropertyNameChecked()));

			MVRSceneActor->RefreshFromDMXLibrary();

			MVRSceneActor->PostEditChange();
		}
	}

	if (World)
	{
		World->PostEditChange();
	}

	RequestRefresh();

	return FReply::Handled();
}

FReply FDMXMVRSceneActorDetails::OnWriteTransformsToDMXLibraryClicked()
{
	const FScopedTransaction WriteTransformsToDMXLibraryTransaction(LOCTEXT("WriteTransformsToDMXLibraryTransaction", "Write MVR Scene Transforms to DMX Library"));

	const TArray<TWeakObjectPtr<UObject>> SelectedObjects = PropertyUtilities->GetSelectedObjects();
	for (const TWeakObjectPtr<UObject>& SelectedObject : SelectedObjects)
	{
		ADMXMVRSceneActor* MVRSceneActor = Cast<ADMXMVRSceneActor>(SelectedObject.Get());
		UDMXLibrary* DMXLibrary = MVRSceneActor ? MVRSceneActor->GetDMXLibrary() : nullptr;

		if (!MVRSceneActor || !DMXLibrary)
		{
			continue;
		}

		const TArray<TSoftObjectPtr<AActor>> SoftRelatedActors = MVRSceneActor->GetRelatedActors();
		for (const TSoftObjectPtr<AActor>& SoftRelatedActor : SoftRelatedActors)
		{
			if (!SoftRelatedActor.IsValid())
			{
				continue;
			}
			AActor* RelatedActor = SoftRelatedActor.Get();
			UDMXEntityFixturePatch* FixturePatch = GetFixturePatchFromActor(RelatedActor);

			if (FixturePatch)
			{
				FixturePatch->PreEditChange(nullptr);
				FixturePatch->SetDefaultTransform(RelatedActor->GetTransform());
				FixturePatch->PostEditChange();
			}
		}
	}

	RequestRefresh();

	return FReply::Handled();
}

UDMXEntityFixturePatch* FDMXMVRSceneActorDetails::GetFixturePatchFromActor(AActor* Actor) const
{
	TArray<UDMXComponent*> DMXComponents;
	Actor->GetComponents<UDMXComponent>(DMXComponents);
	if (!ensureAlwaysMsgf(!DMXComponents.IsEmpty(), TEXT("Cannot find DMX component for Actor '%s'. Cannot get Fixture Patch from Actor."), *Actor->GetName()))
	{
		return nullptr;
	}
	ensureAlwaysMsgf(DMXComponents.Num() == 1, TEXT("Actor '%s' has more than one DMX component. A single DMX component is required to clearly identify the fixture. Cannot get Fixture Patch from Actor."), *Actor->GetName());

	UDMXEntityFixturePatch* FixturePatch = DMXComponents[0]->GetFixturePatch();

	return FixturePatch;
}

FReply FDMXMVRSceneActorDetails::OnFixtureTypeToActorClassGroupSelected(UObject* FixtureTypeObject)
{
	UDMXEntityFixtureType* FixtureType = Cast<UDMXEntityFixtureType>(FixtureTypeObject);
	if (!FixtureType)
	{
		return FReply::Unhandled();
	}

	const TArray<TWeakObjectPtr<UObject>> SelectedObjects = PropertyUtilities->GetSelectedObjects();

	for (const TWeakObjectPtr<UObject>& SelectedObject : SelectedObjects)
	{
		if (ADMXMVRSceneActor* MVRSceneActor = Cast<ADMXMVRSceneActor>(SelectedObject.Get()))
		{
			const TArray<AActor*> ActorsForThisFixtureType = MVRSceneActor->GetActorsSpawnedForFixtureType(FixtureType);
			UEditorActorSubsystem* EditorActorSubsystem = GEditor->GetEditorSubsystem<UEditorActorSubsystem>();

			if (EditorActorSubsystem)
			{
				EditorActorSubsystem->SetSelectedLevelActors(ActorsForThisFixtureType);
			}
		}
	}

	// Set focus on the Scene Outliner so the user can execute keyboard commands right away
	const TWeakPtr<class ILevelEditor> LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor")).GetLevelEditorInstance();
	const TSharedPtr<class ISceneOutliner> SceneOutliner = LevelEditor.IsValid() ? LevelEditor.Pin()->GetMostRecentlyUsedSceneOutliner() : nullptr;
	if (SceneOutliner.IsValid())
	{
		SceneOutliner->SetKeyboardFocus();
	}

	return FReply::Handled();
}

void FDMXMVRSceneActorDetails::OnFixturePatchChanged(const UDMXEntityFixturePatch* FixturePatch)
{
	RequestRefresh();
}

void FDMXMVRSceneActorDetails::OnFixtureTypeChanged(const UDMXEntityFixtureType* FixtureType)
{
	RequestRefresh();
}

void FDMXMVRSceneActorDetails::OnMapChange(uint32 MapChangeFlags)
{
	RequestRefresh();
}

void FDMXMVRSceneActorDetails::OnActorDeleted(AActor* DeletedActor)
{
	RequestRefresh();
}

void FDMXMVRSceneActorDetails::OnPreEditChangeActorClassInFixtureTypeToActorClasses()
{
	for (TWeakObjectPtr<ADMXMVRSceneActor> WeakMVRSceneActor : OuterSceneActors)
	{
		if (ADMXMVRSceneActor* MVRSceneActor = WeakMVRSceneActor.Get())
		{
			MVRSceneActor->PreEditChange(FDMXMVRSceneFixtureTypeToActorClassPair::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FDMXMVRSceneFixtureTypeToActorClassPair, ActorClass)));
		}
	}
}

void FDMXMVRSceneActorDetails::OnPostEditChangeActorClassInFixtureTypeToActorClasses()
{
	for (TWeakObjectPtr<ADMXMVRSceneActor> WeakMVRSceneActor : OuterSceneActors)
	{
		if (ADMXMVRSceneActor* MVRSceneActor = WeakMVRSceneActor.Get())
		{
			MVRSceneActor->PostEditChange();
		}
	}
}

void FDMXMVRSceneActorDetails::RequestRefresh()
{
	PropertyUtilities->RequestRefresh();
}

#undef LOCTEXT_NAMESPACE
