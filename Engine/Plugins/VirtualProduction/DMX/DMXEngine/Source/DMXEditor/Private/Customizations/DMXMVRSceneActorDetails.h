// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Input/Reply.h"

class ADMXMVRSceneActor;
class IPropertyHandle;
class IPropertyUtilities;
class UDMXEntityFixturePatch;
class UDMXEntityFixtureType;
enum class ECheckBoxState : uint8;

/** Details customization for the 'FixtureType FunctionProperties' details view */
class FDMXMVRSceneActorDetails
	: public IDetailCustomization
{
public:
	/** Creates an instance of this details customization */
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization interface

private:
	/** Creates the DMX Library related section */
	void CreateDMXLibrarySection(IDetailLayoutBuilder& DetailBuilder);

	/** Creates the section where the user can select an actor class for each Fixture Type in the MVR Scene */
	void CreateFixtureTypeToActorClassSection(IDetailLayoutBuilder& DetailBuilder);

	/** Returns the Fixture Patch from an Actor, or nullptr if the Actor has no Fixture Patch set */
	UDMXEntityFixturePatch* GetFixturePatchFromActor(AActor* Actor) const;

	/** Called when the Refresh Actors from DMX Library button was clicked */
	FReply OnRefreshActorsFromDMXLibraryClicked();

	/** Called when the Write Transforms To DMX Library button was clicked */
	FReply OnWriteTransformsToDMXLibraryClicked();

	/** Called when a Fixture Type to Actor Class Group was selected */
	FReply OnFixtureTypeToActorClassGroupSelected(UObject* FixtureTypeObject);

	/** Called when a Fixture Patch changed */
	void OnFixturePatchChanged(const UDMXEntityFixturePatch* FixturePatch);

	/** Called when a Fixture Type changed */
	void OnFixtureTypeChanged(const UDMXEntityFixtureType* FixtureType);

	/** Called when a sub-level is loaded */
	void OnMapChange(uint32 MapChangeFlags);

	/** Called when an actor got deleted in editor */
	void OnActorDeleted(AActor* DeletedActor);

	/** Called before an Actor Class changed in the FixtureTypeToActorClasses member of the Actor */
	void OnPreEditChangeActorClassInFixtureTypeToActorClasses();

	/** Called after an Actor Class changed in the FixtureTypeToActorClasses member of the Actor */
	void OnPostEditChangeActorClassInFixtureTypeToActorClasses();

	/** Requests this Details Customization to refresh */
	void RequestRefresh();

	/** The Actors being customized in this Detais Customization */
	TArray<TWeakObjectPtr<ADMXMVRSceneActor>> OuterSceneActors;

	/** Property Utilities for this Details Customization */
	TSharedPtr<IPropertyUtilities> PropertyUtilities;
};
