// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/DMXPixelMappingDistributionCustomization.h"

#include "DMXEditorStyle.h"
#include "DMXProtocolTypes.h"

#include "DetailWidgetRow.h"
#include "PropertyHandle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SUniformGridPanel.h"


TSharedRef<IPropertyTypeCustomization> FDMXPixelMappingDistributionCustomization::MakeInstance()
{
	return MakeShared<FDMXPixelMappingDistributionCustomization>();
}

void FDMXPixelMappingDistributionCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	EnumPropertyHandle = InStructPropertyHandle;

	TSharedRef<SUniformGridPanel> DistributionGridPanel = 
		SNew(SUniformGridPanel)
		.SlotPadding(FMargin(1.f));

	for (int32 XIndex = 0; XIndex < DistributionGridNumXPanels; ++XIndex)
	{
		for (int32 YIndex = 0; YIndex < DistributionGridNumYPanels; ++YIndex)
		{
			FString BrushPath = FString::Printf(TEXT("DMXEditor.PixelMapping.DistributionGrid.%d.%d"), XIndex, YIndex);

			TSharedPtr<SButton> Button = SNew(SButton)
				.ButtonColorAndOpacity(TAttribute<FSlateColor>::Create(TAttribute<FSlateColor>::FGetter::CreateSP(this, &FDMXPixelMappingDistributionCustomization::GetButtonColorAndOpacity, XIndex, YIndex)))
				.OnClicked(FOnClicked::CreateSP(this, &FDMXPixelMappingDistributionCustomization::OnGridButtonClicked, XIndex, YIndex))
				[
					SNew(SImage)
					.Image(FDMXEditorStyle::Get().GetBrush(*BrushPath))
				];

				DistributionGridPanel->AddSlot(XIndex, YIndex)
				[
					Button.ToSharedRef()
				];
		}
	}

	InHeaderRow
		.NameContent()
		[
			EnumPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(200.0f)
		.MaxDesiredWidth(400.0f)
		[
			DistributionGridPanel
		];
}

FReply FDMXPixelMappingDistributionCustomization::OnGridButtonClicked(int32 GridIndexX, int32 GridIndexY)
{
	if (EnumPropertyHandle.IsValid())
	{
		uint8 ChoosenDistribution = (GridIndexX * DistributionGridNumXPanels + GridIndexY);

		EnumPropertyHandle->NotifyPreChange();
		EnumPropertyHandle->SetValue(ChoosenDistribution);
		EnumPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	}

	return FReply::Handled();
}

FSlateColor FDMXPixelMappingDistributionCustomization::GetButtonColorAndOpacity(int32 GridIndexX, int32 GridIndexY)
{
	if (EnumPropertyHandle.IsValid())
	{
		uint8 CurrentDistribution = 0;
		if (EnumPropertyHandle->GetValue(CurrentDistribution) == FPropertyAccess::Result::Success)
		{
			if (CurrentDistribution == (GridIndexX * DistributionGridNumXPanels + GridIndexY))
			{
				return FLinearColor(0.2f, 0.2f, 0.2f, 1.f);
			}
		}
	}

	return FLinearColor::Transparent;
}
