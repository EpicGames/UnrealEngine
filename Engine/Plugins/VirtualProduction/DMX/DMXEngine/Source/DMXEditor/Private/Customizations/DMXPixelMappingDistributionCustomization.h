// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolTypes.h"

#include "IPropertyTypeCustomization.h"

class FReply;

/**
 * Customization for the DMXPixelMappingDistribution enum
 */
class FDMXPixelMappingDistributionCustomization
	: public IPropertyTypeCustomization
{
public:
	/** Creates an instance of this property type customization */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	//~ IPropertyTypeCustomization interface begin
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InStructBuilder, IPropertyTypeCustomizationUtils& InStructCustomizationUtils) override {}
	//~ IPropertyTypeCustomization interface end

protected:
	/** Gets the color and opacity of the enum value buttons */
	FSlateColor GetButtonColorAndOpacity(int32 GridIndexX, int32 GridIndexY);

	/** Called when an enum value button in the grid was clicked */
	FReply OnGridButtonClicked(int32 GridIndexX, int32 GridIndexY);

private:
	/** Number of panels in X direction */
	static const uint8 DistributionGridNumXPanels = 4;

	/** Number of buttons in Y direction */
	static const uint8 DistributionGridNumYPanels = 4;

	/** Handle for the enum property that is customized */
	TSharedPtr<IPropertyHandle> EnumPropertyHandle;
};
