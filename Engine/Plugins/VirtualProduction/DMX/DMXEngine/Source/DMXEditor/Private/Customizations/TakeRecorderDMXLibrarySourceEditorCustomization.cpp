// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/TakeRecorderDMXLibrarySourceEditorCustomization.h"
#include "Sequencer/TakeRecorderDMXLibrarySource.h"
#include "Library/DMXLibrary.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailPropertyRow.h"
#include "Input/Reply.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "FTakeRecorderDMXLibrarySourceEditorCustomization"

TSharedRef<IPropertyTypeCustomization> FTakeRecorderDMXLibrarySourceEditorCustomization::MakeInstance()
{
	return MakeShared<FTakeRecorderDMXLibrarySourceEditorCustomization>();
}

void FTakeRecorderDMXLibrarySourceEditorCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	StructHandle = PropertyHandle;

	// Cache the DMX Library TakeRecorder objects being displayed by the DetailsView that created this customization
	TArray<UObject*> CustomizedObjects;
	PropertyHandle->GetOuterObjects(CustomizedObjects);
	CustomizedDMXRecorders.Reserve(CustomizedObjects.Num());

	for (UObject* Object : CustomizedObjects)
	{
		if (Object == nullptr || !Object->IsValidLowLevelFast())
		{
			continue;
		}

		UTakeRecorderDMXLibrarySource* DMXSource = Cast<UTakeRecorderDMXLibrarySource>(Object);
		if (DMXSource != nullptr)
		{
			CustomizedDMXRecorders.Emplace(DMXSource);
		}
	}
	check(CustomizedDMXRecorders.Num());

	// Create the "Add all patches" button row
	HeaderRow
		.NameContent()
		[
			SNullWidget::NullWidget // Empty label 'cause the button already has its label
		]
		.ValueContent()
		.MinDesiredWidth(0.0f)
		.MaxDesiredWidth(0.0f)
		.HAlign(HAlign_Left)
		[
			SNew(SButton)
			.IsEnabled(this, &FTakeRecorderDMXLibrarySourceEditorCustomization::GetAddAllEnabled)
			.OnClicked(this, &FTakeRecorderDMXLibrarySourceEditorCustomization::HandleOnClicked)
			.ToolTipText(LOCTEXT("ToolTip", "Add all Patches in the DMX Library to be recorded"))
			.Content()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("AddAllPatches", "Add all Fixture Patches"))
			]
		];
}

bool FTakeRecorderDMXLibrarySourceEditorCustomization::GetAddAllEnabled() const
{
	// All selected DMX recorders must have valid DMX Libraries to enable the button
	for (TWeakObjectPtr<UTakeRecorderDMXLibrarySource> DMXRecorder : CustomizedDMXRecorders)
	{
		if (!DMXRecorder.IsValid() || !DMXRecorder->IsValidLowLevelFast()
			|| DMXRecorder->DMXLibrary == nullptr || !DMXRecorder->DMXLibrary->IsValidLowLevelFast())
		{
			return false;
		}
	}

	return true;
}

FReply FTakeRecorderDMXLibrarySourceEditorCustomization::HandleOnClicked()
{
	check(StructHandle.IsValid() && StructHandle->IsValidHandle());

	// Scope to end the Transaction before calling NotifyFinishedChangingProperties,
	// like it's done when using IPropertyHandle::SetValue
	{
		const FScopedTransaction Transaction(LOCTEXT("Transaction", "Add all Fixture Patches for recording"));

		// Notify objects that own this struct so they can call Modify to record their current states
		StructHandle->NotifyPreChange();

		// Add all patches on each selected DMX Take Recorder
		for (TWeakObjectPtr<UTakeRecorderDMXLibrarySource> DMXRecorder : CustomizedDMXRecorders)
		{
			if (DMXRecorder.IsValid() && DMXRecorder->IsValidLowLevelFast())
			{
				DMXRecorder->AddAllPatches();
			}
		}

		StructHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	}

	StructHandle->NotifyFinishedChangingProperties();

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
