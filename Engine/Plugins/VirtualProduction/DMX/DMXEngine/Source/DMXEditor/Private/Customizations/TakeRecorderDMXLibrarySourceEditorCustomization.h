// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FReply;
class IPropertyHandle;
class UTakeRecorderDMXLibrarySource;

class FTakeRecorderDMXLibrarySourceEditorCustomization
	: public IPropertyTypeCustomization
{
public:
	/** Creates an instance of this property type customization */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	//~ Begin IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override {}
	//~ End IPropertyTypeCustomization interface

private:
	/** Checks whether all selected DMX Take Recorders have valid DMX Libraries */
	bool GetAddAllEnabled() const;
	/** Add all Patches to be recorded on each DMX Take Recorder */
	FReply HandleOnClicked();

private:
	/**
	 * Cache of the TakeRecorder sources being customized by the DetailsView that
	 * created this customization
	 */
	TArray<TWeakObjectPtr<UTakeRecorderDMXLibrarySource>> CustomizedDMXRecorders;

	/** Used to generate Change events on the root DMX Take Recorder objects */
	TSharedPtr<IPropertyHandle> StructHandle;
};
