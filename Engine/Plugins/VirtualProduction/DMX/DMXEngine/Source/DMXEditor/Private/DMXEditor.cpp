// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXEditor.h"

#include "Commands/DMXEditorCommands.h"
#include "DesktopPlatformModule.h"
#include "DMXEditorLog.h"
#include "DMXEditorModule.h"
#include "DMXEditorSettings.h"
#include "DMXEditorTabNames.h"
#include "DMXFixturePatchSharedData.h"
#include "DMXFixtureTypeSharedData.h"
#include "DMXRuntimeLog.h"
#include "Exporters/DMXMVRExporter.h"
#include "Framework/Application/SlateApplication.h"
#include "IDesktopPlatform.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/DMXEntityReference.h"
#include "Library/DMXLibrary.h"
#include "Misc/MessageDialog.h"
#include "Modes/DMXEditorApplicationMode.h"
#include "ScopedTransaction.h"
#include "Toolbars/DMXEditorToolbar.h"
#include "Utils.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/FixturePatch/SDMXFixturePatchEditor.h"
#include "Widgets/FixtureType/SDMXFixtureTypeEditor.h"
#include "Widgets/LibrarySettings/SDMXLibraryEditorTab.h"
#include "Widgets/SDMXEntityEditor.h"

#define LOCTEXT_NAMESPACE "FDMXEditor"

const FName FDMXEditor::ToolkitFName(TEXT("DMXEditor"));

FDMXEditor::FDMXEditor()
	: AnalyticsProvider("DMXLibraryEditor")
{}

FName FDMXEditor::GetToolkitFName() const
{
	return ToolkitFName;
}

FText FDMXEditor::GetBaseToolkitName() const
{
	return LOCTEXT("DMXEditor", "DMX Editor");
}

FString FDMXEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix_LevelScript", "Script ").ToString();
}

FLinearColor FDMXEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.0f, 0.0f, 0.3f, 0.5f);
}

void FDMXEditor::InitEditor(const EToolkitMode::Type Mode, const TSharedPtr<class IToolkitHost>& InitToolkitHost, UDMXLibrary* DMXLibrary)
{
	if (!Toolbar.IsValid())
	{
		Toolbar = MakeShared<FDMXEditorToolbar>(SharedThis(this));
	}

	if (!FixtureTypeSharedData.IsValid())
	{
		FixtureTypeSharedData = MakeShared<FDMXFixtureTypeSharedData>(SharedThis(this));
	}
	
	if (!FixturePatchSharedData.IsValid())
	{
		FixturePatchSharedData = MakeShared<FDMXFixturePatchSharedData>(SharedThis(this));
	}

	// Initialize the asset editor and spawn nothing (dummy layout)
	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	const TSharedRef<FTabManager::FLayout> DummyLayout = FTabManager::NewLayout("NullLayout")->AddArea(FTabManager::NewPrimaryArea());
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, FDMXEditorModule::DMXEditorAppIdentifier, DummyLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, (UObject*)DMXLibrary);

	CommonInitialization(DMXLibrary);

	InitalizeExtenders();

	RegenerateMenusAndToolbars();

	const bool bShouldOpenInDefaultsMode = true;
	bool bNewlyCreated = true;
	RegisterApplicationModes(DMXLibrary, bShouldOpenInDefaultsMode, bNewlyCreated);
}

void FDMXEditor::CommonInitialization(UDMXLibrary* DMXLibrary)
{
	CreateDefaultCommands();
	CreateDefaultTabContents(DMXLibrary);
}

void FDMXEditor::InitalizeExtenders()
{
	FDMXEditorModule* DMXEditorModule = &FDMXEditorModule::Get();
	TSharedPtr<FExtender> MenuExtender = DMXEditorModule->GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects());
	AddMenuExtender(MenuExtender);

	TSharedPtr<FExtender> ToolbarExtender = DMXEditorModule->GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects());
	AddToolbarExtender(ToolbarExtender);
}

void FDMXEditor::RegisterApplicationModes(UDMXLibrary* DMXLibrary, bool bShouldOpenInDefaultsMode, bool bNewlyCreated)
{
	// Only one for now
	FWorkflowCentricApplication::AddApplicationMode(
		FDMXEditorApplicationMode::DefaultsMode,
		MakeShared<FDMXEditorDefaultApplicationMode>(SharedThis(this)));
	FWorkflowCentricApplication::SetCurrentMode(FDMXEditorApplicationMode::DefaultsMode);
}

UDMXLibrary* FDMXEditor::GetDMXLibrary() const
{
	return Cast<UDMXLibrary>(GetEditingObject());
}

void FDMXEditor::ImportDMXLibrary() const
{
	UDMXLibrary* DMXLibrary = GetDMXLibrary();
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	UDMXEditorSettings* DMXEditorSettings = GetMutableDefault<UDMXEditorSettings>();
	if (!DMXLibrary || !DesktopPlatform || !DMXEditorSettings)
	{
		return;
	}

	if (!DMXLibrary->GetEntities().IsEmpty())
	{
		const FText MessageText = LOCTEXT("MVRImportDialog", "DMX Library already contains data. Importing the MVR will clear existing data. Do you want to proceed?");
		if (FMessageDialog::Open(EAppMsgType::YesNo, MessageText) == EAppReturnType::No)
		{
			return;
		}
	}
	
	const FString LastMVRImportPath = DMXEditorSettings->LastMVRImportPath;
	const FString DefaultPath = FPaths::DirectoryExists(LastMVRImportPath) ? LastMVRImportPath : FPaths::ProjectSavedDir();
	
	TArray<FString> OpenFilenames;
	DesktopPlatform->OpenFileDialog(
		FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
		LOCTEXT("ImportMVR", "Import MVR").ToString(),
		DefaultPath,
		TEXT(""),
		TEXT("My Virtual Rig (*.mvr)|*.mvr"),
		EFileDialogFlags::None,
		OpenFilenames);

	if (OpenFilenames.IsEmpty())
	{
		return;
	}

	if (ImportObject<UDMXLibrary>(DMXLibrary->GetOuter(), DMXLibrary->GetFName(), RF_Public | RF_Standalone, *OpenFilenames[0], nullptr))
	{
		DMXEditorSettings->LastMVRImportPath = FPaths::GetPath(OpenFilenames[0]);
		DMXEditorSettings->SaveConfig();
	}
}

void FDMXEditor::ExportDMXLibrary() const
{
	UDMXLibrary* DMXLibrary = GetDMXLibrary();
	if (!DMXLibrary)
	{
		return;
	}

	UE::DMX::FDMXMVRExporter::Export(DMXLibrary);
}

void FDMXEditor::RegisterToolbarTab(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);
}

void FDMXEditor::CreateDefaultTabContents(UDMXLibrary* DMXLibrary)
{
	DMXLibraryEditorTab = CreateDMXLibraryEditorTab();
	FixtureTypeEditor = CreateFixtureTypeEditor();
	FixturePatchEditor = CreateFixturePatchEditor();
}

void FDMXEditor::CreateDefaultCommands()
{
	FDMXEditorCommands::Register();

	ToolkitCommands->MapAction(
		FDMXEditorCommands::Get().ImportDMXLibrary,
		FExecuteAction::CreateSP(this, &FDMXEditor::ImportDMXLibrary)
	);
	ToolkitCommands->MapAction(
		FDMXEditorCommands::Get().ExportDMXLibrary,
		FExecuteAction::CreateSP(this, &FDMXEditor::ExportDMXLibrary)
	);
	ToolkitCommands->MapAction(
		FDMXEditorCommands::Get().AddNewEntityFixtureType,
		FExecuteAction::CreateLambda([this]() { OnAddNewEntity(UDMXEntityFixtureType::StaticClass()); })
	);
	ToolkitCommands->MapAction(
		FDMXEditorCommands::Get().AddNewEntityFixturePatch,
		FExecuteAction::CreateLambda([this]() { OnAddNewEntity(UDMXEntityFixturePatch::StaticClass()); })
	);
}

void FDMXEditor::OnAddNewEntity(TSubclassOf<UDMXEntity> InEntityClass)
{
	UDMXLibrary* DMXLibrary = GetDMXLibrary();
	check(DMXLibrary);

	if (InEntityClass == UDMXEntityFixtureType::StaticClass())
	{
		FDMXEntityFixtureTypeConstructionParams FixtureTypeConstructionParams;
		FixtureTypeConstructionParams.ParentDMXLibrary = DMXLibrary;

		UDMXEntityFixtureType* FixtureType = UDMXEntityFixtureType::CreateFixtureTypeInLibrary(FixtureTypeConstructionParams);
		FixtureTypeSharedData->SelectFixtureTypes(TArray<TWeakObjectPtr<UDMXEntityFixtureType>>({ FixtureType }));
	}
	else if (InEntityClass == UDMXEntityFixturePatch::StaticClass())
	{
		if (UDMXEntity* LastAddedEntity = DMXLibrary->GetLastAddedEntity().Get())
		{
			UDMXEntityFixtureType* LastAddedFixtureType = [LastAddedEntity]() -> UDMXEntityFixtureType*
			{
				if (UDMXEntityFixtureType* EntityAsFixtureType = Cast<UDMXEntityFixtureType>(LastAddedEntity))
				{
					return EntityAsFixtureType;
				}
				else if (UDMXEntityFixturePatch* EntityAsFixturePatch = Cast<UDMXEntityFixturePatch>(LastAddedEntity))
				{
					return EntityAsFixturePatch->GetFixtureType();
				}
				return nullptr;
			}();

			if (LastAddedFixtureType)
			{
				FDMXEntityFixturePatchConstructionParams FixturePatchConstructionParams;
				FixturePatchConstructionParams.FixtureTypeRef = FDMXEntityFixtureTypeRef(LastAddedFixtureType);

				const FScopedTransaction CreateFixtureTypeTransaction(LOCTEXT("CreateFixtureTypeTransaction", "Create DMX Fixture Type"));
				DMXLibrary->PreEditChange(nullptr);

				UDMXEntityFixturePatch* NewFixturePatch = UDMXEntityFixturePatch::CreateFixturePatchInLibrary(FixturePatchConstructionParams);
				FixturePatchSharedData->SelectFixturePatch(NewFixturePatch);

				DMXLibrary->PostEditChange();

				return;
			}
		}
		else
		{
			const TArray<UDMXEntityFixtureType*> FixtureTypesInLibrary = DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixtureType>();
			if (FixtureTypesInLibrary.Num() > 0)
			{
				FDMXEntityFixturePatchConstructionParams FixturePatchConstructionParams;
				FixturePatchConstructionParams.FixtureTypeRef = FDMXEntityFixtureTypeRef(FixtureTypesInLibrary[0]);

				const FScopedTransaction CreateFixtureTypeTransaction(LOCTEXT("CreateFixturePatchTransaction", "Create DMX Fixture Patch"));
				DMXLibrary->PreEditChange(nullptr);

				UDMXEntityFixturePatch* NewFixturePatch = UDMXEntityFixturePatch::CreateFixturePatchInLibrary(FixturePatchConstructionParams);
				FixturePatchSharedData->SelectFixturePatch(NewFixturePatch);

				DMXLibrary->PostEditChange();
			}
			else
			{
				UE_LOGF(LogDMXRuntime, Warning, "Cannot create a fixture patch in Library %ls when the Library doesn't define any Fixture Types.", *DMXLibrary->GetName());
			}
		}
	}
}

bool FDMXEditor::InvokeEditorTabFromEntityType(TSubclassOf<UDMXEntity> InEntityClass)
{
	using namespace UE::DMX;

	// Make sure we're in the right tab for the current type
	FName TargetTabId = NAME_None;
	if (InEntityClass->IsChildOf(UDMXEntityFixtureType::StaticClass()))
	{
		TargetTabId = FDMXEditorTabNames::DMXFixtureTypesEditor;
	}
	else if (InEntityClass->IsChildOf(UDMXEntityFixturePatch::StaticClass()))
	{
		TargetTabId = FDMXEditorTabNames::DMXFixturePatchEditor;
	}
	else
	{
		UE_LOG_DMXEDITOR(Error, TEXT("%S: Unimplemented Entity type. Can't set currect Tab."), __FUNCTION__);
	}

	if (!TargetTabId.IsNone())
	{
		FName CurrentTab = FGlobalTabmanager::Get()->GetActiveTab()->GetLayoutIdentifier().TabType;
		if (!CurrentTab.IsEqual(TargetTabId))
		{
			TabManager->TryInvokeTab(MoveTemp(TargetTabId));
		}
		
		return true;
	}

	return false;
}

bool FDMXEditor::NewEntity_IsVisibleForType(TSubclassOf<UDMXEntity> InEntityClass) const
{
	return true;
}

void FDMXEditor::RenameNewlyAddedEntity(UDMXEntity* InEntity, TSubclassOf<UDMXEntity> InEntityClass)
{
	TSharedPtr<SDMXEntityEditor> EntityEditor = GetEditorWidgetForEntityType(InEntityClass);
	if (!EntityEditor.IsValid())
	{
		return;
	}

	// if this check ever fails, something is really wrong!
	// How can an Entity be created without the button in the editor?!
	check(EntityEditor.IsValid());
	
	EntityEditor->RequestRenameOnNewEntity(InEntity, ESelectInfo::OnMouseClick);
}

TSharedPtr<SDMXEntityEditor> FDMXEditor::GetEditorWidgetForEntityType(TSubclassOf<UDMXEntity> InEntityClass) const
{
	TSharedPtr<SDMXEntityEditor> EntityEditor = nullptr;

	if (InEntityClass->IsChildOf(UDMXEntityFixtureType::StaticClass()))
	{
		return FixtureTypeEditor;
	}
	else if (InEntityClass->IsChildOf(UDMXEntityFixturePatch::StaticClass()))
	{
		return FixturePatchEditor;
	}
	else
	{
		UE_LOG_DMXEDITOR(Error, TEXT("%S not implemented for %s"), __FUNCTION__, *InEntityClass->GetFName().ToString());
	}

	return FixtureTypeEditor;
}

void FDMXEditor::SelectEntityInItsTypeTab(UDMXEntity* InEntity, ESelectInfo::Type InSelectionType /*= ESelectInfo::Type::Direct*/)
{
	check(InEntity != nullptr);

	if (!InvokeEditorTabFromEntityType(InEntity->GetClass()))
	{
		return;
	}

	if (TSharedPtr<SDMXEntityEditor> EntityEditor = GetEditorWidgetForEntityType(InEntity->GetClass()))
	{
		EntityEditor->SelectEntity(InEntity, InSelectionType);
	}
}

void FDMXEditor::SelectEntitiesInTypeTab(const TArray<UDMXEntity*>& InEntities, ESelectInfo::Type InSelectionType /*= ESelectInfo::Type::Direct*/)
{
	if (InEntities.Num() == 0 || InEntities[0] == nullptr)
	{
		return; 
	}

	if (!InvokeEditorTabFromEntityType(InEntities[0]->GetClass()))
	{
		return;
	}

	if (TSharedPtr<SDMXEntityEditor> EntityEditor = GetEditorWidgetForEntityType(InEntities[0]->GetClass()))
	{
		EntityEditor->SelectEntities(InEntities, InSelectionType);
	}
}

TArray<UDMXEntity*> FDMXEditor::GetSelectedEntitiesFromTypeTab(TSubclassOf<UDMXEntity> InEntityClass) const
{
	if (TSharedPtr<SDMXEntityEditor> EntityEditor = GetEditorWidgetForEntityType(InEntityClass))
	{
		return EntityEditor->GetSelectedEntities();
	}

	return TArray<UDMXEntity*>();
}

TSharedRef<SDMXLibraryEditorTab> FDMXEditor::CreateDMXLibraryEditorTab()
{
	UDMXLibrary* DMXLibrary = GetDMXLibrary();
	check(DMXLibrary);

	return SNew(SDMXLibraryEditorTab)
		.DMXLibrary(DMXLibrary)
		.DMXEditor(SharedThis(this));
}

TSharedRef<SDMXFixtureTypeEditor> FDMXEditor::CreateFixtureTypeEditor()
{
	return SNew(SDMXFixtureTypeEditor, SharedThis(this));
}

TSharedRef<SDMXFixturePatchEditor> FDMXEditor::CreateFixturePatchEditor()
{
	return SNew(SDMXFixturePatchEditor)
		.DMXEditor(SharedThis(this));
}

TSharedPtr<FDMXFixtureTypeSharedData> FDMXEditor::GetFixtureTypeSharedData() const
{
	return FixtureTypeSharedData;
}

const TSharedPtr<FDMXFixturePatchSharedData>& FDMXEditor::GetFixturePatchSharedData() const
{
	return FixturePatchSharedData;
}

#undef LOCTEXT_NAMESPACE
