// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

DECLARE_LOG_CATEGORY_CLASS(LogDMXEditor, Log, All);

#define UE_LOG_DMXEDITOR(Verbosity, Format, ...) \
{ \
	UE_LOG(LogDMXEditor, Verbosity, Format, ##__VA_ARGS__); \
}

#define UE_CLOG_DMXEDITOR(Conditional, Verbosity, Format, ...) \
{ \
	UE_CLOG(Conditional, LogDMXEditor, Verbosity, Format, ##__VA_ARGS__); \
}