// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define DMXEDITOR_GETENUMSTRING(ETypeName, EnumVal) StaticEnum<ETypeName>()->GetNameStringByValue(static_cast<int64>(EnumVal))
