// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXEditorModule.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "Commands/DMXConflictMonitorCommands.h"
#include "Commands/DMXEditorCommands.h"
#include "Customizations/DMXAttributeNameCustomization.h"
#include "Customizations/DMXAutoExpandedStructCustomization.h"
#include "Customizations/DMXEntityFixtureTypeDetails.h"
#include "Customizations/DMXEntityReferenceCustomization.h"
#include "Customizations/DMXFixtureCategoryCustomization.h"
#include "Customizations/DMXFixtureFunctionCustomization.h"
#include "Customizations/DMXFixtureSignalFormatCustomization.h"
#include "Customizations/DMXLibraryPortReferencesCustomization.h"
#include "Customizations/DMXMVRSceneActorDetails.h"
#include "Customizations/DMXPixelMappingDistributionCustomization.h"
#include "Customizations/TakeRecorderDMXLibrarySourceEditorCustomization.h"
#include "DMXAttribute.h"
#include "DMXEditor.h"
#include "DMXEditorStyle.h"
#include "DMXEditorTabNames.h"
#include "DMXProtocolBlueprintLibrary.h"
#include "DMXProtocolTypes.h"
#include "Exporters/DMXMVRExporter.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IAssetTools.h"
#include "ISequencerModule.h"
#include "LevelEditor.h"
#include "Library/DMXEntity.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/DMXEntityReference.h"
#include "Library/DMXImportGDTF.h"
#include "Library/DMXLibrary.h"
#include "Misc/MessageDialog.h"
#include "MVR/DMXMVRSceneActor.h"
#include "ObjectTools.h"
#include "PropertyEditorModule.h"
#include "Sequencer/DMXLibraryTrackEditor.h"
#include "Sequencer/TakeRecorderDMXLibrarySource.h"
#include "ToolMenus.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Monitors/SDMXActivityMonitor.h"
#include "Widgets/Monitors/SDMXChannelsMonitor.h"
#include "Widgets/Monitors/SDMXConflictMonitor.h"
#include "Widgets/PatchTool/SDMXPatchTool.h"

#define LOCTEXT_NAMESPACE "DMXEditorModule"

const FName FDMXEditorModule::DMXEditorAppIdentifier(TEXT("DMXEditorApp"));

EAssetTypeCategories::Type FDMXEditorModule::DMXEditorAssetCategory;

TSharedPtr<FExtender> FDMXEditorModule::LevelEditorToolbarDMXMenuExtender;

void FDMXEditorModule::StartupModule()
{
	FDMXEditorCommands::Register();
	BindDMXEditorCommands();

	FDMXConflictMonitorCommands::Register();

	MenuExtensibilityManager = MakeShared<FExtensibilityManager>();
	ToolBarExtensibilityManager = MakeShared<FExtensibilityManager>();

	RegisterAssetTypeCategories();

	RegisterClassCustomizations();
	RegisterPropertyTypeCustomizations();
	RegisterSequencerTypes();
	RegisterNomadTabSpawners();
	ExtendLevelEditorToolbar();
	CreateLevelEditorToolbarDMXMenuExtender();
	
	StartupPIEManager();

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	AssetRegistry.OnFilesLoaded().AddStatic(&FDMXEditorModule::FixGDTFAssetNames);
}

void FDMXEditorModule::ShutdownModule()
{
	if(UObjectInitialized())
	{
		UToolMenus::Get()->RemoveSection("LevelEditor.LevelEditorToolBar.User", "DMX");
	}
	
	FDMXEditorCommands::Unregister();
	FDMXConflictMonitorCommands::Unregister();

	MenuExtensibilityManager.Reset();
	ToolBarExtensibilityManager.Reset();
	DMXLevelEditorMenuCommands.Reset();

	UnregisterCustomClassLayouts();
	UnregisterCustomPropertyTypeLayouts();
	UnregisterCustomSequencerTrackTypes();
}

FDMXEditorModule& FDMXEditorModule::Get()
{
	return FModuleManager::GetModuleChecked<FDMXEditorModule>("DMXEditor");
}

TSharedRef<FDMXEditor> FDMXEditorModule::CreateEditor(const EToolkitMode::Type Mode, const TSharedPtr<class IToolkitHost>& InitToolkitHost, UDMXLibrary * DMXLibrary)
{
	TSharedRef<FDMXEditor> NewDMXEditor = MakeShared<FDMXEditor>();
	NewDMXEditor->InitEditor(Mode, InitToolkitHost, DMXLibrary);

	return NewDMXEditor;
}

void FDMXEditorModule::ExportDMXLibraryAsMVRFile(UDMXLibrary* DMXLibrary, const FString& DesiredName) const
{
	if (DMXLibrary)
	{
		UE::DMX::FDMXMVRExporter::Export(DMXLibrary, DesiredName);
	}
}

void FDMXEditorModule::FixGDTFAssetNames()
{
	if (!UObjectInitialized())
	{
		return;
	}

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(FName("AssetRegistry")).Get();

	constexpr bool bSearchSubclasses = true;
	TArray<FAssetData> GDTFAssets;
	AssetRegistry.GetAssetsByClass(UDMXImportGDTF::StaticClass()->GetClassPathName(), GDTFAssets, bSearchSubclasses);

	TArray<UDMXImportGDTF*> InvalidGDTFAssets;
	Algo::TransformIf(GDTFAssets, InvalidGDTFAssets,
		[](const FAssetData& Asset)
		{
			return 
				Asset.IsValid() &&
				ObjectTools::SanitizeObjectName(Asset.AssetName.ToString()) != Asset.AssetName;
		},
		[](const FAssetData& Asset)
		{
			return Cast<UDMXImportGDTF>(Asset.GetAsset());
		});

	if (InvalidGDTFAssets.IsEmpty())
	{
		return;
	}

	FText MessageText = LOCTEXT("FixGDTFAssetNames", "Detected GDTF assets with invalid asset names.\nWould you like to rename them now?\n\n");
	for (UDMXImportGDTF* GDTF : InvalidGDTFAssets)
	{
		MessageText = FText::Format(FText::FromString("{0}\n{1} -> {2}"), MessageText, FText::FromString(GDTF->GetName()), FText::FromString(ObjectTools::SanitizeObjectName(GDTF->GetName())));
	}
	
	if (FMessageDialog::Open(EAppMsgType::YesNo, MessageText) == EAppReturnType::No)
	{
		return;
	}

	for (UDMXImportGDTF* GDTF : InvalidGDTFAssets)
	{
		if (GDTF->GetOutermostObject() == GDTF->GetPackage() && GDTF->HasAnyFlags(RF_ClassDefaultObject | RF_Standalone | RF_Public))
		{
			FString SaneName = ObjectTools::SanitizeObjectName(GDTF->GetName());
			if (!IsUniqueObjectName(*SaneName, GDTF))
			{
				SaneName = MakeUniqueObjectName(GDTF->GetOutermost(), GDTF->GetClass(), *SaneName).ToString();
			}

			const FSoftObjectPath OldObjectPath = FSoftObjectPath(GDTF);

			const FTopLevelAssetPath NewAssetPath(*GDTF->GetPathName(), *SaneName);
			const FSoftObjectPath NewObjectPath(NewAssetPath);

			TArray<FAssetRenameData> AssetToRename;
			AssetToRename.Add(FAssetRenameData(OldObjectPath, NewObjectPath));

			FAssetToolsModule& Module = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
			Module.Get().RenameAssets(AssetToRename);
		}
	}
}

void FDMXEditorModule::BindDMXEditorCommands()
{
	check(!DMXLevelEditorMenuCommands.IsValid());

	FLevelEditorModule& LevelEditorModule = FModuleManager::Get().LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedRef<FUICommandList> CommandList = LevelEditorModule.GetGlobalLevelEditorActions();

	CommandList->MapAction(
		FDMXEditorCommands::Get().OpenChannelsMonitor,
		FExecuteAction::CreateStatic(&FDMXEditorModule::OpenChannelsMonitor)
	);
	CommandList->MapAction(
		FDMXEditorCommands::Get().OpenActivityMonitor,
		FExecuteAction::CreateStatic(&FDMXEditorModule::OpenActivityMonitor)
	);
	CommandList->MapAction(
		FDMXEditorCommands::Get().OpenConflictMonitor,
		FExecuteAction::CreateStatic(&FDMXEditorModule::OpenConflictMonitor)
	);
	CommandList->MapAction(
		FDMXEditorCommands::Get().OpenPatchTool,
		FExecuteAction::CreateStatic(&FDMXEditorModule::OpenPatchTool)
	);
	CommandList->MapAction(
		FDMXEditorCommands::Get().ToggleReceiveDMX,
		FExecuteAction::CreateStatic(&FDMXEditorModule::OnToggleReceiveDMX),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic(&FDMXEditorModule::IsReceiveDMXEnabled)
	);
	CommandList->MapAction(
		FDMXEditorCommands::Get().ToggleSendDMX,
		FExecuteAction::CreateStatic(&FDMXEditorModule::OnToggleSendDMX),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic(&FDMXEditorModule::IsSendDMXEnabled)
	);
}

void FDMXEditorModule::ExtendLevelEditorToolbar()
{
	UToolMenu* const Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorToolBar.User");

	FToolMenuSection& Section = Menu->FindOrAddSection("DMX");

	FToolMenuEntry DMXEntry = FToolMenuEntry::InitComboButton(
		"DMXMenu",
		FUIAction(),
		FOnGetContent::CreateStatic(&FDMXEditorModule::GenerateDMXLevelEditorToolbarMenu),
		LOCTEXT("LevelEditorToolbarDMXButtonLabel", "DMX"),
		LOCTEXT("LevelEditorToolbarDMXButtonTooltip", "DMX Tools"),
		TAttribute<FSlateIcon>::CreateLambda([]()
			{
				bool bSendEnabled = UDMXProtocolBlueprintLibrary::IsSendDMXEnabled();
				bool bReceiveEnabled = UDMXProtocolBlueprintLibrary::IsReceiveDMXEnabled();

				if (bSendEnabled && bReceiveEnabled)
				{
					static const FSlateIcon IconSendReceiveDMXEnabled = FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.SendReceiveDMXEnabled");
					return IconSendReceiveDMXEnabled;
				}
				else if (bSendEnabled)
				{
					static const FSlateIcon IconSendDMXEnabled = FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.SendDMXEnabled");
					return IconSendDMXEnabled;
				}
				else if (bReceiveEnabled)
				{
					static const FSlateIcon IconReceiveDMXEnabled = FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.ReceiveDMXEnabled");
					return IconReceiveDMXEnabled;
				}
				else
				{
					static const FSlateIcon IconSendReceiveDMXDisabled = FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.SendReceiveDMXDisabled");
					return IconSendReceiveDMXDisabled;
				}
			})
		);

	DMXEntry.StyleNameOverride = "CalloutToolbar";
	Section.AddEntry(DMXEntry);
}

void FDMXEditorModule::CreateLevelEditorToolbarDMXMenuExtender()
{
	LevelEditorToolbarDMXMenuExtender = MakeShared<FExtender>();
}

TSharedRef<SWidget> FDMXEditorModule::GenerateDMXLevelEditorToolbarMenu()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<FUICommandList> CommandBindings = LevelEditorModule.GetGlobalLevelEditorActions();

	FMenuBuilder MenuBuilder(true, CommandBindings, LevelEditorToolbarDMXMenuExtender);

	static const FName NoExtensionHook = NAME_None;
	MenuBuilder.BeginSection(NoExtensionHook, LOCTEXT("MenuSectionDMXTools", "DMX Tools"));
	{
		MenuBuilder.AddMenuEntry(FDMXEditorCommands::Get().OpenChannelsMonitor,
			"ChannelsMonitor",
			TAttribute<FText>(), 
			TAttribute<FText>(),
			FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.ChannelsMonitor")
		);
		MenuBuilder.AddMenuEntry(FDMXEditorCommands::Get().OpenActivityMonitor,
			"ActivityMonitor",
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.ActivityMonitor")
		);
		MenuBuilder.AddMenuEntry(FDMXEditorCommands::Get().OpenConflictMonitor,
			"ConflictMonitor",
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.ConflictMonitor")
		);
		MenuBuilder.AddMenuEntry(FDMXEditorCommands::Get().OpenPatchTool,
			"PatchTool",
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.PatchTool")
		);
	}

	MenuBuilder.BeginSection(NoExtensionHook, LOCTEXT("MenuSectionDMXInputOutput", "DMX IO"));
	{
		MenuBuilder.AddMenuEntry(FDMXEditorCommands::Get().ToggleReceiveDMX,
			"ReceiveDMXEnabled",
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.ReceiveDMX")
		);
		MenuBuilder.AddMenuEntry(FDMXEditorCommands::Get().ToggleSendDMX,
			"SendDMXEnabled",
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.SendDMX")
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void FDMXEditorModule::RegisterAssetTypeCategories()
{
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	DMXEditorAssetCategory = AssetTools.RegisterAdvancedAssetCategory("VirtualProduction", LOCTEXT("VirtualProductionCategory", "Virtual Production"));
}

void FDMXEditorModule::RegisterClassCustomizations()
{
	using namespace UE::DMX;

	// Details customization for the UDMXEntityFixtureType class
	RegisterCustomClassLayout(UDMXEntityFixtureType::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FDMXEntityFixtureTypeDetails::MakeInstance)
	);

	// Details customization for the ADMXMVRSceneActor class
	RegisterCustomClassLayout(ADMXMVRSceneActor::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FDMXMVRSceneActorDetails::MakeInstance)
	);
}

void FDMXEditorModule::RegisterPropertyTypeCustomizations()
{	
	using namespace UE::DMX;

	// Property type customization for the EDMXPixelMappingDistribution enum
	RegisterCustomPropertyTypeLayout("EDMXPixelMappingDistribution", 
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDMXPixelMappingDistributionCustomization::MakeInstance)
	);

	// Property type customization for the EDMXFixtureSignalFormat enum
	RegisterCustomPropertyTypeLayout("EDMXFixtureSignalFormat", 
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDMXFixtureSignalFormatCustomization::MakeInstance)
	);

	// Property type customization for the FDMXFixtureFunction struct
	RegisterCustomPropertyTypeLayout(FDMXFixtureFunction::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDMXFixtureFunctionCustomization::MakeInstance)
	);

	// Property type customization for the FDMXAttributeName struct
	RegisterCustomPropertyTypeLayout(FDMXAttributeName::StaticStruct()->GetFName(), 
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDMXAttributeNameCustomization::MakeInstance)
	);

	// Property type customization for the FDMXFixtureCategory struct
	RegisterCustomPropertyTypeLayout(FDMXFixtureCategory::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDMXFixtureCategoryCustomization::MakeInstance)
	);

	// Customizations for FDMXEntityReference structs
	RegisterCustomPropertyTypeLayout(FDMXEntityFixtureTypeRef::StaticStruct()->GetFName(), 
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDMXEntityReferenceCustomization::MakeInstance)
	);

	// Customization for the FDMXEntityFixturePatchRef struct
	RegisterCustomPropertyTypeLayout(FDMXEntityFixturePatchRef::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDMXEntityReferenceCustomization::MakeInstance)
	);

	// Customization for the FDMXLibraryPortReferences struct
	RegisterCustomPropertyTypeLayout(FDMXLibraryPortReferences::StaticStruct()->GetFName(), 
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDMXLibraryPortReferencesCustomization::MakeInstance)
	);

	// Customization for the DMXLibrary TakeRecorder AddAllPatchesButton
	RegisterCustomPropertyTypeLayout(FAddAllPatchesButton::StaticStruct()->GetFName(), 
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTakeRecorderDMXLibrarySourceEditorCustomization::MakeInstance)
	);

	// Customizations for auto expanded structs. The listed structs need to be auto expanded by default in most use cases.
	const TArray<FName> StructNames
	{
		FDMXFixtureMode::StaticStruct()->GetFName(),
		FDMXInputPortReference::StaticStruct()->GetFName(),
		FDMXOutputPortReference::StaticStruct()->GetFName()
	};

	for (const FName& StructName : StructNames)
	{
		RegisterCustomPropertyTypeLayout
		(
			StructName,
			FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDMXAutoExpandedStructCustomization::MakeInstance)
		);
	}
}

void FDMXEditorModule::RegisterSequencerTypes()
{
	// Register the DMX Library Sequencer track type
	RegisterCustomSequencerTrackType(FOnCreateTrackEditor::CreateStatic(&FDMXLibraryTrackEditor::CreateTrackEditor));
}

void FDMXEditorModule::RegisterNomadTabSpawners()
{
	using namespace UE::DMX;

	RegisterNomadTabSpawner(FDMXEditorTabNames::ChannelsMonitor,
		FOnSpawnTab::CreateStatic(&FDMXEditorModule::OnSpawnChannelsMonitorTab))
		.SetDisplayName(LOCTEXT("ChannelsMonitorTabTitle", "DMX Channel Monitor"))
		.SetMenuType(ETabSpawnerMenuType::Hidden)
		.SetIcon(FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.ChannelsMonitor"));

	RegisterNomadTabSpawner(FDMXEditorTabNames::ActivityMonitor,
		FOnSpawnTab::CreateStatic(&FDMXEditorModule::OnSpawnActivityMonitorTab))
		.SetDisplayName(LOCTEXT("ActivityMonitorTabTitle", "DMX Activity Monitor"))
		.SetMenuType(ETabSpawnerMenuType::Hidden)
		.SetIcon(FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.ActivityMonitor"));

	RegisterNomadTabSpawner(FDMXEditorTabNames::ConflictMonitor,
		FOnSpawnTab::CreateStatic(&FDMXEditorModule::OnSpawnConflictMonitorTab))
		.SetDisplayName(LOCTEXT("OutputConflictMonitorTabTitle", "DMX Output Conflict Monitor"))
		.SetMenuType(ETabSpawnerMenuType::Hidden)
		.SetIcon(FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.ConflictMonitor"));

	RegisterNomadTabSpawner(FDMXEditorTabNames::PatchTool,
		FOnSpawnTab::CreateStatic(&FDMXEditorModule::OnSpawnPatchToolTab))
		.SetDisplayName(LOCTEXT("PatchToolTabTitle", "DMX Patch Tool"))
		.SetMenuType(ETabSpawnerMenuType::Hidden)
		.SetIcon(FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.PatchTool"));
}

void FDMXEditorModule::StartupPIEManager()
{
	PIEManager = MakeUnique<FDMXPIEManager>();
}

TSharedRef<SDockTab> FDMXEditorModule::OnSpawnChannelsMonitorTab(const FSpawnTabArgs& InSpawnTabArgs)
{
	return SNew(SDockTab)
		.Label(LOCTEXT("ChannelsMonitorTitle", "DMX Channel Monitor"))
		.TabRole(ETabRole::NomadTab)
		[
			SNew(SDMXChannelsMonitor)
		];
}

TSharedRef<SDockTab> FDMXEditorModule::OnSpawnActivityMonitorTab(const FSpawnTabArgs& InSpawnTabArgs)
{
	return SNew(SDockTab)
		.Label(LOCTEXT("ActivityMonitorTitle", "DMX Activity Monitor"))
		.TabRole(ETabRole::NomadTab)
		[
			SNew(SDMXActivityMonitor)
		];
}

TSharedRef<SDockTab> FDMXEditorModule::OnSpawnConflictMonitorTab(const FSpawnTabArgs& InSpawnTabArgs)
{
	using namespace UE::DMX;
	return SNew(SDockTab)
		.Label(LOCTEXT("ConflictMonitorTitle", "DMX Output Conflict Monitor"))
		.TabRole(ETabRole::NomadTab)
		[
			SNew(SDMXConflictMonitor)
		];
}

TSharedRef<SDockTab> FDMXEditorModule::OnSpawnPatchToolTab(const FSpawnTabArgs& InSpawnTabArgs)
{
	return SNew(SDockTab)
		.Label(LOCTEXT("PatchToolTitle", "DMX Patch Tool"))
		.TabRole(ETabRole::NomadTab)
		[
			SNew(UE::DMX::SDMXPatchTool)
		];
}

void FDMXEditorModule::OpenChannelsMonitor()
{
	using namespace UE::DMX;
	FGlobalTabmanager::Get()->TryInvokeTab(FDMXEditorTabNames::ChannelsMonitor);
}

void FDMXEditorModule::OpenActivityMonitor()
{
	using namespace UE::DMX;
	FGlobalTabmanager::Get()->TryInvokeTab(FDMXEditorTabNames::ActivityMonitor);
}

void FDMXEditorModule::OpenConflictMonitor()
{
	using namespace UE::DMX;
	FGlobalTabmanager::Get()->TryInvokeTab(FDMXEditorTabNames::ConflictMonitor);
}

void FDMXEditorModule::OpenPatchTool()
{
	using namespace UE::DMX;
	FGlobalTabmanager::Get()->TryInvokeTab(FDMXEditorTabNames::PatchTool);
}

void FDMXEditorModule::OnToggleSendDMX()
{
	bool bAffectEditor = true;

	if (UDMXProtocolBlueprintLibrary::IsSendDMXEnabled())
	{
		UDMXProtocolBlueprintLibrary::SetSendDMXEnabled(false, bAffectEditor);
	}
	else
	{
		UDMXProtocolBlueprintLibrary::SetSendDMXEnabled(true, bAffectEditor);
	}
}

bool FDMXEditorModule::IsSendDMXEnabled()
{
	return UDMXProtocolBlueprintLibrary::IsSendDMXEnabled();
}

void FDMXEditorModule::OnToggleReceiveDMX()
{
	bool bAffectEditor = true;

	if (UDMXProtocolBlueprintLibrary::IsReceiveDMXEnabled())
	{
		UDMXProtocolBlueprintLibrary::SetReceiveDMXEnabled(false, bAffectEditor);
	}
	else
	{
		UDMXProtocolBlueprintLibrary::SetReceiveDMXEnabled(true, bAffectEditor);
	}
}

bool FDMXEditorModule::IsReceiveDMXEnabled()
{
	return UDMXProtocolBlueprintLibrary::IsReceiveDMXEnabled();
}

void FDMXEditorModule::RegisterCustomClassLayout(FName ClassName, FOnGetDetailCustomizationInstance DetailLayoutDelegate)
{
	check(ClassName != NAME_None);

	RegisteredClassNames.Add(ClassName);

	static FName PropertyEditor("PropertyEditor");
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditor);
	PropertyModule.RegisterCustomClassLayout(ClassName, DetailLayoutDelegate);

	PropertyModule.NotifyCustomizationModuleChanged();
}

void FDMXEditorModule::UnregisterCustomClassLayouts()
{
	if (UObjectInitialized() && FModuleManager::Get().IsModuleLoaded(TEXT("PropertyEditor")))
	{
		FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");

		for (const FName& ClassName : RegisteredClassNames)
		{
			PropertyModule->UnregisterCustomClassLayout(ClassName);
		}
	}

	RegisteredClassNames.Reset();
}

void FDMXEditorModule::RegisterCustomPropertyTypeLayout(FName PropertyTypeName, FOnGetPropertyTypeCustomizationInstance PropertyTypeLayoutDelegate)
{
	check(PropertyTypeName != NAME_None);

	RegisteredPropertyTypes.Add(PropertyTypeName);

	static FName PropertyEditor("PropertyEditor");
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditor);
	PropertyModule.RegisterCustomPropertyTypeLayout(PropertyTypeName, PropertyTypeLayoutDelegate);

	PropertyModule.NotifyCustomizationModuleChanged();
}

void FDMXEditorModule::UnregisterCustomPropertyTypeLayouts()
{
	if (UObjectInitialized() && FModuleManager::Get().IsModuleLoaded(TEXT("PropertyEditor")))
	{
		FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");

		for (const FName& PropertyTypeName : RegisteredPropertyTypes)
		{
			PropertyModule->UnregisterCustomPropertyTypeLayout(PropertyTypeName);
		}
	}

	RegisteredPropertyTypes.Reset();
}

void  FDMXEditorModule::RegisterCustomSequencerTrackType(const FOnCreateTrackEditor& CreateTrackEditorDelegate)
{
	ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
	RegisteredSequencerTrackHandles.Add(SequencerModule.RegisterTrackEditor(CreateTrackEditorDelegate));
}

void FDMXEditorModule::UnregisterCustomSequencerTrackTypes()
{
	if (UObjectInitialized() && FModuleManager::Get().IsModuleLoaded(TEXT("Sequencer")))
	{
		ISequencerModule* SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>("Sequencer");

		for (const FDelegateHandle& TrackCreateHandle : RegisteredSequencerTrackHandles)
		{
			SequencerModule->UnRegisterTrackEditor(TrackCreateHandle);
		}
	}

	RegisteredSequencerTrackHandles.Reset();
}

FTabSpawnerEntry& FDMXEditorModule::RegisterNomadTabSpawner(const FName TabId, const FOnSpawnTab& OnSpawnTab, const FCanSpawnTab& CanSpawnTab)
{
	RegisteredNomadTabNames.Add(TabId);
	return FGlobalTabmanager::Get()->RegisterNomadTabSpawner(TabId, OnSpawnTab, CanSpawnTab);
}

void FDMXEditorModule::UnregisterNomadTabSpawners()
{
	for (const FName& NomadTabName : RegisteredNomadTabNames)
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(NomadTabName);
	}
}

IMPLEMENT_MODULE(FDMXEditorModule, DMXEditor)

#undef LOCTEXT_NAMESPACE
