// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXEditorStyle.h"

#include "Engine/Font.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"


#define EDITOR_IMAGE_BRUSH_SVG( RelativePath, ... ) FSlateVectorImageBrush(FPaths::EngineContentDir() / "Editor/Slate" / RelativePath + TEXT(".svg"), __VA_ARGS__ )

FDMXEditorStyle::FDMXEditorStyle()
	: FSlateStyleSet("DMXEditorStyle")
{
	static const FVector2D Icon8x8(8.f, 8.f);
	static const FVector2D Icon16x16(16.f, 16.f);
	static const FVector2D Icon64x64(64.f, 64.f);

	static const FVector2D Icon20x20(20.f, 20.f);
	static const FVector2D Icon40x40(40.f, 40.f);
	static const FVector2D Icon50x40(50.f, 40.f);
	static const FVector2D Icon34x29(34.f, 29.f);
	static const FVector2D Icon51x30(51.f, 30.f);
	static const FVector2D Icon51x31(51.f, 31.f);

	const FSlateColor SelectorColor = FAppStyle::GetSlateColor("SelectorColor");

	const TSharedPtr<FSlateStyleSet> Style = MakeShared<FSlateStyleSet>(GetStyleSetName());

	static const TCHAR* DMXEnginePluginName = TEXT("DMXEngine");
	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(DMXEnginePluginName);
	if (ensureMsgf(Plugin.IsValid(), TEXT("Cannot find Plugin 'DMXEngine' hence cannot set the DMXEditorStyle root directory.")))
	{
		SetContentRoot(Plugin->GetBaseDir() / TEXT("Content/Slate"));
	}
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	// Color brushes
	{
		Set("DMXEditor.WhiteBrush", new FSlateColorBrush(FLinearColor(1.f, 1.f, 1.f, 1.f)));
		Set("DMXEditor.BlackBrush", new FSlateColorBrush(FLinearColor(0.f, 0.f, 0.f, 1.f)));
	}

	// Border Brushes
	{
		Set("DMXEditor.RoundedPropertyBorder", new FSlateRoundedBoxBrush(FLinearColor::Black, 4.f));
		Set("DMXEditor.WhiteRoundedPropertyBorder", new FSlateRoundedBoxBrush(FLinearColor::White, 2.f));

		Set("FixturePatcher.FragmentBorder.Normal", new BORDER_BRUSH("FixturePatch_Border_Full_16x", 2.f / 8.f));
		Set("FixturePatcher.FragmentBorder.L", new BORDER_BRUSH("FixturePatch_Border_L_16x", 2.f / 8.f));
		Set("FixturePatcher.FragmentBorder.R", new BORDER_BRUSH("FixturePatch_Border_R_16x", 2.f / 8.f));
		Set("FixturePatcher.FragmentBorder.TB", new BORDER_BRUSH("FixturePatch_Border_M_16x", 2.f / 8.f));
	}

	// Fonts
	{
		static const TCHAR* FontPathRoboto = TEXT("Font'/Engine/EngineFonts/Roboto.Roboto'");
		static const UFont* FontRoboto = Cast<UFont>(StaticLoadObject(UFont::StaticClass(), nullptr, FontPathRoboto));
		check(FontRoboto != nullptr);

		Set("DMXEditor.Font.InputChannelID", FSlateFontInfo(FontRoboto, 8, FName(TEXT("Light"))));
		Set("DMXEditor.Font.InputChannelValue", FSlateFontInfo(FontRoboto, 10, FName(TEXT("Regular"))));

		Set("DMXEditor.Font.InputUniverseHeader", FSlateFontInfo(FontRoboto, 10, FName(TEXT("Bold"))));
		Set("DMXEditor.Font.InputUniverseID", FSlateFontInfo(FontRoboto, 10, FName(TEXT("Regular"))));
		Set("DMXEditor.Font.InputUniverseChannelID", FSlateFontInfo(FontRoboto, 10, FName(TEXT("Regular"))));
		Set("DMXEditor.Font.InputUniverseChannelValue", FSlateFontInfo(FontRoboto, 10, FName(TEXT("Light"))));

		// Normal Text
		const FTextBlockStyle NormalText = FTextBlockStyle()
			.SetFont(DEFAULT_FONT("Regular", 9))
			.SetColorAndOpacity(FSlateColor::UseForeground())
			.SetShadowOffset(FVector2D::ZeroVector)
			.SetShadowColorAndOpacity(FLinearColor::Black)
			.SetHighlightColor(FLinearColor(0.02f, 0.3f, 0.0f))
			.SetHighlightShape(BOX_BRUSH("Common/TextBlockHighlightShape", FMargin(3.f / 8.f)));

		const FTextBlockStyle NormalLogText = FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Mono", 9.f))
			.SetColorAndOpacity(FStyleColors::Foreground)
			.SetSelectedBackgroundColor(FStyleColors::Highlight)
			.SetHighlightColor(FStyleColors::Black);

		Set("ConflictLog.Normal", NormalLogText);

		Set("ConflictLog.Title", FTextBlockStyle(NormalLogText)
			.SetFont(DEFAULT_FONT("Mono", 10.f)));

		Set("ConflictLog.Warning", FTextBlockStyle(NormalLogText)
			.SetColorAndOpacity(FStyleColors::Warning));

		Set("ConflictLog.Error", FTextBlockStyle(NormalLogText)
			.SetColorAndOpacity(FStyleColors::Error));
	}

	// Asset icons
	{
		Set("ClassIcon.DMXLibrary", new IMAGE_BRUSH_SVG("DMXLibrary_16", Icon16x16));
		Set("ClassThumbnail.DMXLibrary", new IMAGE_BRUSH_SVG("DMXLibrary_64", Icon64x64));
	}

	// Icons
	{ 
		Set("Icons.DMXLibrary", new IMAGE_BRUSH_SVG("DMXLibrary_20", Icon20x20));

		Set("Icons.DMXLibrarySettings", new CORE_IMAGE_BRUSH_SVG("Starship/Common/settings", Icon16x16));

		Set("Icons.DMXLibraryToolbar.Import", new CORE_IMAGE_BRUSH_SVG("Starship/Common/import", Icon20x20));
		Set("Icons.DMXLibraryToolbar.Export", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/Export", Icon20x20));

		Set("Icons.FixtureType", new IMAGE_BRUSH_SVG("FixtureType_16", Icon16x16));
		Set("Icons.FixturePatch", new IMAGE_BRUSH_SVG("FixturePatch_16", Icon16x16));

		Set("Icons.SendReceiveDMXEnabled", new IMAGE_BRUSH_SVG("SendReceiveDMXEnabled_20", Icon20x20));
		Set("Icons.ReceiveDMXEnabled", new IMAGE_BRUSH_SVG("ReceiveDMXEnabled_20", Icon20x20));
		Set("Icons.SendDMXEnabled", new IMAGE_BRUSH_SVG("SendDMXEnabled_20", Icon20x20));
		Set("Icons.SendReceiveDMXDisabled", new IMAGE_BRUSH_SVG("SendReceiveDMXDisabled_20", Icon20x20));
	
		Set("Icons.ChannelsMonitor", new IMAGE_BRUSH_SVG("ChannelsMonitor_16", Icon16x16));
		Set("Icons.ActivityMonitor", new IMAGE_BRUSH_SVG("ActivityMonitor_16", Icon16x16));
		Set("Icons.ConflictMonitor", new CORE_IMAGE_BRUSH_SVG("Starship/Common/OutputLog", Icon16x16));
		Set("Icons.OutputConsole", new IMAGE_BRUSH_SVG("OutputConsole_16", Icon16x16));
		Set("Icons.PatchTool", new IMAGE_BRUSH_SVG("PatchTool_16", Icon16x16));
		Set("Icons.ReceiveDMX", new IMAGE_BRUSH_SVG("ToggleReceiveDMX_16", Icon16x16));
		Set("Icons.SendDMX", new IMAGE_BRUSH_SVG("ToggleSendDMX_16", Icon16x16));

		Set("Icons.WarningExclamationMark", new CORE_IMAGE_BRUSH("Old/Tiles/Outer/alertSolid", Icon16x16, FStyleColors::Warning));

		// Play/Stop/Pause related icons
		{
			Set("Icons.Preview", new IMAGE_BRUSH_SVG("Preview", Icon16x16));
			Set("Icons.PlayDMX", new CORE_IMAGE_BRUSH_SVG("Starship/Common/play", Icon20x20));
			Set("Icons.StopDMX", new CORE_IMAGE_BRUSH_SVG("Starship/Common/stop", Icon20x20));

			SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
			Set("Icons.ResumeDMX", new CORE_IMAGE_BRUSH_SVG("Starship/MainToolbar/simulate", Icon20x20));
			Set("Icons.PauseDMX", new CORE_IMAGE_BRUSH_SVG("Starship/MainToolbar/pause", Icon20x20));
			SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));
		}
	}
	
	// Distribution Grid buttons
	{
		Set("DMXEditor.PixelMapping.DistributionGrid.0.0", new IMAGE_BRUSH("icon_PixelDirection_0_0_161x160", Icon34x29));
		Set("DMXEditor.PixelMapping.DistributionGrid.0.1", new IMAGE_BRUSH("icon_PixelDirection_0_1_161x160", Icon34x29));
		Set("DMXEditor.PixelMapping.DistributionGrid.0.2", new IMAGE_BRUSH("icon_PixelDirection_0_2_161x160", Icon34x29));
		Set("DMXEditor.PixelMapping.DistributionGrid.0.3", new IMAGE_BRUSH("icon_PixelDirection_0_3_161x160", Icon34x29));

		Set("DMXEditor.PixelMapping.DistributionGrid.1.0", new IMAGE_BRUSH("icon_PixelDirection_1_0_161x160", Icon34x29));
		Set("DMXEditor.PixelMapping.DistributionGrid.1.1", new IMAGE_BRUSH("icon_PixelDirection_1_1_161x160", Icon34x29));
		Set("DMXEditor.PixelMapping.DistributionGrid.1.2", new IMAGE_BRUSH("icon_PixelDirection_1_2_161x160", Icon34x29));
		Set("DMXEditor.PixelMapping.DistributionGrid.1.3", new IMAGE_BRUSH("icon_PixelDirection_1_3_161x160", Icon34x29));

		Set("DMXEditor.PixelMapping.DistributionGrid.2.0", new IMAGE_BRUSH("icon_PixelDirection_2_0_161x160", Icon34x29));
		Set("DMXEditor.PixelMapping.DistributionGrid.2.1", new IMAGE_BRUSH("icon_PixelDirection_2_1_161x160", Icon34x29));
		Set("DMXEditor.PixelMapping.DistributionGrid.2.2", new IMAGE_BRUSH("icon_PixelDirection_2_2_161x160", Icon34x29));
		Set("DMXEditor.PixelMapping.DistributionGrid.2.3", new IMAGE_BRUSH("icon_PixelDirection_2_3_161x160", Icon34x29));

		Set("DMXEditor.PixelMapping.DistributionGrid.3.0", new IMAGE_BRUSH("icon_PixelDirection_3_0_161x160", Icon34x29));
		Set("DMXEditor.PixelMapping.DistributionGrid.3.1", new IMAGE_BRUSH("icon_PixelDirection_3_1_161x160", Icon34x29));
		Set("DMXEditor.PixelMapping.DistributionGrid.3.2", new IMAGE_BRUSH("icon_PixelDirection_3_2_161x160", Icon34x29));
		Set("DMXEditor.PixelMapping.DistributionGrid.3.3", new IMAGE_BRUSH("icon_PixelDirection_3_3_161x160", Icon34x29));
	}
	
	// Output Console
	{
		Set("DMXEditor.OutputConsole.Fader", new FSlateColorBrush(FLinearColor(.2f, .2f, .2f, .2f)));
		Set("DMXEditor.OutputConsole.Fader_Highlighted", new FSlateColorBrush(FLinearColor(.4f, .4f, .4f, .4f)));

		Set("DMXEditor.OutputConsole.MacroSineWave", new IMAGE_BRUSH("icon_MacroSineWave_51x31", Icon51x31));
		Set("DMXEditor.OutputConsole.MacroMin", new IMAGE_BRUSH("icon_MacroMin_51x30", Icon51x30));
		Set("DMXEditor.OutputConsole.MacroMax", new IMAGE_BRUSH("icon_MacroMax_51x30", Icon51x30));

		static const FLinearColor DefaultFaderBackColor = FLinearColor::FromSRGBColor(FColor::FromHex("191919"));
		static const FLinearColor DefaultFaderFillColor = FLinearColor::FromSRGBColor(FColor::FromHex("00aeef"));
		static const FLinearColor DefeaultFaderForeColor = FLinearColor::FromSRGBColor(FColor::FromHex("ffffff"));

		Set("DMXEditor.OutputConsole.Fader", FSpinBoxStyle(FAppStyle::GetWidgetStyle<FSpinBoxStyle>("SpinBox"))
			.SetBackgroundBrush(CORE_BOX_BRUSH("Common/Spinbox", FMargin(4.f / 16.f), DefaultFaderBackColor))
			.SetHoveredBackgroundBrush(CORE_BOX_BRUSH("Common/Spinbox", FMargin(4.f / 16.f), DefaultFaderBackColor))
			.SetActiveFillBrush(CORE_BOX_BRUSH("Common/Spinbox_Fill", FMargin(4.f / 16.f), DefaultFaderFillColor))
			.SetInactiveFillBrush(CORE_BOX_BRUSH("Common/Spinbox_Fill", FMargin(4.f / 16.f), DefaultFaderFillColor))
			.SetForegroundColor(DefeaultFaderForeColor)
			.SetArrowsImage(FSlateNoResource()));
	}

	// MVR Fixture List / Read Only Fixture Patch List 
	{
		Set("FixturePatchList.Row", FTableRowStyle()
			.SetEvenRowBackgroundBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(.1f, .1f, .1f, .1f)))
			.SetOddRowBackgroundBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(.2f, .2f, .2f, .2f)))
			.SetEvenRowBackgroundHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(.1f, .1f, .1f, .3f)))
			.SetOddRowBackgroundHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(.2f, .2f, .2f, .3f)))
			.SetInactiveBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(.3f, .3f, .3f, .5f)))
			.SetInactiveHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(.3f, .3f, .3f, .7f)))
			.SetActiveBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(.3f, .3f, .3f, .6f)))
			.SetActiveHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(.3f, .3f, .3f, .8f)))
			.SetSelectorFocusedBrush(CORE_BORDER_BRUSH("Common/Selector", FMargin(4.f / 16.f), FLinearColor(1.f, 1.f, 1.f, .7f)))
			.SetDropIndicator_Above(CORE_BOX_BRUSH("Common/DropZoneIndicator_Above", FMargin(10.0f / 16.0f, 10.0f / 16.0f, 0, 0), FLinearColor(1.f, 1.f, 1.f, .7f)))
			.SetDropIndicator_Onto(CORE_BOX_BRUSH("Common/DropZoneIndicator_Onto", FMargin(4.0f / 16.0f), FLinearColor(1.f, 1.f, 1.f, .7f)))
			.SetDropIndicator_Below(CORE_BOX_BRUSH("Common/DropZoneIndicator_Below", FMargin(10.0f / 16.0f, 0, 0, 10.0f / 16.0f), FLinearColor(1.f, 1.f, 1.f, .7f)))
		);

		Set("FixturePatchFragment.ConflictBackground", new IMAGE_BRUSH("FixturePatch_ConflictBack_36x", FVector2D(36.f, 36.f), FStyleColors::Warning, ESlateBrushTileType::Both));
	}

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FDMXEditorStyle::~FDMXEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

const FDMXEditorStyle& FDMXEditorStyle::Get()
{
	static const FDMXEditorStyle Inst;
	return Inst;
}

#undef IMAGE_BRUSH
#undef BOX_BRUSH
#undef CORE_BOX_BRUSH


// DEPRECATED 5.0
TSharedPtr<FSlateStyleSet> FDMXEditorStyle::StyleInstance_DEPRECATED = nullptr;

void FDMXEditorStyle::Initialize()
{
	// DEPRECATED 5.0
	if (!StyleInstance_DEPRECATED.IsValid())
	{
		StyleInstance_DEPRECATED = MakeShared<FDMXEditorStyle>();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance_DEPRECATED);
	}
}

void FDMXEditorStyle::Shutdown()
{
	// DEPRECATED 5.0
	ensureMsgf(StyleInstance_DEPRECATED.IsValid(), TEXT("FDMXEditorStyle::Shutdown called, but StyleInstance wasn't initialized"));
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance_DEPRECATED);
	ensure(StyleInstance_DEPRECATED.IsUnique());
	StyleInstance_DEPRECATED.Reset();
}

void FDMXEditorStyle::ReloadTextures()
{
	// DEPRECATED 5.0
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
	}
}

#undef EDITOR_IMAGE_BRUSH_SVG
