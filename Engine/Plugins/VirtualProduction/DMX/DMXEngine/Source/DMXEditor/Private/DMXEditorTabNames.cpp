// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXEditorTabNames.h"


namespace UE::DMX
{
	const FName FDMXEditorTabNames::ChannelsMonitor(TEXT("ChannelsMonitorTabName"));
	const FName FDMXEditorTabNames::ActivityMonitor(TEXT("ActivityMonitorTabName"));
	const FName FDMXEditorTabNames::ConflictMonitor(TEXT("ConflictMonitorTabName"));
	const FName FDMXEditorTabNames::PatchTool(TEXT("PatchToolTabName"));

	// 'GenericEditor_Properties' was the name of the original controller editor tab. 
	// We keep the old name to show the library editor tab in place of the controller editor in existing projects.
	// This is less confusing for users that upgrade their project from 4.26 to 4.27, otherwise the tab that shows
	// ports wouldn't be visible by default after the update.
	const FName FDMXEditorTabNames::DMXLibraryEditor(TEXT("GenericEditor_Properties"));

	const FName FDMXEditorTabNames::DMXFixtureTypesEditor("GenericEditor_DMXFixtureTypesEditor");
	const FName FDMXEditorTabNames::DMXFixturePatchEditor("GenericEditor_DMXFixturePatchEditor");


}
