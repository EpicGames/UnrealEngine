// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"


namespace UE::DMX
{
	/**  DMX editor tabs identifiers */
	struct FDMXEditorTabNames
	{
		/**	The tab id for the channels monitor tab */
		static const FName ChannelsMonitor;

		/**	The tab id for the activity monitor tab */
		static const FName ActivityMonitor;

		/**	The tab id for the conflict monitor tab */
		static const FName ConflictMonitor;

		/**	The tab id for the patch tool tab */
		static const FName PatchTool;

		/**	The tab id for the dmx library editor tab */
		static const FName DMXLibraryEditor;

		/**	The tab id for the dmx fixture types tab */
		static const FName DMXFixtureTypesEditor;

		/**	The tab id for the dmx fixture patch tab */
		static const FName DMXFixturePatchEditor;

		// Disable default constructor
		FDMXEditorTabNames() = delete;
	};
}
