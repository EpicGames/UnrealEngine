// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXEditorUtils.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Dialogs/Dialogs.h"
#include "DMXEditorLog.h"
#include "DMXRuntimeUtils.h"
#include "DMXSubsystem.h"
#include "Exporters/Exporter.h"
#include "Factories.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Internationalization/Regex.h"
#include "IO/DMXInputPort.h"
#include "IO/DMXOutputPort.h"
#include "IO/DMXPortManager.h"
#include "Misc/StringOutputDevice.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/DMXLibrary.h"
#include "MVR/DMXMVRGeneralSceneDescription.h"
#include "MVR/Types/DMXMVRFixtureNode.h"
#include "PackageTools.h"
#include "ScopedTransaction.h"
#include "UnrealExporter.h"
#include "UObject/Package.h"

#define LOCTEXT_NAMESPACE "FDMXEditorUtils"

/** Text object factory for pasting DMX Entities */
struct FDMXEntityObjectTextFactory : public FCustomizableTextObjectFactory
{
	/** Entities instantiated */
	TArray<UDMXEntity*> NewEntities;

	static bool CanCreate(const FString& InTextBuffer)
	{
		TSharedRef<FDMXEntityObjectTextFactory> Factory = MakeShareable(new FDMXEntityObjectTextFactory());

		// Create new objects if we're allowed to
		return Factory->CanCreateObjectsFromText(InTextBuffer);
	}

	/** Constructs a new object factory from the given text buffer. Returns the factor or nullptr if no factory can be created */
	static TSharedPtr<FDMXEntityObjectTextFactory> Create(const FString& InTextBuffer, UDMXLibrary* InParentLibrary)
	{
		TSharedRef<FDMXEntityObjectTextFactory> Factory = MakeShareable(new FDMXEntityObjectTextFactory());

		// Create new objects if we're allowed to
		if (IsValid(InParentLibrary) && Factory->CanCreateObjectsFromText(InTextBuffer))
		{
			EObjectFlags ObjectFlags = RF_Transactional;

			Factory->ProcessBuffer(InParentLibrary, ObjectFlags, InTextBuffer);

			return Factory;
		}

		return nullptr;
	}

protected:
	/** Constructor; protected to only allow this class to instance itself */
	FDMXEntityObjectTextFactory()
		: FCustomizableTextObjectFactory(GWarn)
	{}

	//~ Begin FCustomizableTextObjectFactory implementation
	virtual bool CanCreateClass(UClass* ObjectClass, bool& bOmitSubObjs) const override
	{
		// Allow DMX Entity types to be created
		return ObjectClass->IsChildOf(UDMXEntity::StaticClass());
	}

	virtual void ProcessConstructedObject(UObject* NewObject) override
	{
		check(NewObject);

		if (UDMXEntity* NewEntity = Cast<UDMXEntity>(NewObject))
		{
			NewEntities.Add(NewEntity);
		}
	}
	//~ End FCustomizableTextObjectFactory implementation
};

bool FDMXEditorUtils::ValidateEntityName(const FString& NewEntityName, const UDMXLibrary* InLibrary, UClass* InEntityClass, FText& OutReason)
{
	if (NewEntityName.Len() >= NAME_SIZE)
	{
		OutReason = LOCTEXT("NameTooLong", "The name is too long");
		return false;
	}

	if (NewEntityName.TrimStartAndEnd().IsEmpty())
	{
		OutReason = LOCTEXT("NameEmpty", "The name can't be blank!");
		return false;
	}

	// Check against existing names for the current entity type
	bool bNameIsUsed = false;
	InLibrary->ForEachEntityOfTypeWithBreak(InEntityClass, [&bNameIsUsed, &NewEntityName](UDMXEntity* Entity)
		{
			if (Entity->GetDisplayName() == NewEntityName)
			{
				bNameIsUsed = true;
				return false; // Break the loop
			}
			return true; // Keep checking Entities' names
		});

	if (bNameIsUsed)
	{
		OutReason = LOCTEXT("ExistingEntityName", "Name already exists");
		return false;
	}
	else
	{
		OutReason = FText::GetEmpty();
		return true;
	}
}

void FDMXEditorUtils::RenameEntity(UDMXLibrary* InLibrary, UDMXEntity* InEntity, const FString& NewName)
{
	if (InEntity == nullptr)
	{
		return;
	}

	if (!NewName.IsEmpty() && !NewName.Equals(InEntity->GetDisplayName()))
	{
		const FScopedTransaction Transaction(LOCTEXT("RenameEntity", "Rename Entity"));
		InEntity->Modify();

		// Update the name
		InEntity->SetName(NewName);
	}
}

bool FDMXEditorUtils::IsEntityUsed(const UDMXLibrary* InLibrary, const UDMXEntity* InEntity)
{
	if (InLibrary != nullptr && InEntity != nullptr)
	{
		if (const UDMXEntityFixtureType* EntityAsFixtureType = Cast<UDMXEntityFixtureType>(InEntity))
		{
			bool bIsUsed = false;
			InLibrary->ForEachEntityOfTypeWithBreak<UDMXEntityFixturePatch>([&](UDMXEntityFixturePatch* Patch)
				{
					if (Patch->GetFixtureType() == InEntity)
					{
						bIsUsed = true;
						return false;
					}
					return true;
				});

			return bIsUsed;
		}
		else
		{
			return false;
		}
	}

	return false;
}

void FDMXEditorUtils::CopyEntities(const TArray<UDMXEntity*>&& EntitiesToCopy)
{
	// Clear the mark state for saving.
	UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));

	const FExportObjectInnerContext Context;
	FStringOutputDevice Archive;

	// Stores duplicates of the Fixture Type Templates because they can't be parsed being children of
	// a DMX Library asset since they're private objects.
	TMap<FName, UDMXEntityFixtureType*> CopiedPatchTemplates;

	// Export the component object(s) to text for copying
	for (UDMXEntity* Entity : EntitiesToCopy)
	{
		// Export the entity object to the given string
		UExporter::ExportToOutputDevice(&Context, Entity, nullptr, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false, GetTransientPackage());
	}

	// Copy text to clipboard
	FString ExportedText = Archive;
	FPlatformApplicationMisc::ClipboardCopy(*ExportedText);
}

bool FDMXEditorUtils::CanPasteEntities(UDMXLibrary* ParentLibrary)
{
	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);

	// Obtain the entity object text factory for the clipboard content and return whether or not we can use it
	return FDMXEntityObjectTextFactory::CanCreate(ClipboardContent);
}

TArray<UDMXEntity*> FDMXEditorUtils::CreateEntitiesFromClipboard(UDMXLibrary* ParentLibrary)
{
	// Get the text from the clipboard
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);

	// Get a new component object factory for the clipboard content
	if (TSharedPtr<FDMXEntityObjectTextFactory> Factory = FDMXEntityObjectTextFactory::Create(TextToImport, ParentLibrary))
	{
		return Factory->NewEntities;
	}

	return TArray<UDMXEntity*>();
}

bool FDMXEditorUtils::AreFixtureTypesIdentical(const UDMXEntityFixtureType* A, const UDMXEntityFixtureType* B)
{
	if (A == B)
	{
		return true;
	}
	if (A == nullptr || B == nullptr)
	{
		return false;
	}
	if (A->GetClass() != B->GetClass())
	{
		return false;
	}

	// Compare each UProperty in the Fixtures
	const UStruct* Struct = UDMXEntityFixtureType::StaticClass();
	TPropertyValueIterator<const FProperty> ItA(Struct, A);
	TPropertyValueIterator<const FProperty> ItB(Struct, B);

	static const FName NAME_ParentLibrary = TEXT("ParentLibrary");
	static const FName NAME_Id = TEXT("Id");

	for (; ItA && ItB; ++ItA, ++ItB)
	{
		const FProperty* PropertyA = ItA->Key;
		const FProperty* PropertyB = ItB->Key;

		if (PropertyA == nullptr || PropertyB == nullptr)
		{
			return false;
		}

		// Properties must be in the exact same order on both Fixtures. Otherwise, it means we have
		// different properties being compared due to differences in array sizes.
		if (!PropertyA->SameType(PropertyB))
		{
			return false;
		}

		// Name and Id don't have to be identical
		if (PropertyA->GetFName() == GET_MEMBER_NAME_CHECKED(UDMXEntity, Name)
			|| PropertyA->GetFName() == NAME_ParentLibrary) // Can't GET_MEMBER_NAME... with private properties
		{
			continue;
		}

		if (PropertyA->GetFName() == NAME_Id)
		{
			// Skip all properties from GUID struct
			for (int32 PropertyCount = 0; PropertyCount < 4; ++PropertyCount)
			{
				++ItA;
				++ItB;
			}
			continue;
		}

		const void* ValueA = ItA->Value;
		const void* ValueB = ItB->Value;

		if (!PropertyA->Identical(ValueA, ValueB))
		{
			return false;
		}
	}

	// If one of the Property Iterators is still valid, one of the Fixtures had
	// less properties due to an array size difference, which means the Fixtures are different.
	if (ItA || ItB)
	{
		return false;
	}

	return true;
}

FText FDMXEditorUtils::GetEntityTypeNameText(TSubclassOf<UDMXEntity> EntityClass, bool bPlural /*= false*/)
{
	if (EntityClass->IsChildOf(UDMXEntityFixtureType::StaticClass()))
	{
		return FText::Format(
			LOCTEXT("EntityTypeName_FixtureType", "Fixture {0}|plural(one=Type, other=Types)"),
			bPlural ? 2 : 1
		);
	}
	else if (EntityClass->IsChildOf(UDMXEntityFixturePatch::StaticClass()))
	{
		return FText::Format(
			LOCTEXT("EntityTypeName_FixturePatch", "Fixture {0}|plural(one=Patch, other=Patches)"),
			bPlural ? 2 : 1
		);
	}
	else
	{
		return FText::Format(
			LOCTEXT("EntityTypeName_NotImplemented", "{0}|plural(one=Entity, other=Entities)"),
			bPlural ? 2 : 1
		);
	}
}

void FDMXEditorUtils::UpdatePatchColors(UDMXLibrary* Library)
{
	check(Library);

	TArray<UDMXEntityFixturePatch*> Patches = Library->GetEntitiesTypeCast<UDMXEntityFixturePatch>();
	for (UDMXEntityFixturePatch* Patch : Patches)
	{
		if (Patch->EditorColor == FLinearColor(1.0f, 0.0f, 1.0f))
		{
			FLinearColor NewColor;

			UDMXEntityFixturePatch** ColoredPatchOfSameType = Patches.FindByPredicate([&](const UDMXEntityFixturePatch* Other) {
				return Other != Patch &&
					Other->GetFixtureType() == Patch->GetFixtureType() &&
					Other->EditorColor != FLinearColor::White;
				});

			if (ColoredPatchOfSameType)
			{
				NewColor = (*ColoredPatchOfSameType)->EditorColor;
			}
			else
			{
				NewColor = FLinearColor::MakeRandomColor();

				// Avoid dominant red values for a bit more of a professional feel
				if (NewColor.R > 0.6f)
				{
					NewColor.R = FMath::Abs(NewColor.R - 1.0f);
				}
			}

			FProperty* ColorProperty = FindFProperty<FProperty>(UDMXEntityFixturePatch::StaticClass(), GET_MEMBER_NAME_CHECKED(UDMXEntityFixturePatch, EditorColor));

			Patch->Modify();
			Patch->PreEditChange(ColorProperty);
			Patch->EditorColor = NewColor;
			Patch->PostEditChange();
		}
	}
}

void FDMXEditorUtils::GetAllAssetsOfClass(UClass* Class, TArray<UObject*>& OutObjects)
{
	check(Class);

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(FName("AssetRegistry")).Get();
	TArray<FAssetData> OutAssets;
	AssetRegistry.GetAssetsByClass(Class->GetClassPathName(), OutAssets, true);

	for (const FAssetData& Asset : OutAssets)
	{
		OutObjects.Add(Asset.GetAsset());
	}
}

bool FDMXEditorUtils::DoesLibraryHaveUniverseConflicts(UDMXLibrary* Library, FText& OutInputPortConflictMessage, FText& OutOutputPortConflictMessage)
{
	check(Library);

	OutInputPortConflictMessage = FText::GetEmpty();
	OutOutputPortConflictMessage = FText::GetEmpty();

	TArray<UObject*> LoadedLibraries;
	GetAllAssetsOfClass(UDMXLibrary::StaticClass(), LoadedLibraries);

	for (UObject* OtherLibrary : LoadedLibraries)
	{
		if (OtherLibrary == Library)
		{
			continue;
		}

		UDMXLibrary* OtherDMXLibrary = CastChecked<UDMXLibrary>(OtherLibrary);
		
		// Find conflicting input ports
		for (const FDMXInputPortSharedRef& InputPort : Library->GetInputPorts())
		{
			for (const FDMXInputPortSharedRef& OtherInputPort : OtherDMXLibrary->GetInputPorts())
			{
				if (InputPort->GetProtocol() == OtherInputPort->GetProtocol())
				{
					if (InputPort->GetLocalUniverseStart() <= OtherInputPort->GetLocalUniverseEnd() &&
						OtherInputPort->GetLocalUniverseStart() <= InputPort->GetLocalUniverseEnd())
					{
						continue;
					}

					if (OutInputPortConflictMessage.IsEmpty())
					{
						OutInputPortConflictMessage = LOCTEXT("LibraryInputPortUniverseConflictMessageStart", "Libraries use the same Input Port: ");
					}
					
					OutInputPortConflictMessage = FText::Format(LOCTEXT("LibraryInputPortUniverseConflictMessage", "{0} {1}"), OutInputPortConflictMessage, FText::FromString(OtherDMXLibrary->GetName()));
				}
			}
		}

		// Find conflicting output ports
		for (const FDMXOutputPortSharedRef& OutputPort : Library->GetOutputPorts())
		{
			for (const FDMXOutputPortSharedRef& OtherOutputPort : OtherDMXLibrary->GetOutputPorts())
			{
				if (OutputPort->GetProtocol() == OtherOutputPort->GetProtocol())
				{
					if (OutputPort->GetLocalUniverseStart() <= OtherOutputPort->GetLocalUniverseEnd() &&
						OtherOutputPort->GetLocalUniverseStart() <= OutputPort->GetLocalUniverseEnd())
					{
						continue;
					}

					if (OutOutputPortConflictMessage.IsEmpty())
					{
						OutOutputPortConflictMessage = LOCTEXT("LibraryOutputPortUniverseConflictMessageStart", "Libraries that use the same Output Port: ");
					}

					OutInputPortConflictMessage = FText::Format(LOCTEXT("LibraryOutputPortUniverseConflictMessage", "{0} {1}"), OutOutputPortConflictMessage, FText::FromString(OtherDMXLibrary->GetName()));
				}
			}
		}
	}

	bool bNoConflictsFound = OutOutputPortConflictMessage.IsEmpty() && OutInputPortConflictMessage.IsEmpty();

	return bNoConflictsFound;
}

void FDMXEditorUtils::ClearAllDMXPortBuffers()
{
	// DEPRECATED 5.5
	FDMXPortManager::Get().ClearBuffers();
}

void FDMXEditorUtils::ClearFixturePatchCachedData()
{	
	// DEPRECATED 5.5
	UDMXSubsystem* Subsystem = UDMXSubsystem::GetDMXSubsystem_Callable();
	if (Subsystem && Subsystem->IsValidLowLevel())
	{
		TArray<TSoftObjectPtr<UDMXLibrary>> DMXLibraries = Subsystem->GetDMXLibraries();
		for (const TSoftObjectPtr<UDMXLibrary>& Library : DMXLibraries)
		{
			if (Library.IsValid())
			{
				Library.Get()->ForEachEntityOfType<UDMXEntityFixturePatch>([](UDMXEntityFixturePatch* Patch) {
					Patch->RebuildCache();
				});
			}
		}
	}
}

UPackage* FDMXEditorUtils::GetOrCreatePackage(TWeakObjectPtr<UObject> Parent, const FString& DesiredName)
{
	UPackage* Package = nullptr;
	FString NewPackageName;

	if (Parent.IsValid() && Parent->IsA(UPackage::StaticClass()))
	{
		Package = StaticCast<UPackage*>(Parent.Get());
	}

	if (!Package)
	{
		if (Parent.IsValid() && Parent->GetOutermost())
		{
			NewPackageName = FPackageName::GetLongPackagePath(Parent->GetOutermost()->GetName()) + "/" + DesiredName;
		}
		else
		{
			return nullptr;
		}

		NewPackageName = UPackageTools::SanitizePackageName(NewPackageName);
		Package = CreatePackage(*NewPackageName);
		Package->FullyLoad();
	}

	return Package;
}

TArray<FString> FDMXEditorUtils::ParseAttributeNames(const FString& InputString)
{
	// Try to match addresses formating, e.g. '1.', '1:' etc.
	static const TCHAR* AttributeNameParamDelimiters[] =
	{
		TEXT("."),
		TEXT(","),
		TEXT(":"),
		TEXT(";")
	};

	TArray<FString> AttributeNameStrings;
	constexpr bool bParseEmpty = false;
	InputString.ParseIntoArray(AttributeNameStrings, AttributeNameParamDelimiters, 4, bParseEmpty);

	TArray<FString> Result;
	for (const FString& String : AttributeNameStrings)
	{
		Result.Add(String.TrimStartAndEnd());
	}

	return Result;
}

TArray<int32> FDMXEditorUtils::ParseUniverses(const FString& InputString)
{
	TArray<int32> Result;

	// Try to match addresses formating, e.g. '1.', '1:' etc.
	static const TCHAR* UniverseAddressParamDelimiters[] =
	{
		TEXT("."),
		TEXT(","),
		TEXT(":"),
		TEXT(";")
	};
	if (InputString.EndsWith(UniverseAddressParamDelimiters[0]) ||
		InputString.EndsWith(UniverseAddressParamDelimiters[1]) ||
		InputString.EndsWith(UniverseAddressParamDelimiters[2]) ||
		InputString.EndsWith(UniverseAddressParamDelimiters[3]))
	{
		TArray<FString> UniverseAddressStringArray;

		constexpr bool bCullEmpty = true;
		InputString.ParseIntoArray(UniverseAddressStringArray, UniverseAddressParamDelimiters, 4, bCullEmpty);
		if (UniverseAddressStringArray.Num() == 1)
		{
			int32 Universe;
			if (LexTryParseString(Universe, *UniverseAddressStringArray[0]))
			{
				Result.Add(Universe);
				return Result;
			}
		}
	}

	// Try to match strings starting with Uni, e.g. 'Uni 1', 'uni1', 'Uni 1, 2', 'universe 1', 'Universe 1, 2 - 3, 4'
	const FRegexPattern UniversesPattern(TEXT("^(?:universe|uni)\\s*(.*)"), ERegexPatternFlags::CaseInsensitive);
	FRegexMatcher Regex(UniversesPattern, *InputString);
	if (Regex.FindNext())
	{
		FString UniversesString = Regex.GetCaptureGroup(1);
		UniversesString.RemoveSpacesInline();

		static const TCHAR* UniversesDelimiter[] =
		{
			TEXT(",")
		};
		TArray<FString> UniveresStringArray;

		constexpr bool bCullEmpty = true;
		UniversesString.ParseIntoArray(UniveresStringArray, UniversesDelimiter, 1, bCullEmpty);
		for (const FString& UniversesSubstring : UniveresStringArray)
		{
			static const TCHAR* UniverseRangeDelimiter[] =
			{
				TEXT("-")
			};

			TArray<FString> UniverseRangeStringArray;
			UniversesSubstring.ParseIntoArray(UniverseRangeStringArray, UniverseRangeDelimiter, 1, bCullEmpty);

			int32 UniverseStart;
			int32 UniverseEnd;
			int32 Universe;
			if (UniverseRangeStringArray.Num() == 2 &&
				LexTryParseString(UniverseStart, *UniverseRangeStringArray[0]) &&
				LexTryParseString(UniverseEnd, *UniverseRangeStringArray[1]) &&
				UniverseStart < UniverseEnd)
			{
				for (Universe = UniverseStart; Universe <= UniverseEnd; Universe++)
				{
					Result.Add(Universe);
				}
			}
			else if (LexTryParseString(Universe, *UniversesSubstring))
			{
				Result.Add(Universe);
			}
		}
	}

	return Result;
}

bool FDMXEditorUtils::ParseAddress(const FString& InputString, int32& OutAddress)
{
	// Try to match addresses formating, e.g. '1.1', '1:1' etc.
	static const TCHAR* ParamDelimiters[] =
	{
		TEXT("."),
		TEXT(":"),
		TEXT(";")
	};

	TArray<FString> ValueStringArray;
	constexpr bool bParseEmpty = false;
	InputString.ParseIntoArray(ValueStringArray, ParamDelimiters, 3, bParseEmpty);

	if (ValueStringArray.Num() == 2)
	{
		if (LexTryParseString<int32>(OutAddress, *ValueStringArray[1]))
		{
			return true;
		}
	}

	// Try to match strings starting with Uni Ad, e.g. 'Uni 1 Ad 1', 'Universe 1 Address 1', 'Universe1Address1'
	if (InputString.StartsWith(TEXT("Uni")) &&
		InputString.Contains(TEXT("Ad")))
	{
		const FRegexPattern SequenceOfDigitsPattern(TEXT("^[^\\d]*(\\d+)[^\\d]*(\\d+)"));
		FRegexMatcher Regex(SequenceOfDigitsPattern, *InputString);
		if (Regex.FindNext())
		{
			const FString AddressString = Regex.GetCaptureGroup(2);
			if (LexTryParseString<int32>(OutAddress, *AddressString))
			{
				return true;
			}
		}
	}

	OutAddress = -1;
	return false;
}

bool FDMXEditorUtils::ParseFixtureID(const FString& InputString, int32& OutFixtureID)
{
	if (LexTryParseString<int32>(OutFixtureID, *InputString))
	{
		return true;
	}

	OutFixtureID = -1;
	return false;
}

TArray<int32> FDMXEditorUtils::ParseFixtureIDs(const FString& FixtureIDsString)
{
	static const TCHAR* FixtureIDsDelimiter[] =
	{
		TEXT(",")
	};
	TArray<FString> FixtureIDsStringArray;

	TArray<int32> Result;

	constexpr bool bCullEmpty = true;
	FixtureIDsString.ParseIntoArray(FixtureIDsStringArray, FixtureIDsDelimiter, 1, bCullEmpty);
	for (const FString& FixtureIDsSubstring : FixtureIDsStringArray)
	{
		static const TCHAR* FixtureIDRangeDelimiter[] =
		{
			TEXT("-")
		};

		TArray<FString> FixtureIDRangeStringArray;
		FixtureIDsSubstring.ParseIntoArray(FixtureIDRangeStringArray, FixtureIDRangeDelimiter, 1, bCullEmpty);

		int32 FixtureIDStart;
		int32 FixtureIDEnd;
		int32 FixtureID;
		if (FixtureIDRangeStringArray.Num() == 2 &&
			LexTryParseString(FixtureIDStart, *FixtureIDRangeStringArray[0]) &&
			LexTryParseString(FixtureIDEnd, *FixtureIDRangeStringArray[1]) &&
			FixtureIDStart < FixtureIDEnd)
		{
			for (FixtureID = FixtureIDStart; FixtureID <= FixtureIDEnd; FixtureID++)
			{
				Result.Add(FixtureID);
			}
		}
		else if (LexTryParseString(FixtureID, *FixtureIDsSubstring))
		{
			Result.Add(FixtureID);
		}
	}

	return Result;
}

#undef DMX_INVALID_NAME_CHARACTERS

#undef LOCTEXT_NAMESPACE
