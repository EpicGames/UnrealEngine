// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXFixturePatchSharedData.h"

#include "DMXEditor.h"
#include "DMXFixturePatchSharedDataSelection.h"
#include "Library/DMXEntityFixturePatch.h"


#define LOCTEXT_NAMESPACE "DMXFixturePatchSharedData"

FDMXFixturePatchSharedData::FDMXFixturePatchSharedData(TWeakPtr<FDMXEditor> InDMXEditorPtr)
	: DMXEditorPtr(InDMXEditorPtr)
{
	Selection = NewObject<UDMXFixturePatchSharedDataSelection>(GetTransientPackage(), NAME_None, RF_Transactional);
}

void FDMXFixturePatchSharedData::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Selection);
}

void FDMXFixturePatchSharedData::PostUndo(bool bSuccess)
{
	check(Selection);
	Selection->SelectedFixturePatches.RemoveAll([](TWeakObjectPtr<UDMXEntityFixturePatch> FixturePatch)
		{
			return !FixturePatch.IsValid();
		});
	OnFixturePatchSelectionChanged.Broadcast();
}

void FDMXFixturePatchSharedData::PostRedo(bool bSuccess)
{
	check(Selection);
	Selection->SelectedFixturePatches.RemoveAll([](TWeakObjectPtr<UDMXEntityFixturePatch> FixturePatch)
		{
			return !FixturePatch.IsValid();
		});
	OnFixturePatchSelectionChanged.Broadcast();
}

void FDMXFixturePatchSharedData::SelectUniverse(int32 UniverseID)
{
	check(Selection);
	check(UniverseID >= 0);

	if (UniverseID == Selection->SelectedUniverse)
	{
		return;
	}

	Selection->SelectedUniverse = UniverseID;
	OnUniverseSelectionChanged.Broadcast();
}

int32 FDMXFixturePatchSharedData::GetSelectedUniverse()
{
	check(Selection);
	return Selection->SelectedUniverse;
}

void FDMXFixturePatchSharedData::SelectFixturePatch(TWeakObjectPtr<UDMXEntityFixturePatch> Patch)
{
	check(Selection);
	if (Selection->SelectedFixturePatches.Num() == 1 &&
		Selection->SelectedFixturePatches[0] == Patch)
	{
		return;
	}

	Selection->Modify();
	Selection->SelectedFixturePatches.Reset();
	Selection->SelectedFixturePatches.Add(Patch);
	OnFixturePatchSelectionChanged.Broadcast();
}

void FDMXFixturePatchSharedData::AddFixturePatchToSelection(TWeakObjectPtr<UDMXEntityFixturePatch> Patch)
{
	check(Selection);
	if (!Selection->SelectedFixturePatches.Contains(Patch))
	{
		Selection->Modify();
		Selection->SelectedFixturePatches.Add(Patch);
		OnFixturePatchSelectionChanged.Broadcast();
	}
}

void FDMXFixturePatchSharedData::SelectFixturePatches(const TArray<TWeakObjectPtr<UDMXEntityFixturePatch>>& Patches)
{
	check(Selection);
	if (Selection->SelectedFixturePatches == Patches)
	{
		return;
	}

	// Make a copy without duplicates
	TArray<TWeakObjectPtr<UDMXEntityFixturePatch>> UniquePatchesOnlyArray;
	UniquePatchesOnlyArray.Reserve(Patches.Num());
	for (TWeakObjectPtr<UDMXEntityFixturePatch> FixturePatch : Patches)
	{
		UniquePatchesOnlyArray.AddUnique(FixturePatch);
	}

	Selection->Modify();
	Selection->SelectedFixturePatches.Reset();
	Selection->SelectedFixturePatches = UniquePatchesOnlyArray;
	OnFixturePatchSelectionChanged.Broadcast();
}

const TArray<TWeakObjectPtr<UDMXEntityFixturePatch>>& FDMXFixturePatchSharedData::GetSelectedFixturePatches() const
{
	check(Selection);
	return Selection->SelectedFixturePatches;
}

#undef LOCTEXT_NAMESPACE
