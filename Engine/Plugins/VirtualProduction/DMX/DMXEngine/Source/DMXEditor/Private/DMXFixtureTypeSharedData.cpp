// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXFixtureTypeSharedData.h"

#include "DMXEditorUtils.h"
#include "DMXFixtureTypeSharedDataSelection.h"
#include "DMXRuntimeUtils.h"
#include "Library/DMXEntityFixtureType.h"

#include "ScopedTransaction.h"
#include "UObject/Package.h"


#define LOCTEXT_NAMESPACE "DMXFixtureTypeSharedData"

namespace
{
	namespace FDMXFixtureTypeSharedDataDetails
	{
		// DEPRECATED 5.0, only here in to keep support for deprecated members of DMXFixtureTypeSharedData
		struct FDMXScopedModeEditChangeChainProperty_DEPRECATED
		{
			FDMXScopedModeEditChangeChainProperty_DEPRECATED(UDMXEntityFixtureType* InFixtureType)
			{
				check(InFixtureType);

				FixtureType = InFixtureType;

				FProperty* ModesProperty = UDMXEntityFixtureType::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_STRING_CHECKED(UDMXEntityFixtureType, Modes));
				check(ModesProperty);

				FEditPropertyChain PropertyChain;
				PropertyChain.AddHead(ModesProperty);
				PropertyChain.SetActivePropertyNode(ModesProperty);

				FixtureType->PreEditChange(PropertyChain);
				FixtureType->Modify();
			}

			~FDMXScopedModeEditChangeChainProperty_DEPRECATED()
			{
				if (FixtureType.IsValid())
				{
					FProperty* ModesProperty = UDMXEntityFixtureType::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_STRING_CHECKED(UDMXEntityFixtureType, Modes));
					check(ModesProperty);

					FEditPropertyChain PropertyChain;
					PropertyChain.AddHead(ModesProperty);
					PropertyChain.SetActivePropertyNode(ModesProperty);

					TMap<FString, int32> IndexMap;
					TArray<UObject*> ObjectsBeingEdited = { FixtureType.Get() };

					FPropertyChangedEvent PropertyChangedEvent(ModesProperty->GetOwnerProperty(), EPropertyChangeType::ArrayAdd, MakeArrayView(ObjectsBeingEdited));
					FPropertyChangedChainEvent PropertyChangedChainEvent(PropertyChain, PropertyChangedEvent);

					IndexMap.Add(FString(ModesProperty->GetName()), FixtureType->Modes.Num() - 1);

					TArray<TMap<FString, int32>> IndexPerObject;
					IndexPerObject.Add(IndexMap);

					PropertyChangedChainEvent.SetArrayIndexPerObject(IndexPerObject);
					PropertyChangedChainEvent.ObjectIteratorIndex = 0;

					FixtureType->PostEditChangeChainProperty(PropertyChangedChainEvent);
				}
			}

		private:
			TWeakObjectPtr<UDMXEntityFixtureType> FixtureType;
		};
	}
}

FDMXFixtureTypeSharedData::FDMXFixtureTypeSharedData(TWeakPtr<FDMXEditor> InDMXEditorPtr)
{
	Selection = NewObject<UDMXFixtureTypeSharedDataSelection>(GetTransientPackage(), NAME_None, RF_Transactional);
}

void FDMXFixtureTypeSharedData::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Selection);
}

void FDMXFixtureTypeSharedData::PostUndo(bool bSuccess)
{
	check(Selection);
	Selection->SelectedFixtureTypes.RemoveAll([](TWeakObjectPtr<UDMXEntityFixtureType> FixtureType)
		{
			return !FixtureType.IsValid();
		});
	OnFixtureTypesSelected.Broadcast();
}

void FDMXFixtureTypeSharedData::PostRedo(bool bSuccess)
{
	check(Selection);
	Selection->SelectedFixtureTypes.RemoveAll([](TWeakObjectPtr<UDMXEntityFixtureType> FixtureType)
		{
			return !FixtureType.IsValid();
		});
	OnFixtureTypesSelected.Broadcast();
}

void FDMXFixtureTypeSharedData::SelectFixtureTypes(const TArray<TWeakObjectPtr<UDMXEntityFixtureType>>& InFixtureTypes)
{
	check(Selection);
	if (Selection->SelectedFixtureTypes != InFixtureTypes)
	{
		// Make a copy without duplicates
		TArray<TWeakObjectPtr<UDMXEntityFixtureType>> UniqueFixtureTypesOnlyArray;
		UniqueFixtureTypesOnlyArray.Reserve(InFixtureTypes.Num());
		for (TWeakObjectPtr<UDMXEntityFixtureType> FixtureType : InFixtureTypes)
		{
			UniqueFixtureTypesOnlyArray.AddUnique(FixtureType);
		}

		Selection->Modify();
		Selection->SelectedFixtureTypes = UniqueFixtureTypesOnlyArray;

		// Selected Modes and Functions turn invalid
		Selection->SelectedModeIndices.Reset();
		Selection->SelectedFunctionIndices.Reset();
		Selection->bFixtureMatrixSelected = false;

		OnFixtureTypesSelected.Broadcast();
	}
}

void FDMXFixtureTypeSharedData::SelectModes(const TArray<int32>& InModeIndices)
{
	check(Selection);
	if (Selection->SelectedModeIndices != InModeIndices)
	{
		Selection->Modify();
		Selection->SelectedModeIndices = InModeIndices;

		// Selected Functions turn invalid
		Selection->SelectedFunctionIndices.Reset();
		Selection->bFixtureMatrixSelected = false;

		OnModesSelected.Broadcast();
	}
}

void FDMXFixtureTypeSharedData::SetFunctionAndMatrixSelection(const TArray<int32>& InFunctionIndices, bool bMatrixSelected)
{
	check(Selection);
	if (Selection->SelectedFunctionIndices != InFunctionIndices ||
		Selection->bFixtureMatrixSelected != bMatrixSelected)
	{
		Selection->Modify();

		Selection->SelectedFunctionIndices = InFunctionIndices;
		Selection->bFixtureMatrixSelected = bMatrixSelected;

		OnFunctionsSelected.Broadcast();
		OnMatrixSelectionChanged.Broadcast();
	}
}

bool FDMXFixtureTypeSharedData::CanCopyModesToClipboard() const
{
	check(Selection);
	if (Selection->SelectedModeIndices.Num() == 0)
	{
		return false;
	}
	else if (Selection->SelectedFixtureTypes.Num() == 1)
	{
		if (UDMXEntityFixtureType* FixtureType = Selection->SelectedFixtureTypes[0].Get())
		{
			for (int32 ModeIndex : Selection->SelectedModeIndices)
			{
				if (!FixtureType->Modes.IsValidIndex(Selection->SelectedModeIndices[0]))
				{
					return false;
				}
			}
		}
	}

	return true;
}

void FDMXFixtureTypeSharedData::CopyModesToClipboard()
{
	check(Selection);
	if (ensureMsgf(CanCopyModesToClipboard(), TEXT("Cannot copy Modes to clipboard. Please test first with FDMXFixtureTypeSharedData::CanCopyModesToClipboard.")))
	{
		ModesClipboard.Reset();

		for (TWeakObjectPtr<UDMXEntityFixtureType> WeakFixtureType : Selection->SelectedFixtureTypes)
		{
			if (UDMXEntityFixtureType* FixtureType = WeakFixtureType.Get())
			{
				for (const int32 ModeIndex : Selection->SelectedModeIndices)
				{
					const FDMXFixtureMode& Mode = FixtureType->Modes[ModeIndex];
					if (FixtureType->Modes.IsValidIndex(ModeIndex))
					{
						const TOptional<FString> CopyStr = FDMXRuntimeUtils::SerializeStructToString<FDMXFixtureMode>(Mode);
						if (CopyStr.IsSet())
						{
							ModesClipboard.Add(CopyStr.GetValue());
						}
					}
				}
			}
		}
	}
}

bool FDMXFixtureTypeSharedData::CanPasteModesFromClipboard() const
{
	check(Selection);
	if (Selection->SelectedFixtureTypes.Num() == 1 && ModesClipboard.Num() > 0)
	{
		if (UDMXEntityFixtureType* FixtureType = Selection->SelectedFixtureTypes[0].Get())
		{
			return true;
		}
	}

	return true;
}

void FDMXFixtureTypeSharedData::PasteModesFromClipboard(TArray<int32>& OutNewlyAddedModeIndices)
{
	check(Selection);
	if (ensureMsgf(CanPasteModesFromClipboard(), TEXT("Cannot paste Modes from clipboard. Please test first with FDMXFixtureTypeSharedData::CanPasteModesFromClipboard.")))
	{
		if (Selection->SelectedFixtureTypes.Num() == 1)
		{
			if (UDMXEntityFixtureType* FixtureType = Selection->SelectedFixtureTypes[0].Get())
			{
				const FText TransactionText = FText::Format(LOCTEXT("PasteModesTransaction", "Paste Fixture {0}|plural(one=Mode, other=Modes)"), GetSelectedModeIndices().Num());
				const FScopedTransaction PasteModesTransaction(TransactionText);

				FixtureType->PreEditChange(UDMXEntityFixtureType::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UDMXEntityFixtureType, Modes)));

				int32 NumPastedElements = 0;
				for (int32 ClipboardIndex = 0; ClipboardIndex < ModesClipboard.Num(); ClipboardIndex++)
				{
					const FString& ClipboardString = ModesClipboard[ClipboardIndex];

					const int32 PasteOntoIndex = [FixtureType, &ClipboardIndex, this]()
					{
						if (Selection->SelectedModeIndices.IsValidIndex(ClipboardIndex))
						{
							return Selection->SelectedModeIndices[ClipboardIndex];
						}
						else if (Selection->SelectedModeIndices.Num() > 0)
						{
							return Selection->SelectedModeIndices.Last();
						}
						else
						{
							return FixtureType->Modes.Num() - 1;
						}
					}();

					TSharedPtr<FJsonObject> RootJsonObject;
					TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(ClipboardString);
					FJsonSerializer::Deserialize(Reader, RootJsonObject);

					if (RootJsonObject.IsValid())
					{
						FDMXFixtureMode ModeFromClipboard;
						if (FJsonObjectConverter::JsonObjectToUStruct(RootJsonObject.ToSharedRef(), FDMXFixtureMode::StaticStruct(), &ModeFromClipboard, 0, 0))
						{
							int32 IndexOfNewlyAddedMode = PasteOntoIndex + 1 + NumPastedElements;
							if (FixtureType->Modes.IsValidIndex(IndexOfNewlyAddedMode))
							{
								IndexOfNewlyAddedMode = FixtureType->Modes.Insert(ModeFromClipboard, IndexOfNewlyAddedMode);
							}
							else
							{
								IndexOfNewlyAddedMode = FixtureType->Modes.Add(ModeFromClipboard);
							}
							OutNewlyAddedModeIndices.Add(IndexOfNewlyAddedMode);

							// Make a uinque mode name by setting it
							FString UnusedString;
							FixtureType->SetModeName(IndexOfNewlyAddedMode, ModeFromClipboard.ModeName, UnusedString);

							NumPastedElements++;
						}
					}
				}

				FixtureType->PostEditChange();
			}
		}
	}
}

bool FDMXFixtureTypeSharedData::CanCopyFunctionsToClipboard() const
{
	check(Selection);
	if (Selection->SelectedFunctionIndices.Num() == 0 || Selection->bFixtureMatrixSelected)
	{
		return false;
	}
	else if (Selection->SelectedFixtureTypes.Num() == 1 && Selection->SelectedModeIndices.Num() == 1)
	{
		if (UDMXEntityFixtureType* FixtureType = Selection->SelectedFixtureTypes[0].Get())
		{
			if(FixtureType->Modes.IsValidIndex(Selection->SelectedModeIndices[0]))
			{
				const FDMXFixtureMode& Mode = FixtureType->Modes[Selection->SelectedModeIndices[0]];

				for (const int32 FunctionIndex : Selection->SelectedFunctionIndices)
				{
					if (!Mode.Functions.IsValidIndex(FunctionIndex))
					{
						return false;
					}
				}
			}
		}
	}

	return true;
}

void FDMXFixtureTypeSharedData::CopyFunctionsToClipboard()
{
	check(Selection);
	if (ensureMsgf(CanCopyFunctionsToClipboard(), TEXT("Cannot copy Functions to clipboard. Please test first with FDMXFixtureTypeSharedData::CopyFunctionsToClipboard.")))
	{
		FunctionsClipboard.Reset();

		if (Selection->SelectedFixtureTypes.Num() == 1 && Selection->SelectedModeIndices.Num() == 1)
		{
			if (UDMXEntityFixtureType* FixtureType = Selection->SelectedFixtureTypes[0].Get())
			{
				if (FixtureType->Modes.IsValidIndex(Selection->SelectedModeIndices[0]))
				{
					const FDMXFixtureMode& Mode = FixtureType->Modes[Selection->SelectedModeIndices[0]];

					// Sort selected function indices by starting channel, so elements are copied in visual right order, not in order of how things were selected
					Selection->SelectedFunctionIndices.Sort([Mode](const int32 FunctionIndexA, const int32 FunctionIndexB)
						{
							if (Mode.Functions.IsValidIndex(FunctionIndexA) && Mode.Functions.IsValidIndex(FunctionIndexB))
							{
								return Mode.Functions[FunctionIndexA].Channel <= Mode.Functions[FunctionIndexB].Channel;
							}
							return false;
						});

					for (const int32 FunctionIndex : Selection->SelectedFunctionIndices)
					{
						if (Mode.Functions.IsValidIndex(FunctionIndex))
						{
							const TOptional<FString> CopyStr = FDMXRuntimeUtils::SerializeStructToString<FDMXFixtureFunction>(Mode.Functions[FunctionIndex]);
							if (CopyStr.IsSet())
							{
								FunctionsClipboard.Add(CopyStr.GetValue());
							}
						}
					}
				}
			}
		}
	}
}

bool FDMXFixtureTypeSharedData::CanPasteFunctionsFromClipboard() const
{
	check(Selection);
	if (Selection->SelectedFixtureTypes.Num() == 1 && Selection->SelectedModeIndices.Num() == 1 && ModesClipboard.Num() > 0)
	{
		if (UDMXEntityFixtureType* FixtureType = Selection->SelectedFixtureTypes[0].Get())
		{
			if (FixtureType->Modes.IsValidIndex(Selection->SelectedModeIndices[0]))
			{
				return true;
			}
		}
	}

	return true;
}

void FDMXFixtureTypeSharedData::PasteFunctionsFromClipboard(TArray<int32>& OutNewlyAddedFunctionIndices)
{
	check(Selection);
	if (ensureMsgf(CanPasteFunctionsFromClipboard(), TEXT("Cannot paste Functions from clipboard. Please test first with FDMXFixtureTypeSharedData::CanPasteFunctionsFromClipboard.")))
	{
		if (Selection->SelectedFixtureTypes.Num() == 1 && Selection->SelectedModeIndices.Num() == 1)
		{
			if (UDMXEntityFixtureType* FixtureType = Selection->SelectedFixtureTypes[0].Get())
			{
				const int32 SelectedModeIndex = Selection->SelectedModeIndices[0];
				if (FixtureType->Modes.IsValidIndex(SelectedModeIndex))
				{
					FDMXFixtureMode& Mode = FixtureType->Modes[SelectedModeIndex];

					// Sort selected function indices by starting channel, so elements are pasted in visual right order, not in order of how things were selected
					Selection->SelectedFunctionIndices.Sort([Mode](const int32 FunctionIndexA, const int32 FunctionIndexB)
						{
							if (Mode.Functions.IsValidIndex(FunctionIndexA) && Mode.Functions.IsValidIndex(FunctionIndexB))
							{
								return Mode.Functions[FunctionIndexA].Channel <= Mode.Functions[FunctionIndexB].Channel;
							}
							return false;
						});

					const FText TransactionText = FText::Format(LOCTEXT("PasteModesTransaction", "Paste Fixture {0}|plural(one=Mode, other=Modes)"), GetSelectedModeIndices().Num());
					const FScopedTransaction PasteModesTransaction(TransactionText);

					FixtureType->PreEditChange(UDMXEntityFixtureType::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FDMXFixtureMode, Functions)));

					int32 NumPastedElements = 0;
					for (int32 ClipboardIndex = 0; ClipboardIndex < FunctionsClipboard.Num(); ClipboardIndex++)
					{
						TSharedPtr<FJsonObject> RootJsonObject;
						TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(FunctionsClipboard[ClipboardIndex]);
						FJsonSerializer::Deserialize(Reader, RootJsonObject);

						if (RootJsonObject.IsValid())
						{
							FDMXFixtureFunction FunctionFromClipboard;
							if (FJsonObjectConverter::JsonObjectToUStruct(RootJsonObject.ToSharedRef(), FDMXFixtureFunction::StaticStruct(), &FunctionFromClipboard, 0, 0))
							{
								// Find an index to paste onto
								const int32 PasteOntoIndex = [Mode, ClipboardIndex, this]()
								{
									if (Selection->SelectedFunctionIndices.IsValidIndex(ClipboardIndex))
									{
										return Selection->SelectedFunctionIndices[ClipboardIndex];
									}
									else if (Selection->SelectedFunctionIndices.Num() > 0)
									{
										return Selection->SelectedFunctionIndices.Last();
									}
									else
									{
										// Paste at the end of the Functions array
										return Mode.Functions.Num() - 1;
									}
								}();

								FDMXFixtureFunction NewFunction = FunctionFromClipboard;

								// Insert the Function
								const int32 DesiredIndex = PasteOntoIndex + 1 + NumPastedElements;
								const int32 ResultingIndex = FixtureType->InsertFunction(SelectedModeIndex, DesiredIndex, NewFunction);
								OutNewlyAddedFunctionIndices.Add(ResultingIndex);

								NumPastedElements++;
							}
						}
					}

					if (NumPastedElements > 0)
					{
						FixtureType->UpdateChannelSpan(SelectedModeIndex);
					}

					FixtureType->PostEditChange();
				}
			}
		}
	}
}

void FDMXFixtureTypeSharedData::SelectFunctions(const TArray<int32>& InFunctionIndices)
{
	// DEPRECATED 5.0
	check(Selection);
	if (Selection->SelectedFunctionIndices != InFunctionIndices)
	{
		Selection->Modify();
		Selection->SelectedFunctionIndices = InFunctionIndices;
		OnFunctionsSelected.Broadcast();
	}
}


const TArray<TWeakObjectPtr<UDMXEntityFixtureType>>& FDMXFixtureTypeSharedData::GetSelectedFixtureTypes() const 
{
	check(Selection);
	return Selection->SelectedFixtureTypes;
}

const TArray<int32>& FDMXFixtureTypeSharedData::GetSelectedModeIndices() const
{
	check(Selection);
	return Selection->SelectedModeIndices;
}

const TArray<int32>& FDMXFixtureTypeSharedData::GetSelectedFunctionIndices() const
{
	check(Selection);
	return Selection->SelectedFunctionIndices;
}

bool FDMXFixtureTypeSharedData::IsFixtureMatrixSelected() const
{
	check(Selection);
	return Selection->bFixtureMatrixSelected;
}

bool FDMXFixtureTypeSharedData::CanAddMode() const
{
	// DEPRECATED 5.0
	check(Selection);
	return Selection->SelectedFixtureTypes.Num() == 1;
}

void FDMXFixtureTypeSharedData::AddMode()
{
	// DEPRECATED 5.0

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const bool bCanAddMode = CanAddMode();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (ensureMsgf(bCanAddMode, TEXT("Call to FDMXFixtureTypeSharedData::AddMode when no mode can be added")))
	{
		check(Selection);
		for (const TWeakObjectPtr<UDMXEntityFixtureType>& FixtureType : Selection->SelectedFixtureTypes)
		{
			if (!FixtureType.IsValid())
			{
				continue;
			}

			FDMXFixtureMode NewMode;

			// Make a unique name for the new mode
			TSet<FString> ModeNames;
			for (const FDMXFixtureMode& Mode : FixtureType->Modes)
			{
				ModeNames.Add(Mode.ModeName);
			}
			NewMode.ModeName = FDMXRuntimeUtils::GenerateUniqueNameFromExisting(ModeNames, LOCTEXT("DefaultModeName", "Mode").ToString());

			const FDMXFixtureTypeSharedDataDetails::FDMXScopedModeEditChangeChainProperty_DEPRECATED ScopedChange(FixtureType.Get());

			FixtureType->Modes.Add(NewMode);
		}
	}
}

void FDMXFixtureTypeSharedData::DuplicateModes(const TArray<int32>& ModeÎndiciesToDuplicate)
{
	// DEPRECATED 5.0
	check(Selection);
	for (const TWeakObjectPtr<UDMXEntityFixtureType>& WeakFixtureType : Selection->SelectedFixtureTypes)
	{ 
		if (UDMXEntityFixtureType* FixtureType = WeakFixtureType.Get())
		{
			for (int32 ModeIndex : ModeÎndiciesToDuplicate)
			{
				if (FixtureType->Modes.IsValidIndex(ModeIndex))
				{
					// Copy
					FDMXFixtureMode NewMode = FixtureType->Modes[ModeIndex];

					// Get a unique name
					TSet<FString> ModeNames;
					for (const FDMXFixtureMode& Mode : FixtureType->Modes)
					{
						ModeNames.Add(Mode.ModeName);
					}
					NewMode.ModeName = FDMXRuntimeUtils::GenerateUniqueNameFromExisting(ModeNames, NewMode.ModeName);

					const FDMXFixtureTypeSharedDataDetails::FDMXScopedModeEditChangeChainProperty_DEPRECATED ScopedChange(FixtureType);

					if (FixtureType->Modes.IsValidIndex(ModeIndex + 1))
					{
						FixtureType->Modes.Insert(NewMode, ModeIndex + 1);
					}
					else
					{
						FixtureType->Modes.Add(NewMode);
					}
				}
			}
		}
	}
}

void FDMXFixtureTypeSharedData::DeleteModes(const TArray<int32>& ModeIndicesToDelete)
{
	// DEPRECATED 5.0
	check(Selection);
	for (const TWeakObjectPtr<UDMXEntityFixtureType>& WeakFixtureType : Selection->SelectedFixtureTypes)
	{
		if (UDMXEntityFixtureType* FixtureType = WeakFixtureType.Get())
		{
			for (int32 ModeIndex : ModeIndicesToDelete)
			{
				if (FixtureType->Modes.IsValidIndex(ModeIndex))
				{
					// Transaction
					const FScopedTransaction Transaction = FScopedTransaction(LOCTEXT("DeleteModeTransaction", "Delete Fixture Type Mode"));

					const FDMXFixtureTypeSharedDataDetails::FDMXScopedModeEditChangeChainProperty_DEPRECATED ScopedChange(FixtureType);

					FixtureType->Modes.RemoveAt(ModeIndex);
				}
			}
		}
	}
}

void FDMXFixtureTypeSharedData::PasteClipboardToModes(const TArray<int32>& ModeIndices)
{	
	// DEPRECATED 5.0

	for (int32 PasteOntoElementIndex = 0; PasteOntoElementIndex < ModeIndices.Num(); PasteOntoElementIndex++)
	{
		const int32 PasteOntoIndex = ModeIndices[PasteOntoElementIndex];

		const FString& ClipboardString = [PasteOntoElementIndex, this]()
		{
			if (ModesClipboard.IsValidIndex(PasteOntoElementIndex))
			{
				return ModesClipboard[PasteOntoElementIndex];
			}
			else if (ModesClipboard.Num() > 0)
			{
				return ModesClipboard.Last();
			}

			return FString();
		}();

		TSharedPtr<FJsonObject> RootJsonObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(ClipboardString);
		FJsonSerializer::Deserialize(Reader, RootJsonObject);

		if (RootJsonObject.IsValid())
		{
			FDMXFixtureMode ModeFromClipboard;
			if (FJsonObjectConverter::JsonObjectToUStruct(RootJsonObject.ToSharedRef(), FDMXFixtureMode::StaticStruct(), &ModeFromClipboard, 0, 0))
			{
				check(Selection);
				for (const TWeakObjectPtr<UDMXEntityFixtureType>& WeakFixtureType : Selection->SelectedFixtureTypes)
				{
					if (UDMXEntityFixtureType* FixtureType = WeakFixtureType.Get())
					{
						if (FixtureType->Modes.IsValidIndex(PasteOntoIndex))
						{
							FDMXFixtureMode& PasteOntoMode = FixtureType->Modes[PasteOntoIndex];

							FString OldModeName = PasteOntoMode.ModeName;

							FixtureType->PreEditChange(UDMXEntityFixtureType::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_STRING_CHECKED(UDMXEntityFixtureType, Modes)));
							const FScopedTransaction Transaction = FScopedTransaction(LOCTEXT("PasteModeTransaction", "Paste Fixture Type Mode"));
							FixtureType->Modify();

							PasteOntoMode = ModeFromClipboard;

							// Keep the original mode name
							PasteOntoMode.ModeName = OldModeName;

							FixtureType->PostEditChange();
						}
					}
				}
			}
		}
	}
}

bool FDMXFixtureTypeSharedData::CanAddFunction() const
{
	// DEPRECATED 5.0
	check(Selection);
	return Selection->SelectedModeIndices.Num() == 1;
}

void FDMXFixtureTypeSharedData::AddFunctionToSelectedMode()
{
	// DEPRECATED 5.0

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const bool bCanAddFunction = CanAddFunction();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (ensureMsgf(bCanAddFunction, TEXT("Call to FDMXFixtureTypeSharedData::AddFunctionToSelectedMode, when no function can be added")))
	{
		check(Selection);
		for (const TWeakObjectPtr<UDMXEntityFixtureType>& WeakFixtureType : Selection->SelectedFixtureTypes)
		{
			if (UDMXEntityFixtureType* FixtureType = WeakFixtureType.Get())
			{
				const FScopedTransaction AddFunctionTransaction = FScopedTransaction(LOCTEXT("AddFunctionTransaction", "Add Fixture Type Function"));

				FixtureType->PreEditChange(UDMXEntityFixtureType::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_STRING_CHECKED(UDMXEntityFixtureType, Modes)));
				FixtureType->Modify();

				int32 ModeIndex = Selection->SelectedModeIndices[0]; // Implicitly valid via CanAddFunctions tested before
				if (FixtureType->Modes.IsValidIndex(ModeIndex))
				{
					FDMXFixtureMode& Mode = FixtureType->Modes[ModeIndex];

					// Get a unique name
					TSet<FString> FunctionNames;
					for (const FDMXFixtureFunction& Function : Mode.Functions)
					{
						FunctionNames.Add(Function.FunctionName);
					}
					FDMXFixtureFunction NewFunction;
					NewFunction.FunctionName = FDMXRuntimeUtils::GenerateUniqueNameFromExisting(FunctionNames, LOCTEXT("DMXFixtureTypeSharedData.DefaultFunctionName", "Function").ToString());

					Mode.Functions.Add(NewFunction);
					FixtureType->UpdateChannelSpan(ModeIndex);
				}

				FixtureType->PostEditChange();
			}
		}
	}
}

void FDMXFixtureTypeSharedData::DuplicateFunctions(const TArray<int32>& FunctionIndicesToDuplicate)
{
	// DEPRECATED 5.0
	check(Selection);
	for (const TWeakObjectPtr<UDMXEntityFixtureType>& WeakFixtureType : Selection->SelectedFixtureTypes)
	{
		if (UDMXEntityFixtureType* FixtureType = WeakFixtureType.Get())
		{
			for (int32 ModeIndex : Selection->SelectedModeIndices)
			{
				if (FixtureType->Modes.IsValidIndex(ModeIndex))
				{
					FDMXFixtureMode& Mode = FixtureType->Modes[ModeIndex];

					// Transaction
					const FScopedTransaction Transaction = FScopedTransaction(LOCTEXT("DuplicatFunctionTransaction", "Duplicate Fixture Type Function"));

					FixtureType->PreEditChange(UDMXEntityFixtureType::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_STRING_CHECKED(UDMXEntityFixtureType, Modes)));
					FixtureType->Modify();

					int32 NumDuplicates = 0;
					for (int32 FunctionIndex : FunctionIndicesToDuplicate)
					{
						const TArray<int32> DuplicateAtIndices({ FunctionIndex + NumDuplicates });
						TArray<int32> NewlyAddedFunctionIndices;
						FixtureType->DuplicateFunctions(ModeIndex, DuplicateAtIndices, NewlyAddedFunctionIndices);

						NumDuplicates++;
					}

					FixtureType->UpdateChannelSpan(ModeIndex);

					FixtureType->PostEditChange();
				}
			}
		}
	}
}

void FDMXFixtureTypeSharedData::DeleteFunctions(const TArray<int32>& FunctionIndicesToDelete)
{
	// DEPRECATED 5.0
	check(Selection);
	for (const TWeakObjectPtr<UDMXEntityFixtureType>& WeakFixtureType : Selection->SelectedFixtureTypes)
	{
		if (UDMXEntityFixtureType* FixtureType = WeakFixtureType.Get())
		{
			for (int32 ModeIndex : Selection->SelectedModeIndices)
			{
				if (FixtureType->Modes.IsValidIndex(ModeIndex))
				{
					FDMXFixtureMode& Mode = FixtureType->Modes[ModeIndex];

					const FScopedTransaction Transaction = FScopedTransaction(LOCTEXT("DeleteFunctionsTransaction", "Deleted Fixture Type Functions"));

					// Required this way for the 5.0 Deprecated UDMXEntityFixtureType::OnDataTypeChanged event
					FArrayProperty* ModesProperty = FindFProperty<FArrayProperty>(UDMXEntityFixtureType::StaticClass(), GET_MEMBER_NAME_CHECKED(UDMXEntityFixtureType, Modes));
					FArrayProperty* FunctionsProperty = FindFProperty<FArrayProperty>(FDMXFixtureMode::StaticStruct(), GET_MEMBER_NAME_CHECKED(FDMXFixtureMode, Functions));
					FEditPropertyChain PropertyChain;
					PropertyChain.AddHead(ModesProperty);
					PropertyChain.AddTail(FunctionsProperty);
					PropertyChain.SetActiveMemberPropertyNode(ModesProperty);
					PropertyChain.SetActivePropertyNode(FunctionsProperty);

					FixtureType->PreEditChange(FunctionsProperty);
					FixtureType->Modify();

					for (const int32 FunctionIndex : FunctionIndicesToDelete)
					{
						if (Mode.Functions.IsValidIndex(FunctionIndex))
						{
							const TArray<int32> RemoveAtIndices({ FunctionIndex });
							FixtureType->RemoveFunctions(ModeIndex, RemoveAtIndices);
						}
					}

					FPropertyChangedEvent PropertyChangedEvent(FunctionsProperty, EPropertyChangeType::ArrayRemove);
					FPropertyChangedChainEvent PropertyChangedChainEvent(PropertyChain, PropertyChangedEvent);

					FixtureType->PostEditChangeChainProperty(PropertyChangedChainEvent);
				}
			}
		}
	}
}

void FDMXFixtureTypeSharedData::PasteClipboardToFunctions(const TArray<int32>& FunctionIndices)
{
	// DEPRECATED 5.0

	for (int32 PasteOntoElementIndex = 0; PasteOntoElementIndex < FunctionIndices.Num(); PasteOntoElementIndex++)
	{
		const int32 PasteOntoIndex = FunctionIndices[PasteOntoElementIndex];

		const FString& ClipboardString = [PasteOntoElementIndex, this]()
		{
			if (FunctionsClipboard.IsValidIndex(PasteOntoElementIndex))
			{
				return FunctionsClipboard[PasteOntoElementIndex];
			}
			else if (FunctionsClipboard.Num() > 0)
			{
				return FunctionsClipboard.Last();
			}

			return FString();
		}();

		TSharedPtr<FJsonObject> RootJsonObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(ClipboardString);
		FJsonSerializer::Deserialize(Reader, RootJsonObject);

		if (RootJsonObject.IsValid())
		{
			FDMXFixtureFunction FunctionFromClipboard;
			if (FJsonObjectConverter::JsonObjectToUStruct(RootJsonObject.ToSharedRef(), FDMXFixtureFunction::StaticStruct(), &FunctionFromClipboard, 0, 0))
			{
				check(Selection);
				for (const TWeakObjectPtr<UDMXEntityFixtureType>& WeakFixtureType : Selection->SelectedFixtureTypes)
				{
					if (UDMXEntityFixtureType* FixtureType = WeakFixtureType.Get())
					{
						const FScopedTransaction ModeNameTransaction = FScopedTransaction(LOCTEXT("PasteModeTransaction", "Paste Fixture Type Mode"));

						FixtureType->PreEditChange(UDMXEntityFixtureType::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_STRING_CHECKED(UDMXEntityFixtureType, Modes)));
						FixtureType->Modify();

						for (int32 ModeIndex : Selection->SelectedModeIndices)
						{
							if (FixtureType->Modes.IsValidIndex(ModeIndex))
							{
								FDMXFixtureMode& Mode = FixtureType->Modes[ModeIndex];

								if (Mode.Functions.IsValidIndex(PasteOntoIndex))
								{
									FDMXFixtureFunction& PasteOntoFunction = Mode.Functions[PasteOntoIndex];

									FString OldFunctionName = PasteOntoFunction.FunctionName;

									PasteOntoFunction = FunctionFromClipboard;

									// Keep the original function name
									PasteOntoFunction.FunctionName = OldFunctionName;
								}
							}
						}

						FixtureType->PostEditChange();
					}
				}
			}
		}
	}
}

bool FDMXFixtureTypeSharedData::CanAddCellAttribute() const
{
	// DEPRECATED 5.0
	check(Selection);
	return 
		Selection->SelectedFixtureTypes.Num() == 1 &&
		Selection->SelectedModeIndices.Num() == 1;
}

void FDMXFixtureTypeSharedData::AddCellAttributeToSelectedMode()
{
	// DEPRECATED 5.0

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const bool bCanAddCellAttribute = CanAddCellAttribute();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (ensureMsgf(bCanAddCellAttribute, TEXT("Call to FDMXFixtureTypeSharedData::AddCellAttributeToSelectedMode, when no Cell Attribute can be added.")))
	{
		check(Selection);
		if (UDMXEntityFixtureType* FixtureType = Selection->SelectedFixtureTypes[0].Get())
		{
			const int32 SelectedModeIndex = Selection->SelectedModeIndices[0];
			if (FixtureType->Modes.IsValidIndex(SelectedModeIndex) && FixtureType->Modes[SelectedModeIndex].bFixtureMatrixEnabled)
			{
				FDMXFixtureMode& Mode = FixtureType->Modes[Selection->SelectedModeIndices[0]];

				FDMXFixtureCellAttribute NewAttribute;
				TArray<FName> AvailableAttributes = FDMXAttributeName::GetPredefinedValues();
				NewAttribute.Attribute = AvailableAttributes.Num() > 0 ? FDMXAttributeName(AvailableAttributes[0]) : FDMXAttributeName();

				const FDMXFixtureTypeSharedDataDetails::FDMXScopedModeEditChangeChainProperty_DEPRECATED ScopedChange(FixtureType);

				Mode.FixtureMatrixConfig.CellAttributes.Add(NewAttribute);

				// Update affected properties
				FixtureType->UpdateChannelSpan(SelectedModeIndex);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
