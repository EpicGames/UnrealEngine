// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPIEManager.h"

#include "DMXSubsystem.h"
#include "Editor.h"


FDMXPIEManager::FDMXPIEManager()
{
	FEditorDelegates::BeginPIE.AddRaw(this, &FDMXPIEManager::OnBeginPIE);
	FEditorDelegates::EndPIE.AddRaw(this, &FDMXPIEManager::OnEndPIE);
}

FDMXPIEManager::~FDMXPIEManager()
{
	FEditorDelegates::BeginPIE.RemoveAll(this);
	FEditorDelegates::EndPIE.RemoveAll(this);
}

void FDMXPIEManager::OnBeginPIE(const bool bIsSimulating)
{
	UDMXSubsystem::ClearDMXBuffers();
}

void FDMXPIEManager::OnEndPIE(const bool bIsSimulating)
{
	UDMXSubsystem::ClearDMXBuffers();
}
