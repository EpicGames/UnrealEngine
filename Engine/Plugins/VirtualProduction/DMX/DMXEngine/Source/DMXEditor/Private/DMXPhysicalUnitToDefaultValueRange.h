// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "GDTF/AttributeDefinitions/DMXGDTFPhysicalUnit.h"

namespace UE::DMX
{
	struct FDMXPhysicalUnitToDefaultValueRange
	{
		/** Returns the value range for specified physical unit */
		static const TPair<double, double>& GetValueRange(EDMXGDTFPhysicalUnit PhysicalUnit);

	private:
		/** A map to list all GDTF compliant Physical Units and their matching Physical Value default ranges. */
		static const TMap<EDMXGDTFPhysicalUnit, TPair<double, double>> PhysicalUnitToValueRangeMap;
	};
}
