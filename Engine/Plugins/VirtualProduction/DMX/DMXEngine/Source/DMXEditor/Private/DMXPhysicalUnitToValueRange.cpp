// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPhysicalUnitToDefaultValueRange.h"

namespace UE::DMX
{
	const TPair<double, double>& FDMXPhysicalUnitToDefaultValueRange::GetValueRange(EDMXGDTFPhysicalUnit PhysicalUnit)
	{
		return PhysicalUnitToValueRangeMap.FindChecked(PhysicalUnit);
	}

	const TMap<EDMXGDTFPhysicalUnit, TPair<double, double>> FDMXPhysicalUnitToDefaultValueRange::PhysicalUnitToValueRangeMap
	{
		{ EDMXGDTFPhysicalUnit::None, {0.0, 1.0} },
		{ EDMXGDTFPhysicalUnit::Percent, {0.0, 100.0} },
		{ EDMXGDTFPhysicalUnit::Length, {0.1, 100.0} },
		{ EDMXGDTFPhysicalUnit::Mass, {0.1, 100.0} },
		{ EDMXGDTFPhysicalUnit::Time, {0.001, 60.0} },
		{ EDMXGDTFPhysicalUnit::Temperature, {1000.0, 20000.0} },
		{ EDMXGDTFPhysicalUnit::LuminousIntensity, {0.0, 100000.0} },
		{ EDMXGDTFPhysicalUnit::Angle, {-120.0, 120.0} },
		{ EDMXGDTFPhysicalUnit::Force, {0.1, 1000.0} },
		{ EDMXGDTFPhysicalUnit::Frequency, {0.0, 30.0} },
		{ EDMXGDTFPhysicalUnit::Current, {0.01, 100.0} },
		{ EDMXGDTFPhysicalUnit::Voltage, {5.0, 400.0} },
		{ EDMXGDTFPhysicalUnit::Power, {1.0, 10000.0} },
		{ EDMXGDTFPhysicalUnit::Energy, {0.001, 100000.0} },
		{ EDMXGDTFPhysicalUnit::Area, {0.0001, 10.0} },
		{ EDMXGDTFPhysicalUnit::Volume, {0.001, 1.0} },
		{ EDMXGDTFPhysicalUnit::Speed, {0.1, 10.0} },
		{ EDMXGDTFPhysicalUnit::Acceleration, {0.1, 50.0} },
		{ EDMXGDTFPhysicalUnit::AngularSpeed, {1.0, 360.0} },
		{ EDMXGDTFPhysicalUnit::AngularAccc, {1.0, 1000.0} },
		{ EDMXGDTFPhysicalUnit::WaveLength, {100.0, 1000000.0} },
		{ EDMXGDTFPhysicalUnit::ColorComponent, {0.0, 100.0} },
		{ EDMXGDTFPhysicalUnit::MaxEnumValue, {0.0, 0.0} }
	};
}
