// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXEntityDragDropOp.h"

#include "DMXEditorUtils.h"
#include "Library/DMXEntity.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXLibrary.h"

#include "Dialogs/Dialogs.h"

#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "ScopedTransaction.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "SDMXEntityListBase"

///////////////////////////////////////////////////////////////////////////////
// FDMXEntityDragDropOperation

FDMXEntityDragDropOperation::FDMXEntityDragDropOperation(UDMXLibrary* InLibrary, const TArray<TWeakObjectPtr<UDMXEntity>>& InEntities)
	: DraggedFromLibrary(InLibrary)
	, DraggedEntities(InEntities)
{
	DraggedEntitiesName = DraggedEntities.Num() == 1
		? FText::FromString(TEXT("'") + DraggedEntities[0]->GetDisplayName() + TEXT("'"))
		: FDMXEditorUtils::GetEntityTypeNameText(DraggedEntities[0]->GetClass(), true);

	Construct();
}

void FDMXEntityDragDropOperation::Construct()
{
	// Create the drag-drop decorator window
	CursorDecoratorWindow = SWindow::MakeCursorDecorator();
	const bool bShowImmediately = false;
	FSlateApplication::Get().AddWindow(CursorDecoratorWindow.ToSharedRef(), bShowImmediately);
}

void FDMXEntityDragDropOperation::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(DraggedFromLibrary);
}

TArray<UClass*> FDMXEntityDragDropOperation::GetDraggedEntityTypes() const
{
	TArray<UClass*> EntityClasses;
	for (TWeakObjectPtr<UDMXEntity> Entity : DraggedEntities)
	{
		if (!Entity.IsValid())
		{
			continue;
		}

		EntityClasses.Add(Entity->GetClass());
	}

	return EntityClasses;
}

void FDMXEntityDragDropOperation::SetFeedbackMessageError(const FText& Message)
{
	const FSlateBrush* StatusSymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
	SetFeedbackMessage(StatusSymbol, Message);
}

void FDMXEntityDragDropOperation::SetFeedbackMessageOK(const FText& Message)
{
	const FSlateBrush* StatusSymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
	SetFeedbackMessage(StatusSymbol, Message);
}

void FDMXEntityDragDropOperation::SetFeedbackMessage(const FSlateBrush* Icon, const FText& Message)
{
	if (!Message.IsEmpty())
	{
		CursorDecoratorWindow->ShowWindow();
		CursorDecoratorWindow->SetContent
		(
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Graph.ConnectorFeedback.Border"))
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(3.0f)
				[
					SNew(SScaleBox)
					.Stretch(EStretch::ScaleToFit)
					[
						SNew(SImage)
						.Image(Icon)
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(3.0f)
				.MaxWidth(500)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.WrapTextAt(480.0f)
					.Text(Message)
					.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
				]
			]
		);
	}
	else
	{
		CursorDecoratorWindow->HideWindow();
		CursorDecoratorWindow->SetContent(SNullWidget::NullWidget);
	}
}

void FDMXEntityDragDropOperation::SetCustomFeedbackWidget(const TSharedRef<SWidget>& Widget)
{
	CursorDecoratorWindow->ShowWindow();
	CursorDecoratorWindow->SetContent
	(
		Widget
	);
}

#undef LOCTEXT_NAMESPACE
