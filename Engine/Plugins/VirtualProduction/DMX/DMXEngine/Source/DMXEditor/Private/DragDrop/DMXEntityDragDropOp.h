// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/DragAndDrop.h"
#include "UObject/GCObject.h"

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FDMXTreeNodeBase;
class SDMXCategoryRow;
class FDMXEntityTreeNode;
class SDMXEntityRow;
class UDMXEntity;
class UDMXEntityFixturePatch;
class UDMXLibrary;
struct FSlateBrush;

class FDMXEntityDragDropOperation
	: public FDragDropOperation
	, public FGCObject
{
public:
	DRAG_DROP_OPERATOR_TYPE(FDMXEntityDragDropOperation, FDragDropOperation)

	/**  Constructs the entity drag drop operation */
	FDMXEntityDragDropOperation(UDMXLibrary* InLibrary, const TArray<TWeakObjectPtr<UDMXEntity>>& InEntities);

protected:
	/** Constructs the tooltip widget that follows the mouse */
	virtual void Construct() override;

	//~ Begin FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FDMXEntityDragDropOperation");
	}
	//~ End FGCObject interface

public:
	/** Returns the entities dragged with this drag drop op */
	const TArray<TWeakObjectPtr<UDMXEntity>>& GetDraggedEntities() const { return DraggedEntities; }

	/** Returns the dragged entity's names or the first of the dragged entities names */
	const FText& GetDraggedEntitiesName() const { return DraggedEntitiesName; }

	/** Returns the types of the entities */
	TArray<UClass*> GetDraggedEntityTypes() const;

	/** Sets the cursor decorrator to show an error sign and a message */
	void SetFeedbackMessageError(const FText& Message);

	/** Sets the cursor decorrator to show an ok sign and a message */
	void SetFeedbackMessageOK(const FText& Message);

	/** Sets the cursor decorator to show an icon and a message */
	void SetFeedbackMessage(const FSlateBrush* Icon, const FText& Message);

	/** Sets a custom cursor decorator */
	void SetCustomFeedbackWidget(const TSharedRef<SWidget>& Widget);

private:
	/** The library that the entites were dragged out from */
	TObjectPtr<UDMXLibrary> DraggedFromLibrary;

	/** The entities being draged with this drag drop op */
	TArray<TWeakObjectPtr<UDMXEntity>> DraggedEntities;

	/** Name of the entity being dragged or entities type for several ones */
	FText DraggedEntitiesName;
};
