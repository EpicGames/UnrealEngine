// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXEntityFixturePatchDragDropOp.h"

#include "Library/DMXEntityFixturePatch.h"

#include "Editor.h"


#define LOCTEXT_NAMESPACE "FDMXEntityFixturePatchDragDropOperation"

FDMXEntityFixturePatchDragDropOperation::FDMXEntityFixturePatchDragDropOperation(const TMap<TWeakObjectPtr<UDMXEntityFixturePatch>, int64>& InFixturePatchToAbsoluteChannelOffsetMap)
	: FixturePatchToAbsoluteChannelOffsetMap(InFixturePatchToAbsoluteChannelOffsetMap)
{
	GEditor->BeginTransaction(FText::Format(LOCTEXT("DragDropTransaction", "Drag Fixture {0}|plural(one=Patch, other=Patches)"), FixturePatchToAbsoluteChannelOffsetMap.Num()));

	for (const TTuple<TWeakObjectPtr<UDMXEntityFixturePatch>, int64>& FixturePatchToChannelOffsetPair : FixturePatchToAbsoluteChannelOffsetMap)
	{
		if (FixturePatchToChannelOffsetPair.Key.IsValid())
		{
			FixturePatchToChannelOffsetPair.Key->PreEditChange(UDMXEntityFixturePatch::StaticClass()->FindPropertyByName(UDMXEntityFixturePatch::GetUniverseIDPropertyNameChecked()));
			FixturePatchToChannelOffsetPair.Key->PreEditChange(UDMXEntityFixturePatch::StaticClass()->FindPropertyByName(UDMXEntityFixturePatch::GetStartingChannelPropertyNameChecked()));
		}
	}
}

FDMXEntityFixturePatchDragDropOperation::~FDMXEntityFixturePatchDragDropOperation()
{
	for (const TTuple<TWeakObjectPtr<UDMXEntityFixturePatch>, int64>& FixturePatchToChannelOffsetPair : FixturePatchToAbsoluteChannelOffsetMap)
	{
		if (FixturePatchToChannelOffsetPair.Key.IsValid())
		{
			FixturePatchToChannelOffsetPair.Key->PostEditChange();
		}
	}
	GEditor->EndTransaction();
}

#undef LOCTEXT_NAMESPACE
