// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXEntityDragDropOp.h"

#include "Input/DragAndDrop.h"


class FDMXEntityFixturePatchDragDropOperation
	: public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FDMXEntityDragDropOperation, FDragDropOperation)

	/** 
	 * Constructs the fixture patch drag drop operation 
	 * 
	 * @param InLibrary								The library that contains the entities. Dragging from different libraries at once is not supported.
	 * @param InFixturePatchToChannelOffsetMap		Each Fixture Patch being dragged, along with it's absolute channel offset from the anchor position.
	 */
	FDMXEntityFixturePatchDragDropOperation(const TMap<TWeakObjectPtr<UDMXEntityFixturePatch>, int64>& FixturePatchToAbsoluteChannelOffsetMap);

	/** Destructor */
	virtual ~FDMXEntityFixturePatchDragDropOperation();

	/** Returns Each Fixture Patch being dragged, along with it's channel offset from the anchor position. */
	const TMap<TWeakObjectPtr<UDMXEntityFixturePatch>, int64>& GetFixturePatchToAbsoluteChannelOffsetMap() const { return FixturePatchToAbsoluteChannelOffsetMap; }

protected:
	/*	Each Fixture Patch being dragged, along with it's channel offset from the anchor position. */
	TMap<TWeakObjectPtr<UDMXEntityFixturePatch>, int64> FixturePatchToAbsoluteChannelOffsetMap;
};
