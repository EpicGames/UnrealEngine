// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXFixtureFunctionDragDropOp.h"

#include "Widgets/FixtureType/DMXFixtureTypeFunctionsEditorFunctionItem.h"
#include "Widgets/FixtureType/SDMXFixtureTypeFunctionsEditorFunctionRow.h"

#include "Styling/AppStyle.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"


FDMXFixtureFunctionDragDropOp::FDMXFixtureFunctionDragDropOp(TSharedPtr<SDMXFixtureTypeFunctionsEditorFunctionRow> InRow)
{
	Row = InRow;

	InRow->SetIsBeingDragged(true);

	DecoratorWidget = SNew(SBorder)
		.Padding(8.f)
		.BorderImage(FAppStyle::GetBrush("Graph.ConnectorFeedback.Border"))
		.Content()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::Format(NSLOCTEXT("FixtureFunctionDragDrop", "DecoratorTexxt", "Reorder {0}"), InRow->GetFunctionItem()->GetFunctionName()))
			]
		];

	Construct();
}

void FDMXFixtureFunctionDragDropOp::OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent)
{
	FDecoratedDragDropOp::OnDrop(bDropWasHandled, MouseEvent);

	if (TSharedPtr<SDMXFixtureTypeFunctionsEditorFunctionRow> PinnedRow = Row.Pin())
	{
		PinnedRow->SetIsBeingDragged(false);
	}
}
