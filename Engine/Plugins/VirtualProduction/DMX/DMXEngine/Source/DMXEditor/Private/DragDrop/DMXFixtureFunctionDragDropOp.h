// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragAndDrop/DecoratedDragDropOp.h"

#include "Widgets/Views/SListView.h"

class SDMXFixtureTypeFunctionsEditorFunctionRow;


class FDMXFixtureFunctionDragDropOp 
	: public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FDMXFixtureFunctionDragDropOp, FDecoratedDragDropOp)

	FDMXFixtureFunctionDragDropOp(TSharedPtr<SDMXFixtureTypeFunctionsEditorFunctionRow> InRow);

	void OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent);

	TSharedPtr<SWidget> DecoratorWidget;

	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		return DecoratorWidget;
	}

	TWeakPtr<SDMXFixtureTypeFunctionsEditorFunctionRow> Row;
};
