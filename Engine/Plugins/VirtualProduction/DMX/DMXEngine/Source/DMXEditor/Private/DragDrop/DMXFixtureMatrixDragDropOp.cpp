// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXFixtureMatrixDragDropOp.h"

#include "Widgets/FixtureType/DMXFixtureTypeFunctionsEditorMatrixItem.h"
#include "Widgets/FixtureType/SDMXFixtureTypeFunctionsEditorMatrixRow.h"

#include "Styling/AppStyle.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"


FDMXFixtureMatrixDragDropOp::FDMXFixtureMatrixDragDropOp(TSharedPtr<SDMXFixtureTypeFunctionsEditorMatrixRow> InRow)
{
	Row = InRow;

	InRow->SetIsBeingDragged(true);

	DecoratorWidget = SNew(SBorder)
		.Padding(8.f)
		.BorderImage(FAppStyle::GetBrush("Graph.ConnectorFeedback.Border"))
		.Content()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(NSLOCTEXT("FixtureMatrixDragDrop", "DecoratorTexxt", "Reorder Fixture Matrix"))
			]
		];

	Construct();
}

void FDMXFixtureMatrixDragDropOp::OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent)
{
	FDecoratedDragDropOp::OnDrop(bDropWasHandled, MouseEvent);

	if (TSharedPtr<SDMXFixtureTypeFunctionsEditorMatrixRow> PinnedRow = Row.Pin())
	{
		PinnedRow->SetIsBeingDragged(false);
	}
}
