// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragAndDrop/DecoratedDragDropOp.h"

#include "Widgets/Views/SListView.h"

class SDMXFixtureTypeFunctionsEditorMatrixRow;


class FDMXFixtureMatrixDragDropOp 
	: public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FDMXFixtureMatrixDragDropOp, FDecoratedDragDropOp)

	FDMXFixtureMatrixDragDropOp(TSharedPtr<SDMXFixtureTypeFunctionsEditorMatrixRow> InRow);

	void OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent);

	TSharedPtr<SWidget> DecoratorWidget;

	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		return DecoratorWidget;
	}

	TWeakPtr<SDMXFixtureTypeFunctionsEditorMatrixRow> Row;
};
