// Copyright Epic Games, Inc. All Rights Reserved.

#include "Exporters/DMXFixtureTypeToGDTFConverter.h"

#include "Algo/Find.h"
#include "Algo/Sort.h"
#include "DMXFixtureTypeToGDTFGeometryFactory.h"
#include "DMXGDTF.h"
#include "DMXUnrealToGDTFAttributeConversion.h"
#include "GDTF/AttributeDefinitions/DMXGDTFAttribute.h"
#include "GDTF/AttributeDefinitions/DMXGDTFAttributeDefinitions.h"
#include "GDTF/AttributeDefinitions/DMXGDTFFeature.h"
#include "GDTF/AttributeDefinitions/DMXGDTFFeatureGroup.h"
#include "GDTF/AttributeDefinitions/DMXGDTFPhysicalUnit.h"
#include "GDTF/DMXGDTFFixtureType.h"
#include "GDTF/DMXModes/DMXGDTFChannelFunction.h"
#include "GDTF/DMXModes/DMXGDTFDMXChannel.h"
#include "GDTF/DMXModes/DMXGDTFDMXMode.h"
#include "GDTF/DMXModes/DMXGDTFLogicalChannel.h"
#include "GDTF/Geometries/DMXGDTFGeometry.h"
#include "GDTF/Geometries/DMXGDTFGeometryCollect.h"
#include "GDTF/Models/DMXGDTFModel.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/DMXLibrary.h"
#include "XmlFile.h"

namespace UE::DMX::GDTF
{
	TSharedPtr<FXmlFile> FDMXFixtureTypeToGDTFConverter::Convert(const UDMXEntityFixtureType* UnrealFixtureType)
	{
		if (!UnrealFixtureType)
		{
			return nullptr;
		}

		FDMXFixtureTypeToGDTFConverter Converter;

		// Convert to GDTF
		const TSharedRef<FDMXGDTFFixtureType> GDTFFixtureType = Converter.CreateFixtureType(*UnrealFixtureType);

		// Create the XML file
		UDMXGDTF* GDTF = NewObject<UDMXGDTF>();
		GDTF->InitializeFromFixtureType(GDTFFixtureType);
		const TSharedPtr<FXmlFile> DescriptionXml = GDTF->ExportAsXml();

		return DescriptionXml;
	}

	TSharedRef<FDMXGDTFFixtureType> FDMXFixtureTypeToGDTFConverter::CreateFixtureType(const UDMXEntityFixtureType& UnrealFixtureType)
	{
		const TSharedRef<FDMXGDTFFixtureType> GDTFFixtureType = MakeShared<FDMXGDTFFixtureType>();

		GDTFFixtureType->Name = *UnrealFixtureType.Name;
		GDTFFixtureType->ShortName = *UnrealFixtureType.Name;
		GDTFFixtureType->LongName = UnrealFixtureType.GetParentLibrary()->GetName() + TEXT(" ") + UnrealFixtureType.Name;
		GDTFFixtureType->Manufacturer = TEXT("Epic Games");
		GDTFFixtureType->Description = FString::Printf(TEXT("Unreal Engine generated Fixture Type"));
		GDTFFixtureType->FixtureTypeID = FGuid::NewGuid(); // Avoid any ambiguity with previously exported GDTFs, even if they're identical.
		GDTFFixtureType->bCanHaveChildren = false;

		CreateAttributeDefinitions(UnrealFixtureType, GDTFFixtureType);
		CreateModels(UnrealFixtureType, GDTFFixtureType);
		CreateGeometryCollect(UnrealFixtureType, GDTFFixtureType);
		CreateDMXModes(UnrealFixtureType, GDTFFixtureType);

		return GDTFFixtureType;
	}

	void FDMXFixtureTypeToGDTFConverter::CreateAttributeDefinitions(const UDMXEntityFixtureType& UnrealFixtureType, const TSharedRef<FDMXGDTFFixtureType>& GDTFFixtureType)
	{
		// Map of attribute names with their function. If function is nullptr, it's a matrix cell attribute
		TMap<FName, const FDMXFixtureFunction*> AttributeNameToFunctionPtrMap;
		for (const FDMXFixtureMode& Mode : UnrealFixtureType.Modes)
		{
			for (const FDMXFixtureFunction& Function : Mode.Functions)
			{
				AttributeNameToFunctionPtrMap.Add(Function.Attribute.Name, &Function);
			}

			if (Mode.bFixtureMatrixEnabled)
			{
				for (const FDMXFixtureCellAttribute& MatrixAttribute : Mode.FixtureMatrixConfig.CellAttributes)
				{
					AttributeNameToFunctionPtrMap.Add(MatrixAttribute.Attribute.Name, nullptr);
				}
			}
		}

		GDTFFixtureType->AttributeDefinitions = MakeShared<FDMXGDTFAttributeDefinitions>(GDTFFixtureType);
		for (const TTuple<FName, const FDMXFixtureFunction*>& AttributeNameToFunctionPtrPair : AttributeNameToFunctionPtrMap)
		{
			const FName GDTFAttributeName = FDMXUnrealToGDTFAttributeConversion::ConvertUnrealToGDTFAttribute(AttributeNameToFunctionPtrPair.Key);
			
			const FName PrettyName = FDMXUnrealToGDTFAttributeConversion::GetPrettyFromGDTFAttribute(GDTFAttributeName);
			const FName FeatureGroupName = FDMXUnrealToGDTFAttributeConversion::GetFeatureGroupForGDTFAttribute(GDTFAttributeName);
			const FName FeatureName = FDMXUnrealToGDTFAttributeConversion::GetFeatureForGDTFAttribute(GDTFAttributeName);

			// Get or create the GDTF feature group
			const TSharedRef<FDMXGDTFFeatureGroup> GDTFFeatureGroup = [&GDTFFixtureType, &FeatureGroupName]()
				{		
					const TSharedPtr<FDMXGDTFFeatureGroup>* GDTFFeatureGroupPtr = Algo::FindBy(GDTFFixtureType->AttributeDefinitions->FeatureGroups, FeatureGroupName, &FDMXGDTFFeatureGroup::Name);

					if (GDTFFeatureGroupPtr)
					{
						return (*GDTFFeatureGroupPtr).ToSharedRef();
					}
					else
					{
						const TSharedRef<FDMXGDTFFeatureGroup> NewFeatureGroup = MakeShared<FDMXGDTFFeatureGroup>(GDTFFixtureType->AttributeDefinitions.ToSharedRef());
						GDTFFixtureType->AttributeDefinitions->FeatureGroups.Add(NewFeatureGroup);

						return NewFeatureGroup;
					}
				}();

			GDTFFeatureGroup->Name = FeatureGroupName;

			// Get or create the GDTF feature
			const TSharedRef<FDMXGDTFFeature> GDTFFeature = [GDTFFeatureGroup, &FeatureName]()
				{					
					const TSharedPtr<FDMXGDTFFeature>* GDTFFeaturePtr = Algo::FindBy(GDTFFeatureGroup->FeatureArray, FeatureName, &FDMXGDTFFeature::Name);
					if (GDTFFeaturePtr)
					{
						return (*GDTFFeaturePtr).ToSharedRef();
					}
					else
					{
						const TSharedRef<FDMXGDTFFeature> NewFeature = MakeShared<FDMXGDTFFeature>(GDTFFeatureGroup);
						GDTFFeatureGroup->FeatureArray.Add(NewFeature);

						return NewFeature;
					}
				}();

			GDTFFeature->Name = FeatureName;

			// Create GDTF attribute
			const TSharedRef<FDMXGDTFAttribute> GDTFAttribute = MakeShared<FDMXGDTFAttribute>(GDTFFixtureType->AttributeDefinitions.ToSharedRef());
			GDTFFixtureType->AttributeDefinitions->Attributes.Add(GDTFAttribute);

			const FDMXFixtureFunction* FunctionPtr = AttributeNameToFunctionPtrPair.Value;

			GDTFAttribute->Name = GDTFAttributeName;
			GDTFAttribute->Pretty = PrettyName.ToString();
			GDTFAttribute->PhysicalUnit = FunctionPtr ? FunctionPtr->GetPhysicalUnit() : EDMXGDTFPhysicalUnit::None;
			GDTFAttribute->Feature = FeatureGroupName.ToString() + TEXT(".") + FeatureName.ToString();
		}
	}

	void FDMXFixtureTypeToGDTFConverter::CreateModels(const UDMXEntityFixtureType& UnrealFixtureType, const TSharedRef<FDMXGDTFFixtureType>& GDTFFixtureType)
	{
		// Create a model for the matrix if this is a matrix
		const bool bIsMatrix = Algo::FindBy(UnrealFixtureType.Modes, true, &FDMXFixtureMode::bFixtureMatrixEnabled) != nullptr;
		if (bIsMatrix)
		{
			const TSharedRef<FDMXGDTFModel> InstanceModel = MakeShared<FDMXGDTFModel>(GDTFFixtureType);
			GDTFFixtureType->Models.Add(InstanceModel);

			InstanceModel->Name = FDMXFixtureTypeToGDTFGeometryFactory::CellsModelName;
			InstanceModel->PrimitiveType = EDMXGDTFModelPrimitiveType::Cube;
			InstanceModel->Height = 0.01f;
			InstanceModel->Length = 1.f;
			InstanceModel->Width = 0.3f;
		}
	}

	void FDMXFixtureTypeToGDTFConverter::CreateGeometryCollect(const UDMXEntityFixtureType& UnrealFixtureType, const TSharedRef<FDMXGDTFFixtureType>& GDTFFixtureType)
	{
		GDTFFixtureType->GeometryCollect = MakeShared<FDMXGDTFGeometryCollect>(GDTFFixtureType);

		FDMXFixtureTypeToGDTFGeometryFactory GeometryFactory(UnrealFixtureType, GDTFFixtureType->GeometryCollect.ToSharedRef());

		ModesWithBaseGeometry = GeometryFactory.GetModesWithBaseGeometry();
		FunctionsWithControlledGeometry = GeometryFactory.GetFunctionsWithControlledGeometry();
	}

	void FDMXFixtureTypeToGDTFConverter::CreateDMXModes(const UDMXEntityFixtureType& UnrealFixtureType, const TSharedRef<FDMXGDTFFixtureType>& GDTFFixtureType)
	{
		for (const FDMXFixtureMode& UnrealMode : UnrealFixtureType.Modes)
		{
			const FDMXFixtureModeWithBaseGeometry* ModeWithBaseGeometryPtr = Algo::FindBy(ModesWithBaseGeometry, &UnrealMode, &FDMXFixtureModeWithBaseGeometry::ModePtr);
			if (!ensureMsgf(ModeWithBaseGeometryPtr, TEXT("%hs: Unexpected cannot controlled base geometry for DMX Mode '%s'. Failed to convert mode to GDTF."), __FUNCTION__, *UnrealMode.ModeName))
			{
				continue;
			}

			// Create the mode
			const TSharedRef<FDMXGDTFDMXMode> DMXMode = MakeShared<FDMXGDTFDMXMode>(GDTFFixtureType);
			GDTFFixtureType->DMXModes.Add(DMXMode);

			DMXMode->Name = *UnrealMode.ModeName;
			DMXMode->Description = TEXT("Unreal Engine generated DMX Mode");
			DMXMode->Geometry = ModeWithBaseGeometryPtr->BaseGeometry->Name;

			CreateDMXChannels(UnrealMode, DMXMode);
		}
	}

	void FDMXFixtureTypeToGDTFConverter::CreateDMXChannels(const FDMXFixtureMode& UnrealMode, const TSharedRef<FDMXGDTFDMXMode>& GDTFDMXMode)
	{			
		// Create DMX Channels for non-matrix Unreal Functions
		for (const FDMXFixtureFunction& UnrealFunction : UnrealMode.Functions)
		{
			const FDMXFixtureFunctionWithControlledGeometry* FunctionWithControlledGeometryPtr = Algo::FindBy(FunctionsWithControlledGeometry, &UnrealFunction, &FDMXFixtureFunctionWithControlledGeometry::FunctionPtr);
			if (!ensureMsgf(FunctionWithControlledGeometryPtr, TEXT("%hs: Unexpected cannot find controlled geometry for DMX Function '%s'. Failed to convert mode to GDTF."), __FUNCTION__, *UnrealFunction.FunctionName))
			{
				continue;
			}

			const TSharedRef<FDMXGDTFDMXChannel> DMXChannel = MakeShared<FDMXGDTFDMXChannel>(GDTFDMXMode);
			GDTFDMXMode->DMXChannels.Add(DMXChannel);

			const FString ControlledGeometryName = FunctionWithControlledGeometryPtr->ControlledGeometry->Name.ToString();
			const FString GDTFAttribute = FDMXUnrealToGDTFAttributeConversion::ConvertUnrealToGDTFAttribute(UnrealFunction.Attribute.Name).ToString();
			const FString ChannelFunctionName = UnrealFunction.FunctionName;

			// The initial function has to be written in following format "GeometryName_LogicalChannelAttribute.ChannelFunctionAttribute.ChannelFunctionName"
			DMXChannel->InitialFunction = FString::Printf(TEXT("%s_%s.%s.%s"), *ControlledGeometryName, *GDTFAttribute, *GDTFAttribute, *ChannelFunctionName);
			DMXChannel->Geometry = *ControlledGeometryName;
			DMXChannel->Offset = [UnrealFunction]()
				{			
					const int32 Offset = UnrealFunction.Channel;
					const uint8 Size = UnrealFunction.GetNumChannels();

					TArray<uint32> Offsets;
					for (int32 ByteOffset = Offset; ByteOffset < Offset + Size; ByteOffset++)
					{
						Offsets.Add(ByteOffset);
					}
					const bool bUseLSBMode = UnrealFunction.bUseLSBMode;
					Algo::Sort(Offsets, [bUseLSBMode](uint32 OffsetA, uint32 OffsetB)
						{
							return bUseLSBMode ? OffsetA >= OffsetB : OffsetA <= OffsetB;
						});

					return Offsets;
				}();

			CreateLogicalChannel(UnrealFunction, DMXChannel, GDTFAttribute);
		}

		// Create DMX Channels for Unreal Matrix Cells if this is a matrix mode
		if (UnrealMode.bFixtureMatrixEnabled)
		{
			const int32 MatrixStartingChannel = UnrealMode.FixtureMatrixConfig.FirstCellChannel;

			int32 Offset = UnrealMode.FixtureMatrixConfig.FirstCellChannel;
			for (const FDMXFixtureCellAttribute& UnrealCellAttribute : UnrealMode.FixtureMatrixConfig.CellAttributes)
			{
				const TSharedRef<FDMXGDTFDMXChannel> DMXChannel = MakeShared<FDMXGDTFDMXChannel>(GDTFDMXMode);
				GDTFDMXMode->DMXChannels.Add(DMXChannel);

				const FString MatrixBeamGeometryName = *FDMXFixtureTypeToGDTFGeometryFactory::MatrixBeamGeometryName.ToString();
				const FString GDTFAttribute = FDMXUnrealToGDTFAttributeConversion::ConvertUnrealToGDTFAttribute(UnrealCellAttribute.Attribute.Name).ToString();

				// For a matrix with geometry references, the initial function has to be written in following format "GeometryName_LogicalChannelAttribute.ChannelFunctionAttribute.ChannelFunctionName"
				DMXChannel->InitialFunction = FString::Printf(TEXT("%s_%s.%s.%s"), *MatrixBeamGeometryName, *GDTFAttribute, *GDTFAttribute, *UnrealCellAttribute.Attribute.Name.ToString());
				DMXChannel->Geometry = *MatrixBeamGeometryName;
				DMXChannel->Offset = [&Offset, UnrealCellAttribute]()
					{					
						const uint8 Size = UnrealCellAttribute.GetNumChannels();

						TArray<uint32> Offsets;
						for (int32 ByteOffset = Offset; ByteOffset < Offset + Size; ByteOffset++)
						{
							Offsets.Add(ByteOffset);
						}
						const bool bUseLSBMode = UnrealCellAttribute.bUseLSBMode;
						Algo::Sort(Offsets, [bUseLSBMode](uint32 OffsetA, uint32 OffsetB)
							{
								return bUseLSBMode ? OffsetA >= OffsetB : OffsetA <= OffsetB;
							});

						Offset += Size;

						return Offsets;
					}();

				// Using a negative value to express special value "Overwrite"
				DMXChannel->DMXBreak = -1;

				CreateLogicalChannel(UnrealCellAttribute, DMXChannel, GDTFAttribute);
			}
		}

		// Sort by Offset
		Algo::SortBy(GDTFDMXMode->DMXChannels, [](const TSharedPtr<FDMXGDTFDMXChannel>& DMXChannel)
			{
				const uint32* MinElementPtr = Algo::MinElementBy(DMXChannel->Offset, [](int32 Element)
					{
						return Element;
					});

				return MinElementPtr ? *MinElementPtr : 0;
			});
	}

	void FDMXFixtureTypeToGDTFConverter::CreateLogicalChannel(const FDMXFixtureFunction& UnrealFunction, const TSharedRef<FDMXGDTFDMXChannel>& GDTFDMXChannel, const FString& GDTFAttribute)
	{
		const TSharedRef<FDMXGDTFLogicalChannel> LogicalChannel = MakeShared<FDMXGDTFLogicalChannel>(GDTFDMXChannel);
		GDTFDMXChannel->LogicalChannelArray.Add(LogicalChannel);

		LogicalChannel->Attribute = *GDTFAttribute;
	
		CrateChannelFunction(UnrealFunction, LogicalChannel, GDTFAttribute);
	}

	void FDMXFixtureTypeToGDTFConverter::CreateLogicalChannel(const FDMXFixtureCellAttribute& UnrealCellAttribute, const TSharedRef<FDMXGDTFDMXChannel>& GDTFDMXChannel, const FString& GDTFAttribute)
	{
		const TSharedRef<FDMXGDTFLogicalChannel> LogicalChannel = MakeShared<FDMXGDTFLogicalChannel>(GDTFDMXChannel);
		GDTFDMXChannel->LogicalChannelArray.Add(LogicalChannel);

		LogicalChannel->Attribute = *GDTFAttribute;

		CrateChannelFunction(UnrealCellAttribute, LogicalChannel, GDTFAttribute);
	}

	void FDMXFixtureTypeToGDTFConverter::CrateChannelFunction(const FDMXFixtureFunction& UnrealFunction, const TSharedRef<FDMXGDTFLogicalChannel>& GDTFLogicalChannel, const FString& GDTFAttribute)
	{
		const TSharedRef<FDMXGDTFChannelFunction> ChannelFunction = MakeShared<FDMXGDTFChannelFunction>(GDTFLogicalChannel);
		GDTFLogicalChannel->ChannelFunctionArray.Add(ChannelFunction);

		ChannelFunction->Name = *UnrealFunction.FunctionName;
		ChannelFunction->Attribute = GDTFAttribute;

		const FDMXGDTFDMXValue Default = FDMXGDTFDMXValue(UnrealFunction.DefaultValue, UnrealFunction.GetNumChannels());
		ChannelFunction->Default = Default;
		ChannelFunction->DMXFrom = 0;

		ChannelFunction->PhysicalFrom = UnrealFunction.GetPhysicalFrom();
		ChannelFunction->PhysicalTo = UnrealFunction.GetPhysicalTo();
	}

	void FDMXFixtureTypeToGDTFConverter::CrateChannelFunction(const FDMXFixtureCellAttribute& UnrealCellAttribute, const TSharedRef<FDMXGDTFLogicalChannel>& GDTFLogicalChannel, const FString& GDTFAttribute)
	{
		const TSharedRef<FDMXGDTFChannelFunction> ChannelFunction = MakeShared<FDMXGDTFChannelFunction>(GDTFLogicalChannel);
		GDTFLogicalChannel->ChannelFunctionArray.Add(ChannelFunction);

		ChannelFunction->Name = *(UnrealCellAttribute.Attribute.Name.ToString());
		ChannelFunction->Attribute = GDTFAttribute;
		ChannelFunction->Default = 0;
		ChannelFunction->DMXFrom = 0;
	}
}
