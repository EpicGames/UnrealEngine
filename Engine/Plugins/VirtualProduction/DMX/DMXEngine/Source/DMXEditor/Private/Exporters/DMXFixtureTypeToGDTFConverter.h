// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

struct FDMXFixtureFunction;
struct FDMXFixtureCellAttribute;
struct FDMXFixtureMode;
class FXmlFile;
class UDMXEntityFixtureType;

namespace UE::DMX::GDTF
{
	struct FDMXFixtureFunctionWithControlledGeometry;
	struct FDMXFixtureModeWithBaseGeometry;
	class FDMXGDTFAttributeDefinitions;
	class FDMXGDTFDMXChannel;
	class FDMXGDTFDMXMode;
	class FDMXGDTFFixtureType;
	class FDMXGDTFGeometry;
	class FDMXGDTFGeometryCollect;
	class FDMXGDTFGeometryReference;
	class FDMXGDTFLogicalChannel;

	/** Converts a Fixture Type to a GDTF. Internally caches of each collect, and finally assembles the GDTF. */
	class FDMXFixtureTypeToGDTFConverter
	{
	public:
		/** Converts the Fixture Type to a GDTF description */
		static TSharedPtr<FXmlFile> Convert(const UDMXEntityFixtureType* UnrealFixtureType);

	private:
		/** Creates the GDTF Fixture Type */
		TSharedRef<FDMXGDTFFixtureType> CreateFixtureType(const UDMXEntityFixtureType& UnrealFixtureType);

		/** Creates Attribute Aefinitions */
		void CreateAttributeDefinitions(const UDMXEntityFixtureType& UnrealFixtureType, const TSharedRef<FDMXGDTFFixtureType>& GDTFFixtureType);

		/** Creates Models */
		void CreateModels(const UDMXEntityFixtureType& UnrealFixtureType, const TSharedRef<FDMXGDTFFixtureType>& GDTFFixtureType);

		/** Creates the Geometry Collect */
		void CreateGeometryCollect(const UDMXEntityFixtureType& UnrealFixtureType, const TSharedRef<FDMXGDTFFixtureType>& GDTFFixtureType);

		/** Creates DMX Modes */
		void CreateDMXModes(const UDMXEntityFixtureType& UnrealFixtureType, const TSharedRef<FDMXGDTFFixtureType>& GDTFFixtureType);

		/** Creates DMX Channels for specified mode */
		void CreateDMXChannels(const FDMXFixtureMode& UnrealMode, const TSharedRef<FDMXGDTFDMXMode>& GDTFDMXMode);

		/** Creates a Logical Channel for the DMX Channel, using an Unreal Function for as input */
		void CreateLogicalChannel(const FDMXFixtureFunction& UnrealFunction, const TSharedRef<FDMXGDTFDMXChannel>& GDTFDMXChannel, const FString& GDTFAttribute);

		/** Creates a Logical Channel for the DMX Channel, using an Unreal Cell Attribute as the input */
		void CreateLogicalChannel(const FDMXFixtureCellAttribute& UnrealCellAttribute, const TSharedRef<FDMXGDTFDMXChannel>& GDTFDMXChannel, const FString& GDTFAttribute);

		/** Creates a Channel Function for the Logical Channel, using an Unreal Function for as input */
		void CrateChannelFunction(const FDMXFixtureFunction& UnrealFunction, const TSharedRef<FDMXGDTFLogicalChannel>& GDTFLogicalChannel, const FString& GDTFAttribute);

		/** Creates a Channel Function for the Logical Channel, using an Unreal Cell Attribute as the input */
		void CrateChannelFunction(const FDMXFixtureCellAttribute& UnrealCellAttribute, const TSharedRef<FDMXGDTFLogicalChannel>& GDTFLogicalChannel, const FString& GDTFAttribute);

		/** Map of Modes with their base geometry */
		TArray<FDMXFixtureModeWithBaseGeometry> ModesWithBaseGeometry;

		/** Map of Functions with the geometry they control */
		TArray<FDMXFixtureFunctionWithControlledGeometry> FunctionsWithControlledGeometry;
	};
}
