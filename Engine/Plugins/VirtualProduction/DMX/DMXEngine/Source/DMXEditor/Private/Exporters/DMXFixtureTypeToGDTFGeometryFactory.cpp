// Copyright Epic Games, Inc. All Rights Reserved.

#include "Exporters/DMXFixtureTypeToGDTFGeometryFactory.h"

#include "Algo/AnyOf.h"
#include "GDTF/Geometries/DMXGDTFAxisGeometry.h"
#include "GDTF/Geometries/DMXGDTFGeometry.h"
#include "GDTF/Geometries/DMXGDTFGeometryBreak.h"
#include "GDTF/Geometries/DMXGDTFGeometryCollect.h"
#include "GDTF/Geometries/DMXGDTFGeometryReference.h"
#include "GDTF/Geometries/DMXGDTFBeamGeometry.h"
#include "Library/DMXEntityFixtureType.h"

namespace UE::DMX::GDTF
{
	const FName FDMXFixtureTypeToGDTFGeometryFactory::CellsModelName = "Cells";

	const FName FDMXFixtureTypeToGDTFGeometryFactory::BaseGeometryName = "Base";
	const FName FDMXFixtureTypeToGDTFGeometryFactory::PanGeometryName = "Yoke";
	const FName FDMXFixtureTypeToGDTFGeometryFactory::HeadGeometryName = "Head";
	const FName FDMXFixtureTypeToGDTFGeometryFactory::BeamGeometryName = "Beam";
	const FName FDMXFixtureTypeToGDTFGeometryFactory::MatrixBeamGeometryName = "Instance";

	const FName FDMXFixtureTypeToGDTFGeometryFactory::PanAttributeName = "Pan";
	const FName FDMXFixtureTypeToGDTFGeometryFactory::TiltAttributeName = "Tilt";

	namespace Internal
	{
		FDMXUniqueModeGeometry::FDMXUniqueModeGeometry(const FDMXFixtureMode& Mode)
		{
			bIsMatrix = Mode.bFixtureMatrixEnabled;

			TotalNumChannels = bIsMatrix ? Mode.FixtureMatrixConfig.GetNumChannels() : 0;
			for (const FDMXFixtureFunction& Function : Mode.Functions)
			{
				TotalNumChannels += Function.GetNumChannels();
			}

			bWithPan = Algo::AnyOf(Mode.Functions, [](const FDMXFixtureFunction& Function) { return Function.Attribute.Name == FDMXFixtureTypeToGDTFGeometryFactory::PanAttributeName; });
			bWithTilt = Algo::AnyOf(Mode.Functions, [](const FDMXFixtureFunction& Function) { return Function.Attribute.Name == FDMXFixtureTypeToGDTFGeometryFactory::TiltAttributeName; });
		}
	}

	FDMXFixtureModeWithBaseGeometry::FDMXFixtureModeWithBaseGeometry(const FDMXFixtureMode* InModePtr, TSharedRef<const FDMXGDTFGeometry> InBaseGeometry)
		: ModePtr(InModePtr)
		, BaseGeometry(InBaseGeometry)
	{}

	FDMXFixtureFunctionWithControlledGeometry::FDMXFixtureFunctionWithControlledGeometry(const FDMXFixtureMode* InModePtr, const FDMXFixtureFunction* InFunctionPtr, TSharedRef<const FDMXGDTFGeometry> InControlledGeometry)
		: ModePtr(InModePtr)
		, FunctionPtr(InFunctionPtr)
		, ControlledGeometry(InControlledGeometry)
	{}

	FDMXFixtureTypeToGDTFGeometryFactory::FDMXFixtureTypeToGDTFGeometryFactory(const UDMXEntityFixtureType& InFixtureType, const TSharedRef<FDMXGDTFGeometryCollect>& InBaseGeometryCollect)
		: FixtureType(InFixtureType)
		, BaseGeometryCollect(InBaseGeometryCollect)
	{
		BuildGeometries();
	}

	TArray<FDMXFixtureModeWithBaseGeometry> FDMXFixtureTypeToGDTFGeometryFactory::GetModesWithBaseGeometry() const
	{
		TArray<FDMXFixtureModeWithBaseGeometry> Result;

		for (const FDMXFixtureMode& Mode : FixtureType.Modes)
		{
			const FDMXUniqueModeGeometry UniqueModeGeometry(Mode);
			const TSharedRef<FDMXGDTFGeometry>* BaseGeometryPtr = UniqueModeGeometryToGeometryMap.Find(UniqueModeGeometry);
			if (!ensureMsgf(BaseGeometryPtr, TEXT("Unexpected cannot find Geometry for Mode %s."), *Mode.ModeName))
			{
				return Result;
			}

			Result.Add(FDMXFixtureModeWithBaseGeometry(&Mode, *BaseGeometryPtr));
		}

		return Result;
	}

	TArray<FDMXFixtureFunctionWithControlledGeometry> FDMXFixtureTypeToGDTFGeometryFactory::GetFunctionsWithControlledGeometry() const
	{
		TArray<FDMXFixtureFunctionWithControlledGeometry> Result;

		for (const FDMXFixtureMode& Mode : FixtureType.Modes)
		{
			const FDMXUniqueModeGeometry UniqueModeGeometry(Mode);
			const TSharedRef<FDMXGDTFGeometry>* BaseGeometryPtr = UniqueModeGeometryToGeometryMap.Find(UniqueModeGeometry);
			if (!ensureMsgf(BaseGeometryPtr, TEXT("Unexpected cannot find Geometry for Mode %s."), *Mode.ModeName))
			{
				return Result;
			}

			const TSharedRef<FDMXGDTFGeometry> BaseGeometry = *BaseGeometryPtr;

			const bool bWithPrimaryAxis = UniqueModeGeometry.bWithPan || UniqueModeGeometry.bWithTilt;
			const TSharedPtr<FDMXGDTFGeometry> PrimaryAxisGeometry = bWithPrimaryAxis ? BaseGeometry->AxisArray[0] : nullptr;

			const bool bWithSecondaryAxisGeometry = UniqueModeGeometry.bWithPan && UniqueModeGeometry.bWithTilt && PrimaryAxisGeometry.IsValid() && !PrimaryAxisGeometry->AxisArray.IsEmpty();
			const TSharedPtr<FDMXGDTFGeometry> SecondaryAxisGeometry = bWithSecondaryAxisGeometry ? PrimaryAxisGeometry->AxisArray[0] : nullptr;
			
			const TSharedPtr<FDMXGDTFGeometry> BeamGeometry =
				bWithSecondaryAxisGeometry && !SecondaryAxisGeometry->BeamArray.IsEmpty() ? SecondaryAxisGeometry->BeamArray[0] :
				bWithPrimaryAxis && !PrimaryAxisGeometry->BeamArray.IsEmpty() ? PrimaryAxisGeometry->BeamArray[0] :
				!BaseGeometry->BeamArray.IsEmpty() ? BaseGeometry->BeamArray[0] :
				nullptr;

			Algo::Transform(Mode.Functions, Result,
				[&Mode, &UniqueModeGeometry, &BaseGeometry, &PrimaryAxisGeometry, &SecondaryAxisGeometry, &BeamGeometry](const FDMXFixtureFunction& Function)
				{
					// Return axis geometries for pan and tilt
					const bool bPanFunction = Function.Attribute == PanAttributeName;
					const bool bTiltFunction = Function.Attribute == TiltAttributeName;
					if (bPanFunction || bTiltFunction)
					{
						const bool bOnlyOneAxis = UniqueModeGeometry.bWithPan != UniqueModeGeometry.bWithTilt;
						if (PrimaryAxisGeometry.IsValid() && (bOnlyOneAxis || bPanFunction))
						{
							return FDMXFixtureFunctionWithControlledGeometry(&Mode, &Function, PrimaryAxisGeometry.ToSharedRef());
						}
						else if (SecondaryAxisGeometry.IsValid())
						{
							return FDMXFixtureFunctionWithControlledGeometry(&Mode, &Function, SecondaryAxisGeometry.ToSharedRef());
						}

						ensureMsgf(0, TEXT("Unexpected could not find axis geometry for Pan or Tilt"));
						return FDMXFixtureFunctionWithControlledGeometry(&Mode, &Function, BaseGeometry);
					}

					// Other attributes control the beam if available
					if (BeamGeometry.IsValid())
					{
						return FDMXFixtureFunctionWithControlledGeometry(&Mode, &Function, BeamGeometry.ToSharedRef());
					}
					else
					{
						return FDMXFixtureFunctionWithControlledGeometry(&Mode, &Function, BaseGeometry);
					}
				});
		}

		return Result;
	}

	void FDMXFixtureTypeToGDTFGeometryFactory::BuildGeometries()
	{
		for (const FDMXFixtureMode& Mode : FixtureType.Modes)
		{
			BuildGeometry(Mode);
		}
	}

	void FDMXFixtureTypeToGDTFGeometryFactory::BuildGeometry(const FDMXFixtureMode& Mode)
	{
		const FDMXUniqueModeGeometry UniqueModeGeometry(Mode);
		const TSharedRef<FDMXGDTFGeometry>* BasePtr = UniqueModeGeometryToGeometryMap.Find(UniqueModeGeometry);

		if (!BasePtr)
		{
			const TSharedRef<FDMXGDTFGeometry> Base = MakeShared<FDMXGDTFGeometry>(BaseGeometryCollect);
			BasePtr = &Base;
			Base->Name = BaseGeometryName;
			BaseGeometryCollect->GeometryArray.Add(Base);

			TSharedPtr<FDMXGDTFGeometry> Head;
			if (UniqueModeGeometry.bWithPan && UniqueModeGeometry.bWithTilt)
			{
				const TSharedRef<FDMXGDTFGeometry> Yoke = AddYoke(Base);
				Head = AddHead(Yoke);
			}
			else if (UniqueModeGeometry.bWithPan || UniqueModeGeometry.bWithTilt)
			{
				Head = AddHead(Base);
			}
			else
			{
				Head = Base;
			}
			check(Head.IsValid());

			if (UniqueModeGeometry.bIsMatrix)
			{
				BuildMatrixGeometryReferences(Mode, Head.ToSharedRef());
			}
			else
			{
				AddBeam(Head.ToSharedRef());
			}

			UniqueModeGeometryToGeometryMap.Add(UniqueModeGeometry, Base);
		}
	}

	TSharedRef<FDMXGDTFGeometry> FDMXFixtureTypeToGDTFGeometryFactory::GetOrCreateMatrixBeamGeometryInstance() const
	{
		if (BaseGeometryCollect->BeamArray.IsEmpty())
		{
			const TSharedRef<FDMXGDTFBeamGeometry> BeamGeometry = MakeShared<FDMXGDTFBeamGeometry>(BaseGeometryCollect);
			BaseGeometryCollect->BeamArray.Add(BeamGeometry);

			BeamGeometry->Name = MatrixBeamGeometryName;
		}

		return BaseGeometryCollect->BeamArray.Last().ToSharedRef();
	}

	TSharedRef<FDMXGDTFAxisGeometry> FDMXFixtureTypeToGDTFGeometryFactory::AddYoke(const TSharedRef<FDMXGDTFGeometry>& OuterGeometry)
	{
		const TSharedRef<FDMXGDTFAxisGeometry> Yoke = MakeShared<FDMXGDTFAxisGeometry>(OuterGeometry);
		OuterGeometry->AxisArray.Add(Yoke);

		Yoke->Name = PanGeometryName;

		return Yoke;
	}

	TSharedRef<FDMXGDTFAxisGeometry> FDMXFixtureTypeToGDTFGeometryFactory::AddHead(const TSharedRef<FDMXGDTFGeometry>& OuterGeometry)
	{
		const TSharedRef<FDMXGDTFAxisGeometry> Head = MakeShared<FDMXGDTFAxisGeometry>(OuterGeometry);
		OuterGeometry->AxisArray.Add(Head);

		Head->Name = HeadGeometryName;

		return Head;
	}

	TSharedRef<FDMXGDTFBeamGeometry> FDMXFixtureTypeToGDTFGeometryFactory::AddBeam(const TSharedRef<FDMXGDTFGeometry>& OuterGeometry)
	{
		const TSharedRef<FDMXGDTFBeamGeometry> Beam = MakeShared<FDMXGDTFBeamGeometry>(OuterGeometry);
		OuterGeometry->BeamArray.Add(Beam);
		
		Beam->Name = BeamGeometryName;

		return Beam;
	}

	void FDMXFixtureTypeToGDTFGeometryFactory::BuildMatrixGeometryReferences(const FDMXFixtureMode& Mode, const TSharedRef<FDMXGDTFGeometry>& OuterGeometry)
	{
		if (Mode.bFixtureMatrixEnabled && Mode.FixtureMatrixConfig.GetNumChannels() > 0)
		{
			// Get the byte size of an Unreal Matrix Cell
			const int32 CellSize = [Mode]()
			{
				int32 OutCellSize = 0;
				for (const FDMXFixtureCellAttribute& CellAttribute : Mode.FixtureMatrixConfig.CellAttributes)
				{
					OutCellSize += CellAttribute.GetNumChannels();
				}

				return OutCellSize;
			}();

			// Create Geometry References for each Unreal Matrix Cell
			const int32 NumCells = Mode.FixtureMatrixConfig.XCells * Mode.FixtureMatrixConfig.YCells;
			int32 Offset = 1;
			for (int32 CellID = 0; CellID < NumCells; CellID++)
			{
				const int32 DMXOffset = CellID * CellSize + 1;

				const int32 Row = CellID / Mode.FixtureMatrixConfig.YCells + 1;
				const int32 Column = CellID % Mode.FixtureMatrixConfig.YCells + 1;
				const FName GeometryReferenceName = *FString::Printf(TEXT("Cell_%i_%i"), Row, Column);

				const TSharedRef<FDMXGDTFGeometryReference> GeometryReference = MakeShared<FDMXGDTFGeometryReference>(OuterGeometry);
				OuterGeometry->GeometryReferenceArray.Add(GeometryReference);

				GeometryReference->Name = GeometryReferenceName;
				GeometryReference->Geometry = GetOrCreateMatrixBeamGeometryInstance()->Name;
				GeometryReference->Model = CellsModelName;

				const TSharedRef<FDMXGDTFGeometryBreak> Break = MakeShared<FDMXGDTFGeometryBreak>(GeometryReference);
				Break->DMXBreak = 1;
				Break->DMXOffset = Offset;
				GeometryReference->BreakArray.Add(Break);

				Break->DMXBreak = 1; // Unreal does not support multi universe patches, DMXBreak is always 1
				Break->DMXOffset = DMXOffset;

				Offset += CellSize;
			}
		}
	}
}
