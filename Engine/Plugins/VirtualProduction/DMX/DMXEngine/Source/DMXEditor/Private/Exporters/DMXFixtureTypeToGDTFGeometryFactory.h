// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

struct FDMXFixtureFunction;
struct FDMXFixtureMode;
class UDMXEntityFixtureType;

namespace UE::DMX::GDTF
{
	class FDMXGDTFAxisGeometry;
	class FDMXGDTFBeamGeometry;
	class FDMXGDTFGeometry;
	class FDMXGDTFGeometryCollect;
	class FDMXGDTFGeometryReference;

	namespace Internal
	{
		struct FDMXUniqueModeGeometry
		{
			FDMXUniqueModeGeometry(const FDMXFixtureMode& Mode);

			uint32 TotalNumChannels = 0; 

			bool bIsMatrix = false;
			bool bWithPan = false;
			bool bWithTilt = false;

			bool operator==(const FDMXUniqueModeGeometry& Other) const 
			{ 
				return
					TotalNumChannels == Other.TotalNumChannels &&
					bIsMatrix == Other.bIsMatrix &&
					bWithPan == Other.bWithPan &&
					bWithTilt == Other.bWithTilt;
			}

			friend uint32 GetTypeHash(const FDMXUniqueModeGeometry& UniqueMatrixDefinition)
			{
				return UniqueMatrixDefinition.TotalNumChannels |
					static_cast<uint32>(UniqueMatrixDefinition.bWithPan) << 31 |
					static_cast<uint32>(UniqueMatrixDefinition.bWithTilt << 30) |
					static_cast<uint32>(UniqueMatrixDefinition.bWithTilt << 29);
			}
		};
	}

	/** A struct that links a mode to its base geometry */
	struct FDMXFixtureModeWithBaseGeometry
	{
		FDMXFixtureModeWithBaseGeometry(const FDMXFixtureMode* InModePtr, TSharedRef<const FDMXGDTFGeometry> InBaseGeometry);

		const FDMXFixtureMode* ModePtr = nullptr;
		const TSharedRef<const FDMXGDTFGeometry> BaseGeometry;
	};

	/** A struct that links a function to its controlled geometry */
	struct FDMXFixtureFunctionWithControlledGeometry
	{
		FDMXFixtureFunctionWithControlledGeometry(const FDMXFixtureMode* InModePtr, const FDMXFixtureFunction* InFunctionPtr, TSharedRef<const FDMXGDTFGeometry> InControlledGeometry);

		const FDMXFixtureMode* ModePtr = nullptr;
		const FDMXFixtureFunction* FunctionPtr = nullptr;
		const TSharedRef<const FDMXGDTFGeometry> ControlledGeometry;
	};

	/** Helper to build geometries for the Fixture Type */
	class FDMXFixtureTypeToGDTFGeometryFactory
		: public FNoncopyable
	{
		using FDMXUniqueModeGeometry = Internal::FDMXUniqueModeGeometry;

	public:
		FDMXFixtureTypeToGDTFGeometryFactory(const UDMXEntityFixtureType& InFixtureType, const TSharedRef<FDMXGDTFGeometryCollect>& InBaseGeometryCollect);

		/** Returns an array of functions with their linked geometry */
		TArray<FDMXFixtureModeWithBaseGeometry> GetModesWithBaseGeometry() const;

		/** Returns an array of functions with their linked geometry */
		TArray<FDMXFixtureFunctionWithControlledGeometry> GetFunctionsWithControlledGeometry() const;

		static const FName CellsModelName;

		static const FName BaseGeometryName;
		static const FName PanGeometryName;
		static const FName HeadGeometryName;
		static const FName BeamGeometryName;
		static const FName MatrixBeamGeometryName;

		static const FName PanAttributeName;
		static const FName TiltAttributeName;

	private:
		/** Builds all geometries */
		void BuildGeometries();

		/** Builds the geometry for a mode */
		void BuildGeometry(const FDMXFixtureMode& Mode);

		/** Gets the matrix beam instance, creates it if it doesn't exist yet */
		TSharedRef<FDMXGDTFGeometry> GetOrCreateMatrixBeamGeometryInstance() const;

		/** Adds a yoke geometry child to the outer geometry */
		TSharedRef<FDMXGDTFAxisGeometry> AddYoke(const TSharedRef<FDMXGDTFGeometry>& OuterGeometry);

		/** Adds a head geometry child to the outer geometry */
		TSharedRef<FDMXGDTFAxisGeometry> AddHead(const TSharedRef<FDMXGDTFGeometry>& OuterGeometry);

		/** Adds a beam geometry child to the outer geometry */
		TSharedRef<FDMXGDTFBeamGeometry> AddBeam(const TSharedRef<FDMXGDTFGeometry>& OuterGeometry);

		/** Builds geometry references for a matrix */
		void BuildMatrixGeometryReferences(const FDMXFixtureMode& Mode, const TSharedRef<FDMXGDTFGeometry>& OuterGeometry);

		/** A map unique matrix definitions with teir base geometry, to reuse. */
		TMap<FDMXUniqueModeGeometry, TSharedRef<FDMXGDTFGeometry>> UniqueModeGeometryToGeometryMap;

		const UDMXEntityFixtureType& FixtureType;
		const TSharedRef<FDMXGDTFGeometryCollect>& BaseGeometryCollect;
	};
}