// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "DMXMVRExportOptions.generated.h"

UCLASS(Config = DMXEditor)
class UDMXMVRExportOptions
	: public UObject
{
	GENERATED_BODY()

public:
	/** If checked, exports the fixtures with transforms as in the current level */
	UPROPERTY(EditAnywhere, Category = "Level Options")
	bool bConsiderLevel = true;

	/** If checked, exports patches that are not in use in the level */
	UPROPERTY(EditAnywhere, Meta = (EditCondition = "bConsiderLevel", HideEditConditionToggle, EditConditionHides), Category = "Level Options")
	bool bExportPatchesNotPresentInWorld = true;

	/** If checked, exports the fixtures with transforms as in the current level */
	UPROPERTY(EditAnywhere, Meta = (EditCondition = "bConsiderLevel", HideEditConditionToggle, EditConditionHides), Category = "Level Options")
	bool bUseTransformsFromLevel = true;

	/** If checked, creates individual MVR Fixtures for each Fixture that uses the same Patch in the Level. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Meta = (EditCondition = "bConsiderLevel", HideEditConditionToggle, EditConditionHides, DisplayName = "Create Multi Patch Fixtures (experimental)"), Category = "Level Options")
	bool bCreateMultiPatchFixtures = false;

	/** If set to true, the import was canceled */
	UPROPERTY(Transient)
	bool bCanceled = false;
};
