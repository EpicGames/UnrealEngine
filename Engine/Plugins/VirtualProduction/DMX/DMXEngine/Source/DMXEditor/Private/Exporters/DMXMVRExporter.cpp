// Copyright Epic Games, Inc. All Rights Reserved.

#include "Exporters/DMXMVRExporter.h"

#include "DesktopPlatformModule.h"
#include "DMXEditorLog.h"
#include "DMXEditorSettings.h"
#include "DMXFixtureTypeToGDTFConverter.h"
#include "DMXMVRExportOptions.h"
#include "DMXMVRXmlMergeUtility.h"
#include "DMXZipper.h"
#include "Editor.h"
#include "EngineUtils.h"
#include "Factories/DMXGDTFToFixtureTypeConverter.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Game/DMXComponent.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "IDesktopPlatform.h"
#include "Interfaces/IMainFrameModule.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/DMXGDTFAssetImportData.h"
#include "Library/DMXImportGDTF.h"
#include "Library/DMXLibrary.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "MVR/DMXMVRAssetImportData.h"
#include "MVR/DMXMVRGeneralSceneDescription.h"
#include "Subsystems/UnrealEditorSubsystem.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SDMXMVRExportOptions.h"
#include "XmlFile.h"

#define LOCTEXT_NAMESPACE "DMXMVRExporter"

namespace UE::DMX
{
	void FDMXMVRExporter::Export(UDMXLibrary* DMXLibrary, const FString& DesiredName)
	{
		FDMXMVRExporter Instance;
		FText ErrorReason;
		FString FilePathAndName;
		Instance.ExportInternal(DMXLibrary, DesiredName, ErrorReason, FilePathAndName);

		if (ErrorReason.IsEmpty())
		{
			FNotificationInfo NotificationInfo(FText::Format(LOCTEXT("ExportDMXLibraryAsMVRSuccessNotification", "Successfully exported MVR to {0}."), FText::FromString(FilePathAndName)));
			NotificationInfo.ExpireDuration = 5.f;

			FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		}
		else
		{
			FNotificationInfo NotificationInfo(ErrorReason);
			NotificationInfo.ExpireDuration = 10.f;
			NotificationInfo.Image = FAppStyle::GetBrush("Icons.Warning");

			FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		}
	}

	void FDMXMVRExporter::ExportInternal(UDMXLibrary* InDMXLibrary, const FString& InDesiredName, FText& OutErrorReason, FString& OutFilePathAndName)
	{
		if (!ensureAlwaysMsgf(InDMXLibrary, TEXT("Trying to export DMX Library '%s' as MVR file, but the DMX Library is invalid."), *InDMXLibrary->GetName()))
		{
			OutErrorReason = LOCTEXT("MVRExportDMXLibraryInvalidReason", "DMX Library {0} is invalid. Cannot export MVR file.");
			return;
		}

		UpdateExportOptions(*InDMXLibrary);
		if (GetDefault<UDMXMVRExportOptions>()->bCanceled)
		{
			OutErrorReason = LOCTEXT("MVRExportCanceledInvalidReason", "Canceled MVR export.");
			return;
		}

		OutFilePathAndName = [InDMXLibrary, &InDesiredName]()
			{					
				UDMXEditorSettings* DMXEditorSettings = GetMutableDefault<UDMXEditorSettings>();
				const FString LastMVRExportPath = DMXEditorSettings->LastMVRExportPath;
				const FString DefaultPath = FPaths::DirectoryExists(LastMVRExportPath) ? LastMVRExportPath : FPaths::ProjectSavedDir();
				const FString DefaultFileName = InDesiredName.IsEmpty() ?
					InDMXLibrary->GetName() + TEXT(".mvr") :
					FPaths::GetBaseFilename(InDesiredName) + TEXT(".mvr");

				IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
				if (!DesktopPlatform)
				{
					return FString();
				}

				TArray<FString> SaveFilenames;
				const bool bSaveFile = DesktopPlatform->SaveFileDialog(
					FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
					LOCTEXT("ExportMVR", "Export MVR").ToString(),
					DefaultPath,
					DefaultFileName,
					TEXT("My Virtual Rig (*.mvr)|*.mvr"),
					EFileDialogFlags::None,
					SaveFilenames);

				if (!bSaveFile || SaveFilenames.IsEmpty())
				{
					return FString();
				}

				DMXEditorSettings->LastMVRExportPath = FPaths::GetPath(SaveFilenames[0]);
				DMXEditorSettings->SaveConfig();

				return SaveFilenames[0];
			}();

		// Create a copy of the library's general scene description, so transforms exported aren't written to the dmx library
		UDMXMVRGeneralSceneDescription* TempGeneralSceneDescription = DuplicateObject<UDMXMVRGeneralSceneDescription>(InDMXLibrary->GetLazyGeneralSceneDescription(), GetTransientPackage());
		if (!ensureAlwaysMsgf(TempGeneralSceneDescription, TEXT("Trying to export DMX Library '%s' as MVR file, but its General Scene Description is invalid."), *InDMXLibrary->GetName()))
		{
			OutErrorReason = FText::Format(LOCTEXT("MVRExportGeneralSceneDescriptionInvalidReason", "DMX Library is invalid. Cannot export {0}."), FText::FromString(OutFilePathAndName));
			return;
		}		
		
		// Get Export options
		UUnrealEditorSubsystem* UnrealEditorSubsystem = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>();
		const UDMXMVRExportOptions* ExportOptions = GetDefault<UDMXMVRExportOptions>();

		FDMXMVRGeneralSceneDescriptionWorldParams WorldParams;
		WorldParams.World = UnrealEditorSubsystem->GetEditorWorld();
		WorldParams.bCreateMultiPatchFixtures = ExportOptions->bCreateMultiPatchFixtures;
		WorldParams.bExportPatchesNotPresentInWorld = ExportOptions->bExportPatchesNotPresentInWorld;
		WorldParams.bUseTransformsFromLevel = ExportOptions->bUseTransformsFromLevel;

		TempGeneralSceneDescription->WriteDMXLibrary(*InDMXLibrary, WorldParams);

		const TSharedRef<FDMXZipper> Zip = MakeShared<FDMXZipper>();
		if (!ZipGeneralSceneDescription(Zip, TempGeneralSceneDescription, OutErrorReason))
		{
			return;
		}

		if (!ZipGDTFs(Zip, InDMXLibrary))
		{
			OutErrorReason = FText::Format(LOCTEXT("MVRExportZipGDTFsFailedReason", "Some Fixture Types could not be converted to GDTF. Exported MVR to {0}."), FText::FromString(OutFilePathAndName));
			// Allow continuation of export
		}

		ZipThirdPartyData(Zip, TempGeneralSceneDescription);
		if (!Zip->SaveToFile(OutFilePathAndName))
		{
			OutErrorReason = FText::Format(LOCTEXT("MVRExportWriteZipFailedReason", "File is not writable or locked by another process. Cannot export {0}."), FText::FromString(OutFilePathAndName));
			return;
		}
	}

	void FDMXMVRExporter::UpdateExportOptions(const UDMXLibrary& DMXLibrary) const
	{
		UUnrealEditorSubsystem* UnrealEditorSubsystem = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>();
		const bool bHasEditorWorld = UnrealEditorSubsystem && UnrealEditorSubsystem->GetEditorWorld();

		// No need to show options when there is no world, as of now the options are related to the current level only
		if (!bHasEditorWorld)
		{
			return;
		}

		TSharedPtr<SWindow> ParentWindow;
		if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
		{
			IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
			ParentWindow = MainFrame.GetParentWindow();
		}

		const FVector2D WindowSize = FVector2D(512.f, 338.f);

		const FSlateRect WorkAreaRect = FSlateApplicationBase::Get().GetPreferredWorkArea();
		const FVector2D DisplayTopLeft(WorkAreaRect.Left, WorkAreaRect.Top);
		const FVector2D DisplaySize(WorkAreaRect.Right - WorkAreaRect.Left, WorkAreaRect.Bottom - WorkAreaRect.Top);

		const FVector2D WindowPosition = (DisplayTopLeft + (DisplaySize - WindowSize) / 2.0f);

		const FText Caption = LOCTEXT("ExportWindowCaption", "MVR Export Options");

		const TSharedRef<SWindow> Window = SNew(SWindow)
			.Title(Caption)
			.SizingRule(ESizingRule::FixedSize)
			.AutoCenter(EAutoCenter::None)
			.ClientSize(WindowSize)
			.ScreenPosition(WindowPosition);

		Window->SetContent
		(
			SNew(SDMXMVRExportOptions, Window)
		);

		constexpr bool bSlowTaskWindow = false;
		FSlateApplication::Get().AddModalWindow(Window, ParentWindow, bSlowTaskWindow);
	}

	TMap<const UDMXComponent*, const AActor*> FDMXMVRExporter::GetDMXComponentToActorMap() const
	{
		TMap<const UDMXComponent*, const AActor*> DMXComponentToActorMap;

		UUnrealEditorSubsystem* UnrealEditorSubsystem = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>();
		const UWorld* EditorWorld = UnrealEditorSubsystem ? UnrealEditorSubsystem->GetEditorWorld() : nullptr;
		if (!EditorWorld)
		{
			return DMXComponentToActorMap;
		}

		// Find actors with a patched DMX component
		for (TActorIterator<AActor> It(EditorWorld, AActor::StaticClass()); It; ++It)
		{
			const AActor* Actor = *It;
			if (!Actor)
			{
				continue;
			}

			const TSet<UActorComponent*> Components = Actor->GetComponents();

			TArray<const UDMXComponent*> DMXComponents;
			Algo::TransformIf(Components, DMXComponents,
				[](const UActorComponent* Component)
				{
					return Component &&
						Component->IsA(UDMXComponent::StaticClass()) &&
						CastChecked<UDMXComponent>(Component)->GetFixturePatch();
				},
				[](const UActorComponent* Component)
				{
					return CastChecked<UDMXComponent>(Component);
				});

			for (const UDMXComponent* DMXComponent : DMXComponents)
			{
				DMXComponentToActorMap.Add(TTuple<const UDMXComponent*, const AActor*>(DMXComponent, Actor));
			}
		}

		return DMXComponentToActorMap;
	}

	bool FDMXMVRExporter::ZipGeneralSceneDescription(const TSharedRef<FDMXZipper>& Zip, const UDMXMVRGeneralSceneDescription* GeneralSceneDescription, FText& OutErrorReason)
	{
		if (!GeneralSceneDescription->CanCreateXmlFile(OutErrorReason))
		{
			return false;
		}

		TSharedPtr<FXmlFile> XmlFile = GeneralSceneDescription->CreateXmlFile();
		if (!ensureMsgf(XmlFile.IsValid(), TEXT("General Scene Descriptions returns CanCreateXmlFile() as true, but CreateXmlFile() fails.")))
		{
			return false;
		}

		// Try to merge the source General Scene Description Xml
		const TSharedPtr<FXmlFile> SourceXmlFile = CreateSourceGeneralSceneDescriptionXmlFile(GeneralSceneDescription);
		if (SourceXmlFile.IsValid())
		{
			// Merge the General Scene Description with source data's xml (source) to retain 3rd party data
			XmlFile = FDMXXmlMergeUtility::Merge(GeneralSceneDescription, SourceXmlFile.ToSharedRef());
		}

		// Create a temp GeneralSceneDescription.xml file
		const FString TempPath = FPaths::ConvertRelativePathToFull(FPaths::EngineSavedDir() / TEXT("DMX_Temp"));
		constexpr TCHAR GeneralSceneDescriptionFileName[] = TEXT("GeneralSceneDescription.xml");
		const FString TempGeneralSceneDescriptionFilePathAndName = TempPath / GeneralSceneDescriptionFileName;
		if (!XmlFile->Save(TempGeneralSceneDescriptionFilePathAndName))
		{
			UE_LOGF(LogDMXEditor, Error, "Failed to save General Scene Description. See previous errors for details.");
			return false;
		}

		TArray64<uint8> GeneralSceneDescriptionData;
		FFileHelper::LoadFileToArray(GeneralSceneDescriptionData, *TempGeneralSceneDescriptionFilePathAndName);
		Zip->AddFile(GeneralSceneDescriptionFileName, GeneralSceneDescriptionData);

		// Delete temp GeneralSceneDescription.xml file
		IFileManager& FileManager = IFileManager::Get();
		constexpr bool bRequireExists = true;
		constexpr bool bEvenIfReadOnly = false;
		constexpr bool bQuiet = true;
		FileManager.Delete(*TempGeneralSceneDescriptionFilePathAndName, bRequireExists, bEvenIfReadOnly, bQuiet);

		return true;
	}

	bool FDMXMVRExporter::ZipGDTFs(const TSharedRef<FDMXZipper>& Zip, UDMXLibrary* DMXLibrary)
	{
		check(DMXLibrary);

		// Gather GDTFs
		TArray<UDMXEntityFixtureType*> FixtureTypesToExport;
		const TArray<UDMXEntityFixturePatch*> FixturePatches = DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>();
		for (UDMXEntityFixturePatch* FixturePatch : FixturePatches)
		{
			UDMXEntityFixtureType* FixtureType = FixturePatch->GetFixtureType();
			if (!FixtureType || FixtureType->Modes.IsEmpty())
			{
				continue;
			}

			FixtureTypesToExport.AddUnique(FixtureType);
		}

		// Zip GDTFs
		const FString TempGDTFDir = FPaths::Combine(FPaths::GameAgnosticSavedDir() / TEXT("GDTFExport"));

		bool bAllZippedSuccessfully = true;
		for (UDMXEntityFixtureType* FixtureType : FixtureTypesToExport)
		{
			if (FixtureType->GDTFSource.IsNull() || FixtureType->bExportGeneratedGDTF)
			{
				constexpr bool bWithExtension = true;
				const FString CleanGDTFFilename = FixtureType->GetCleanGDTFFileNameSynchronous(bWithExtension);

				using namespace UE::DMX::GDTF;
				const TSharedPtr<FXmlFile> XmlFile = FDMXFixtureTypeToGDTFConverter::Convert(FixtureType);
				if (!XmlFile.IsValid())
				{
					UE_LOGF(LogDMXEditor, Warning, "Failed to create GDTF from Fixture Type '%ls'. See previous errors.", *FixtureType->Name);
					bAllZippedSuccessfully = false;
					continue;
				}

				if (!FPaths::DirectoryExists(TempGDTFDir))
				{
					IPlatformFile::GetPlatformPhysical().CreateDirectory(*TempGDTFDir);
				}

				if (!ensureMsgf(FPaths::DirectoryExists(TempGDTFDir), TEXT("Cannot create temporary directory for GDTFs.")))
				{
					continue;
				}

				const FString TempXmlFilename = TempGDTFDir / FixtureType->Name + TEXT(".description.xml");
				XmlFile->Save(TempXmlFilename);

				TArray64<uint8> DescriptionXmlData;
				if (FFileHelper::LoadFileToArray(DescriptionXmlData, *TempXmlFilename))
				{
					FDMXZipper NewGDTFZip;
					NewGDTFZip.AddFile(TEXT("description.xml"), DescriptionXmlData);
					const FString TempGDTFilePathAndName = TempGDTFDir / CleanGDTFFilename;

					TArray64<uint8> GDTFData;
					if (NewGDTFZip.GetData(GDTFData))
					{
						Zip->AddFile(CleanGDTFFilename, GDTFData);
					}
					else
					{
						UE_LOGF(LogDMXEditor, Warning, "Failed to load temporary GDTF for Fixture Type '%ls'.", *FixtureType->Name);
						bAllZippedSuccessfully = false;
						continue;
					}
				}
				else
				{
					UE_LOGF(LogDMXEditor, Warning, "Failed to load temporary description.xml for Fixture Type '%ls'.", *FixtureType->Name);
					bAllZippedSuccessfully = false;
					continue;
				}
			}
			else
			{
				UDMXImportGDTF* DMXImportGDTF = FixtureType->GDTFSource.LoadSynchronous();
				if (!DMXImportGDTF)
				{
					UE_LOGF(LogDMXEditor, Warning, "Cannot export Fixture Type '%ls' to MVR, but the Fixture Type has a DMX Import Type which is not GDTF.", *FixtureType->Name);
					continue;
				}

				UDMXGDTFAssetImportData* GDTFAssetImportData = DMXImportGDTF->GetGDTFAssetImportData();
				if (!ensureAlwaysMsgf(GDTFAssetImportData, TEXT("Missing default GDTF Asset Import Data subobject in GDTF.")))
				{
					continue;
				}

				const TArray64<uint8>& RawSourceData = RefreshSourceDataAndFixtureType(*FixtureType, *GDTFAssetImportData);
				if (!RawSourceData.IsEmpty())
				{
					const FString GDTFFilename = FPaths::GetCleanFilename(GDTFAssetImportData->GetFilePathAndName());
					Zip->AddFile(GDTFFilename, GDTFAssetImportData->GetRawSourceData());
				}
				else
				{
					bAllZippedSuccessfully = false;
					const UDMXImportGDTF* GDTF = Cast<UDMXImportGDTF>(GDTFAssetImportData->GetOuter());
					UE_LOGF(LogDMXEditor, Error, "Cannot export '%ls' to MVR File. The asset is missing source data.", GDTF ? *GDTF->GetName() : TEXT("Invalid GDTF Asset"));
				}
			}
		}

		IPlatformFile::GetPlatformPhysical().DeleteDirectory(*TempGDTFDir);

		return bAllZippedSuccessfully;
	}

	void FDMXMVRExporter::ZipThirdPartyData(const TSharedRef<FDMXZipper>& Zip, const UDMXMVRGeneralSceneDescription* GeneralSceneDescription)
	{
		const UDMXMVRAssetImportData* AssetImportData = GeneralSceneDescription->GetMVRAssetImportData();
		if (!AssetImportData)
		{
			return;
		}

		if (AssetImportData && AssetImportData->GetRawSourceData().Num() > 0)
		{
			const TSharedRef<FDMXZipper> SourceZip = MakeShared<FDMXZipper>();
			if (SourceZip->LoadFromData(AssetImportData->GetRawSourceData()))
			{
				for (const FString& SourceFileName : SourceZip->GetFiles())
				{
					// Don't add GeneralSceneDescription.xml and GDTFs
					constexpr TCHAR GeneralSceneDescriptionFileName[] = TEXT("GeneralSceneDescription.xml");
					constexpr TCHAR GDTFExtension[] = TEXT("gdtf");
					if (SourceFileName.EndsWith(GeneralSceneDescriptionFileName) || FPaths::GetExtension(SourceFileName) == GDTFExtension)
					{
						continue;
					}

					TArray64<uint8> SourceFileData;
					if (SourceZip->GetFileContent(SourceFileName, SourceFileData))
					{
						Zip->AddFile(SourceFileName, SourceFileData);
					}
				}
			}
		}
	}

	const TSharedPtr<FXmlFile> FDMXMVRExporter::CreateSourceGeneralSceneDescriptionXmlFile(const UDMXMVRGeneralSceneDescription* GeneralSceneDescription) const
	{
		const UDMXMVRAssetImportData* AssetImportData = GeneralSceneDescription->GetMVRAssetImportData();
		if (!AssetImportData)
		{
			return nullptr;
		}

		if (AssetImportData->GetRawSourceData().IsEmpty())
		{
			return nullptr;
		}

		const TSharedRef<FDMXZipper> DMXZipper = MakeShared<FDMXZipper>();
		if (!DMXZipper->LoadFromData(AssetImportData->GetRawSourceData()))
		{
			return nullptr;
		}

		constexpr TCHAR GeneralSceneDescriptionFilename[] = TEXT("GeneralSceneDescription.xml");
		FDMXZipper::FDMXScopedUnzipToTempFile UnzipTempFileScope(DMXZipper, GeneralSceneDescriptionFilename);

		if (UnzipTempFileScope.TempFilePathAndName.IsEmpty())
		{
			return nullptr;
		}

		const TSharedRef<FXmlFile> XmlFile = MakeShared<FXmlFile>();
		if (!XmlFile->LoadFile(UnzipTempFileScope.TempFilePathAndName))
		{
			return nullptr;
		}

		return XmlFile;
	}

	const TArray64<uint8>& FDMXMVRExporter::RefreshSourceDataAndFixtureType(UDMXEntityFixtureType& FixtureType, UDMXGDTFAssetImportData& InOutGDTFAssetImportData) const
	{
		const UDMXImportGDTF* GDTF = Cast<UDMXImportGDTF>(InOutGDTFAssetImportData.GetOuter());
		if (!InOutGDTFAssetImportData.GetRawSourceData().IsEmpty() || !GDTF)
		{
			return InOutGDTFAssetImportData.GetRawSourceData();
		}

		static bool bAskForReloadAgain = true;
		static EAppReturnType::Type MessageDialogResult = EAppReturnType::Yes;

		if (bAskForReloadAgain)
		{
			static const FText MessageTitle = LOCTEXT("NoGDTFSourceAvailableTitle", "Trying to use old GDTF asset.");
			static const FText Message = FText::Format(LOCTEXT("NoGDTFSourceAvailableMessage", "Insufficient data to export '{0}' to MVR file. The GDTF asset was created prior to UE5.1. Do you want to reload the source GDTF?"), FText::FromString(GDTF->GetName()));
			MessageDialogResult = FMessageDialog::Open(EAppMsgType::YesNoYesAllNoAll, Message, MessageTitle);

			if (MessageDialogResult == EAppReturnType::YesAll || MessageDialogResult == EAppReturnType::NoAll)
			{
				bAskForReloadAgain = false;
			}
		}

		if (MessageDialogResult == EAppReturnType::YesAll || MessageDialogResult == EAppReturnType::Yes)
		{
			IDesktopPlatform* const DesktopPlatform = FDesktopPlatformModule::Get();

			if (DesktopPlatform)
			{
				UDMXEditorSettings* EditorSettings = GetMutableDefault<UDMXEditorSettings>();
				if (!ensureAlwaysMsgf(EditorSettings, TEXT("Unexpected cannot access DMX Editor Settings CDO.")))
				{
					return InOutGDTFAssetImportData.GetRawSourceData();
				}

				TArray<FString> Filenames;
				if (!InOutGDTFAssetImportData.GetSourceData().SourceFiles.IsEmpty() &&
					FPaths::FileExists(InOutGDTFAssetImportData.GetFilePathAndName()))
				{
					Filenames.Add(InOutGDTFAssetImportData.GetFilePathAndName());
				}
				else
				{
					DesktopPlatform->OpenFileDialog(
						nullptr,
						FText::Format(LOCTEXT("OpenGDTFTitle", "Choose a GDTF file for '%s'."), FText::FromString(GDTF->GetName())).ToString(),
						EditorSettings->LastGDTFImportPath,
						TEXT(""),
						TEXT("General Scene Description (*.gdtf)|*.gdtf"),
						EFileDialogFlags::None,
						Filenames
					);
				}

				if (Filenames.Num() > 0)
				{
					EditorSettings->LastGDTFImportPath = FPaths::GetPath(Filenames[0]);

					InOutGDTFAssetImportData.PreEditChange(nullptr);
					InOutGDTFAssetImportData.SetSourceFile(Filenames[0]);
					InOutGDTFAssetImportData.PostEditChange();

					using namespace UE::DMX::GDTF;
					constexpr bool bUpdateFixtureTypeName = true;
					FDMXGDTFToFixtureTypeConverter::ConvertGDTF(FixtureType, *GDTF, bUpdateFixtureTypeName);

					if (InOutGDTFAssetImportData.GetRawSourceData().Num() == 0)
					{
						FNotificationInfo NotificationInfo(FText::Format(LOCTEXT("ReloadGDTFFailure", "Failed to update GDTF '{0}' from '{1}."), FText::FromString(GDTF->GetName()), FText::FromString(Filenames[0])));
						NotificationInfo.ExpireDuration = 10.f;

						FSlateNotificationManager::Get().AddNotification(NotificationInfo);
					}
				}
			}
		}

		return InOutGDTFAssetImportData.GetRawSourceData();
	}
}

#undef LOCTEXT_NAMESPACE
