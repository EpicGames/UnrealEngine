// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class AActor;
class FDMXZipper;
class FXmlFile;
class UDMXEntityFixtureType;
class UDMXGDTFAssetImportData;
class UDMXComponent;
class UDMXLibrary;
class UDMXMVRAssetImportData;
class UDMXMVRFixtureNode;
class UDMXMVRGeneralSceneDescription;
class UWorld;

namespace UE::DMX
{
	/** Helper class to export a DMX Library as MVR file */
	class FDMXMVRExporter
	{
	public:
		/** Exports the DMX Library as MVR File */
		static void Export(UDMXLibrary* DMXLibrary, const FString& DesiredName = TEXT(""));

	private:
		/** Exports the DMX Library as MVR File. If OutErrorReason is not empty there were issues with the export. */
		void ExportInternal(UDMXLibrary* InDMXLibrary, const FString& InDesiredName, FText& OutErrorReason, FString& OutFilePathAndName);

		/** Updates the MVR export options. Returns false if the import was canceled */
		void UpdateExportOptions(const UDMXLibrary& DMXLibrary) const;
		
		/** Builds the DMX component to actor map */
		TMap<const UDMXComponent*, const AActor*> GetDMXComponentToActorMap() const;

		/** Zips the GeneralSceneDescription.xml */
		[[nodiscard]] bool ZipGeneralSceneDescription(const TSharedRef<FDMXZipper>& Zip, const UDMXMVRGeneralSceneDescription* GeneralSceneDescription, FText& OutErrorReason);

		/** Zips the GDTFs from the Library */
		[[nodiscard]] bool ZipGDTFs(const TSharedRef<FDMXZipper>& Zip, UDMXLibrary* DMXLibrary);

		/** Zips 3rd Party Data from the MVR Asset Import Data */
		void ZipThirdPartyData(const TSharedRef<FDMXZipper>& Zip, const UDMXMVRGeneralSceneDescription* GeneralSceneDescription);

		/** Creates an General Scene Description Xml File from the MVR Source, as it was imported */
		const TSharedPtr<FXmlFile> CreateSourceGeneralSceneDescriptionXmlFile(const UDMXMVRGeneralSceneDescription* GeneralSceneDescription) const;

		/**
		 * Gets raw source data or creates (possibly empty) source data where the source data is not present.
		 * Prior 5.1 there was no source data stored. Offers a dialog to load missing data
		 */
		const TArray64<uint8>& RefreshSourceDataAndFixtureType(UDMXEntityFixtureType& FixtureType, UDMXGDTFAssetImportData& InOutGDTFAssetImportData) const;
	};
}
