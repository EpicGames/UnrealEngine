// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXMVRXmlMergeUtility.h"

#include "DMXRuntimeLog.h"
#include "DMXRuntimeUtils.h"
#include "MVR/DMXMVRGeneralSceneDescription.h"
#include "MVR/Types/DMXMVRFixtureNode.h"
#include "MVR/Types/DMXMVRRootNode.h"
#include "MVR/Types/DMXMVRUnrealEngineDataNode.h"

#include "XmlFile.h"
#include "XmlNode.h"
#include "Algo/Find.h"


TSharedPtr<FXmlFile> FDMXXmlMergeUtility::Merge(const UDMXMVRGeneralSceneDescription* GeneralSceneDescription, const TSharedRef<FXmlFile>& OtherXml)
{
	checkf(GeneralSceneDescription, TEXT("Invalid General Scene Description provided when trying to merge MVR data"));

	FDMXXmlMergeUtility Instance;
	return Instance.MergeInternal(GeneralSceneDescription, OtherXml);
}


TSharedPtr<FXmlFile> FDMXXmlMergeUtility::MergeInternal(const UDMXMVRGeneralSceneDescription* GeneralSceneDescription, const TSharedRef<FXmlFile>& OtherXml)
{
	TSharedPtr<FXmlFile> GeneralSceneDescriptionXml = GeneralSceneDescription->CreateXmlFile();
	if (!GeneralSceneDescriptionXml.IsValid() || !OtherXml->GetRootNode())
	{
		return nullptr;
	}

	FXmlNode* RootNode = GeneralSceneDescriptionXml->GetRootNode();
	if (!RootNode)
	{
		return GeneralSceneDescriptionXml;
	}

	// 1. Merge User Data
	const TArray<FXmlNode*> AdditionalDataNodes = AcquireAdditionalData(OtherXml);
	if (AdditionalDataNodes.Num() > 0)
	{
		constexpr TCHAR UserDataTag[] = TEXT("UserData");
		FXmlNode* UserDataNode = RootNode->FindChildNode(UserDataTag);
		if (!UserDataNode)
		{
			RootNode->AppendChildNode(UserDataTag);

			UserDataNode = RootNode->GetChildrenNodes().Last();
		}
		check(UserDataNode);

		for (FXmlNode* AdditionalDataNode : AdditionalDataNodes)
		{
			AddXmlNodeWithChildren(UserDataNode, AdditionalDataNode);
		}
	}
	
	// 2. Merge Scene Data
	FXmlNode* SceneNode = GetSceneNode(GeneralSceneDescriptionXml.ToSharedRef());
	if (!SceneNode)
	{
		constexpr TCHAR SceneTag[] = TEXT("Scene");
		RootNode->AppendChildNode(SceneTag);

		SceneNode = RootNode->GetChildrenNodes().Last();
	}
	check(SceneNode);

	const FXmlNode* OtherSceneNode = GetSceneNode(OtherXml);
	if (!OtherSceneNode)
	{
		return GeneralSceneDescriptionXml;
	}

	// Scene
	if (SceneNode && OtherSceneNode)
	{
		MergeScenes(GeneralSceneDescription, SceneNode, OtherSceneNode);
	}
	
	// AUXData
	const FXmlNode* AdditionalAUXDataNode = AcquireAdditionalAUXData(OtherXml);
	if (AdditionalAUXDataNode)
	{
		constexpr TCHAR AUXDataTag[] = TEXT("AUXData");
		FXmlNode* AUXDataNode = SceneNode->FindChildNode(AUXDataTag);
		if (!AUXDataNode)
		{
			RootNode->AppendChildNode(AUXDataTag);

			AUXDataNode = RootNode->GetChildrenNodes().Last();
		}
		check(AUXDataNode);

		for (const FXmlNode* AdditionalAUXDataChild : AUXDataNode->GetChildrenNodes())
		{
			AddXmlNodeWithChildren(AUXDataNode, AdditionalAUXDataChild);
		}
	}

	return GeneralSceneDescriptionXml;
}

TArray<FXmlNode*> FDMXXmlMergeUtility::AcquireAdditionalData(const TSharedRef<FXmlFile>& Xml)
{
	TArray<FXmlNode*> Result;
	const FXmlNode* RootNode = Xml->GetRootNode();
	if (!RootNode)
	{
		return Result;
	}

	constexpr TCHAR UserDataTag[] = TEXT("UserData");
	const FXmlNode* UserDataNode = RootNode->FindChildNode(UserDataTag);
	if (!UserDataNode)
	{
		return Result;
	}

	// All children of 'UserData' are 'Data' nodes
	Result = UserDataNode->GetChildrenNodes();

	// Remove previous Unreal Engine Data nodes
	Result.RemoveAll([](const FXmlNode* Node)
		{	
			constexpr TCHAR ProviderAttributeName[] = TEXT("provider");
			const FString Provider = Node->GetAttribute(ProviderAttributeName);

			if (Provider == UDMXMVRUnrealEngineDataNode::ProviderNameUnrealEngine)
			{
				return true;
			}

			return false;
		});

	return Result;
}

const FXmlNode* FDMXXmlMergeUtility::AcquireAdditionalAUXData(const TSharedRef<FXmlFile>& Xml)
{
	const FXmlNode* SceneNode = GetSceneNode(Xml);
	if (!SceneNode)
	{
		return nullptr;
	}

	constexpr TCHAR AUXDataTag[] = TEXT("AUXData");
	return SceneNode->FindChildNode(AUXDataTag);
}

void FDMXXmlMergeUtility::MergeScenes(const UDMXMVRGeneralSceneDescription* GeneralSceneDescription, FXmlNode* InOutPrimarySceneNode, const FXmlNode* InSecondarySceneNode)
{
	checkf(InOutPrimarySceneNode->GetTag() == TEXT("scene") && InSecondarySceneNode->GetTag() == TEXT("scene"), TEXT("Trying to merge scene nodes, but nodes are '%s' and '%s' nodes."), *InOutPrimarySceneNode->GetTag(), *InSecondarySceneNode->GetTag());

	// Don't merge Fixtures, so the merged result is what sourced from the DMX Library and its General Scene Description object only
	constexpr TCHAR FixtureTagToIgnore[] = TEXT("Fixture");

	// If the primary has no Layer, just copy what's in the secondary
	constexpr TCHAR LayersTag[] = TEXT("Layers");
	const FXmlNode* PrimaryLayersNode = InOutPrimarySceneNode->FindChildNode(LayersTag);
	const FXmlNode* SecondaryLayersNode = InSecondarySceneNode->FindChildNode(LayersTag);
	if (!PrimaryLayersNode && SecondaryLayersNode)
	{
		AddXmlNodeWithChildren(InOutPrimarySceneNode, SecondaryLayersNode, FixtureTagToIgnore);
		return;
	}

	// Merge scenes
	checkf(PrimaryLayersNode, TEXT("Primary scene node has no layers node, but cases where there is no layers node should already have been handled."));
	for (const FXmlNode* SecondaryChildNode : InSecondarySceneNode->GetChildrenNodes())
	{
		AddXmlNodeWithChildren(InOutPrimarySceneNode, SecondaryChildNode, FixtureTagToIgnore);
	}
}

TArray<FGuid> FDMXXmlMergeUtility::GetFixtureUUIDs(const UDMXMVRGeneralSceneDescription* GeneralSceneDescription)
{
	check(GeneralSceneDescription);

	TArray<UDMXMVRFixtureNode*> FixtureNodes;
	GeneralSceneDescription->GetFixtureNodes(FixtureNodes);

	TArray<FGuid> Result;
	for (UDMXMVRFixtureNode* FixtureNode : FixtureNodes)
	{
		Result.Add(FixtureNode->UUID);
	}
	return Result;
}

FXmlNode* FDMXXmlMergeUtility::GetSceneNode(const TSharedRef<FXmlFile>& Xml)
{
	FXmlNode* RootNode = Xml->GetRootNode();
	if (!RootNode)
	{
		return nullptr;
	}

	constexpr TCHAR ScenenTag[] = TEXT("Scene");
	return RootNode->FindChildNode(ScenenTag);
}

void FDMXXmlMergeUtility::AddXmlNodeWithChildren(FXmlNode* ParentTarget, const FXmlNode* ChildToAdd, const FString& IgnoredTag)
{		
	// Ignore Tags where requested
	if (ChildToAdd->GetTag() == IgnoredTag)
	{
		return;
	}

	FXmlNode* ExistingNode = FindMatchingMVRNode(ParentTarget, ChildToAdd);
	if (ExistingNode)
	{
		for (const FXmlNode* ChildNode : TArray<const FXmlNode*>(ChildToAdd->GetChildrenNodes()))
		{
			AddXmlNodeWithChildren(ExistingNode, ChildNode, IgnoredTag);
		}
	}
	else
	{
		ParentTarget->AppendChildNode(ChildToAdd->GetTag(), ChildToAdd->GetContent(), ChildToAdd->GetAttributes());
		FXmlNode* NewNode = ParentTarget->GetChildrenNodes().Last();
		checkf(NewNode, TEXT("Expected a new node being added to Parent, but the last node is null"));

		// Recursive for children
		for (const FXmlNode* ChildNode : TArray<const FXmlNode*>(ChildToAdd->GetChildrenNodes()))
		{
			AddXmlNodeWithChildren(NewNode, ChildNode, IgnoredTag);
		}
	}
}

FXmlNode* FDMXXmlMergeUtility::FindMatchingMVRNode(const FXmlNode* ParentNode, const FXmlNode* OtherChild)
{
	const TArray<FXmlNode*> ChildNodes = ParentNode->GetChildrenNodes();
	FXmlNode* const* MatchingNodePtr = nullptr;

	// Identical UUIDs			
	constexpr TCHAR UUIDAttributeName[] = TEXT("UUID");
	const FString OtherChildUUIDString = OtherChild->GetAttribute(UUIDAttributeName);
	FGuid OtherChildUUID;
	if (FGuid::Parse(OtherChildUUIDString, OtherChildUUID))
	{
		MatchingNodePtr = Algo::FindByPredicate(ChildNodes, [UUIDAttributeName, &OtherChildUUID](const FXmlNode* Child)
			{
				const FString ChildUUIDString = Child->GetAttribute(UUIDAttributeName);

				FGuid ChildUUID;
				if (FGuid::Parse(ChildUUIDString, ChildUUID))
				{
					if (ChildUUID == OtherChildUUID)
					{
						return true;
					}
				}

				return false;
			});
	}
	
	// Identical Provider
	if (!MatchingNodePtr)
	{
		constexpr TCHAR ProviderTag[] = TEXT("provider");
		const FString OtherProvider = OtherChild->GetAttribute(ProviderTag);

		if (!OtherProvider.IsEmpty())
		{
			MatchingNodePtr = Algo::FindByPredicate(ChildNodes, [ProviderTag, &OtherProvider](const FXmlNode* Child)
				{
					const FString Provider = Child->GetAttribute(ProviderTag);
					if (Provider == OtherProvider)
					{
						return true;
					}

					return false;
				});
		}
	}

	// Specific tags that are unique in their hierarchy and match by tag
	if (!MatchingNodePtr)
	{
		MatchingNodePtr = Algo::FindByPredicate(ChildNodes, [OtherChild](const FXmlNode* Child)
			{		
				constexpr TCHAR ProviderTag[] = TEXT("provider");
				constexpr TCHAR LayersTag[] = TEXT("Layers");
				constexpr TCHAR LayerTag[] = TEXT("Layer");
				constexpr TCHAR ChildListTag[] = TEXT("ChildList");
				constexpr TCHAR GroupObjectTag[] = TEXT("GroupObject");

				const FString& ChildTag = Child->GetTag();
				if (ChildTag == ProviderTag ||
					ChildTag == LayersTag ||
					ChildTag == LayerTag ||
					ChildTag == ChildListTag ||
					ChildTag == GroupObjectTag)
				{
					if (Child->GetTag() == OtherChild->GetTag())
					{
						return true;
					}
				}

				return false;
			});
	}

	if (MatchingNodePtr)
	{
		return *MatchingNodePtr;
	}

	return nullptr;
}
