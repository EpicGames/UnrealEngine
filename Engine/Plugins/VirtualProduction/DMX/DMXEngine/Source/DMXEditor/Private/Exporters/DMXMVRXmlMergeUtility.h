// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UDMXMVRUnrealEngineDataNode;
class UDMXMVRGeneralSceneDescription;

class FXmlFile;
class FXmlNode;


/** Utility to merge two General Scene Description Xmls, prefers primary for duplicate entries. Not a generic xml merger. */
class FDMXXmlMergeUtility
{
public:
	/** Merges the secondary General Scene Description Xml into the GeneralSceneDescription from engine. Returns an Xml File of the merged files or nullptr if data couldn't be merged. */
	static TSharedPtr<FXmlFile> Merge(const UDMXMVRGeneralSceneDescription* GeneralSceneDescription, const TSharedRef<FXmlFile>& OtherXml);

private:
	/** Merges two General Scene Description Xmls */
	TSharedPtr<FXmlFile> MergeInternal(const UDMXMVRGeneralSceneDescription* GeneralSceneDescription, const TSharedRef<FXmlFile>& OtherXml);

	/** Acquires additional Data nodes, considering the UnrealEngineDataNode already being present */
	TArray<FXmlNode*> AcquireAdditionalData(const TSharedRef<FXmlFile>& Xml);

	/** Acquires additional AUXData nodes (child of Scene) */
	const FXmlNode* AcquireAdditionalAUXData(const TSharedRef<FXmlFile>& Xml);

	/** Merges the secondary scene into the primary */
	void MergeScenes(const UDMXMVRGeneralSceneDescription* GeneralSceneDescription, FXmlNode* InOutPrimarySceneNode, const FXmlNode* InSecondarySceneNode);

	/** Acquires additional AUXData nodes (child of Scene) */
	TArray<FGuid> GetFixtureUUIDs(const UDMXMVRGeneralSceneDescription* GeneralSceneDescription);

	/** Returns the scene node of the GeneralSceneDescription Xml File */
	FXmlNode* GetSceneNode(const TSharedRef<FXmlFile>& Xml);

	/** Adds source node to target with all its children. Optionaly ignores Nodes that specify a Tag */
	void AddXmlNodeWithChildren(FXmlNode* ParentTarget, const FXmlNode* ChildToAdd, const FString& IgnoredTag = FString());

	/** Tries ot find a Node in Parent (of Xml File A) that matches Child (of Xml File B) */
	FXmlNode* FindMatchingMVRNode(const FXmlNode* ParentNode, const FXmlNode* Child);
};
