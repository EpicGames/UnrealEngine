// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXUnrealToGDTFAttributeConversion.h"

namespace UE::DMX::GDTF
{
	const TMap<FName, FName> FDMXUnrealToGDTFAttributeConversion::UnrealToGDTFAttributeMap =
	{
		{ "Intensity", "Dimmer" },
		{ "Strength", "Dimmer" },
		{ "Brightness", "Dimmer" },

		{ "Red", "ColorAdd_R" },
		{ "Green", "ColorAdd_G" },
		{ "Blue", "ColorAdd_B" },
		{ "Cyan", "ColorAdd_C" },
		{ "Magenta", "ColorAdd_M" },
		{ "Yellow", "ColorAdd_Y" },
		{ "White", "ColorAdd_W" },
		{ "Amber", "ColorAdd_A" },

		{ "Gobo Spin", "GoboSpin" },
		{ "Gobo Wheel Rotate", "GoboWheel" }
	};

	const TMap<FName, FName> FDMXUnrealToGDTFAttributeConversion::GDTFAttributeToPrettyMap =
	{
		{ "Dimmer", "Dim" },

		{ "ColorAdd_R", "R" },
		{ "ColorAdd_G", "G" },
		{ "ColorAdd_B", "B" },
		{ "ColorAdd_C", "C" },
		{ "ColorAdd_M", "M" },
		{ "ColorAdd_Y", "Y" },
		{ "ColorAdd_W", "W" },
		{ "ColorAdd_A", "A" },

		{ "Pan", "P" },
		{ "Tilt", "T" }
	};

	const TMap<FName, TPair<FName, FName>> FDMXUnrealToGDTFAttributeConversion::GDTFAttributeToFeatureMap =
	{
		// Dimmer feature group
		{ "Dimmer",			{ "Dimmer", "Dimmer" } },

		// Color feature group
		{ "Color",			{ "Color", "Color" } },
		{ "CTC",			{ "Color", "Color" } },
		{ "ColorAdd_R",		{ "Color", "RGB" } },
		{ "ColorAdd_G",		{ "Color", "RGB" } },
		{ "ColorAdd_B",		{ "Color", "RGB" } },
		{ "ColorAdd_C",		{ "Color", "RGB" } },
		{ "ColorAdd_M",		{ "Color", "RGB" } },
		{ "ColorAdd_Y",		{ "Color", "RGB" } },
		{ "ColorAdd_W",		{ "Color", "RGB" } },
		{ "ColorAdd_A",		{ "Color", "RGB" } },
		{ "CIE_X",			{ "Color", "CIE" } },
		{ "CIE_Y",			{ "Color", "CIE" } },
		{ "CIE_Brightness", { "Color", "CIE" } },

		// Position feature group
		{ "Pan",			{ "Position", "PanTilt" } },
		{ "Tilt",			{ "Position", "PanTilt" } },

		// Gobo feature group
		{ "GoboSpin",		{ "Gobo", "Gobo" } },
		{ "GoboWheel",		{ "Gobo", "Gobo" } },

		// Focus feature group
		{ "Focus",			{ "Focus", "Focus" } },
		{ "Zoom",			{ "Focus", "Focus" } },

		// Beam feature group
		{ "Shutter",		{ "Beam", "Beam" } },
		{ "Frost",			{ "Beam", "Beam" } }
	};
}
