// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Internationalization/Regex.h"
#include "UObject/NameTypes.h"

namespace UE::DMX::GDTF
{
	class FDMXUnrealToGDTFAttributeConversion
	{
	public:
		/** Creates a GDTF attribute from an unreal attribute */
		static FName ConvertUnrealToGDTFAttribute(const FName& InUnrealAttribute)
		{			
			// Remove, but remember trailing numbers
			FString CleanAttribute;
			FString TrailingNumberString; 

			const FRegexPattern RegexPattern(TEXT("(f\\d+oo)(\\d+)$"));
			FRegexMatcher RegexMatcher(RegexPattern, InUnrealAttribute.ToString());
			if (RegexMatcher.FindNext())
			{
				CleanAttribute = *RegexMatcher.GetCaptureGroup(0);
				TrailingNumberString = RegexMatcher.GetCaptureGroup(1);
			}
			else
			{
				CleanAttribute = InUnrealAttribute.ToString();
			}

			const FName* AttributeNamePtr = UnrealToGDTFAttributeMap.Find(*CleanAttribute);
			if (AttributeNamePtr)
			{
				return *((*AttributeNamePtr).ToString() + TrailingNumberString);
			}
			else
			{
				return InUnrealAttribute;
			}
		}

		static FName GetPrettyFromGDTFAttribute(const FName& InGDTFAttribute)
		{
			const FName* PrettyPtr = GDTFAttributeToPrettyMap.Find(InGDTFAttribute);
			if (PrettyPtr)
			{
				return (*PrettyPtr);
			}
			else
			{
				return InGDTFAttribute;
			}
		}

		/** Creates a GDTF feature group for a GDTF Attribute */
		static FName GetFeatureGroupForGDTFAttribute(const FName& InGDTFAttribute)
		{
			const TTuple<FName, FName>* FeaturePairPtr = GDTFAttributeToFeatureMap.Find(InGDTFAttribute);
			if (FeaturePairPtr)
			{
				return (*FeaturePairPtr).Key;
			}
			else
			{
				return "Control";
			}
		}

		/** Creates a GDTF feature for a GDTF Attribute */
		static FName GetFeatureForGDTFAttribute(const FName& InGDTFAttribute)
		{
			const TPair<FName, FName>* FeaturePairPtr = GDTFAttributeToFeatureMap.Find(InGDTFAttribute);
			if (FeaturePairPtr)
			{
				return (*FeaturePairPtr).Value;
			}
			else
			{
				return "Control";
			}
		}

	private:
		/** Conversion from Unreal attributes to GDTF attributes. See also how UDMXProtocolSettings::Attributes is initialized and applied. */
		static const TMap<FName, FName> UnrealToGDTFAttributeMap;

		/** Defines pretty attribute names for GDTF attribute names */
		static const TMap<FName, FName> GDTFAttributeToPrettyMap;

		/** 
		 * Defines attributes that should be assigned to a specific feature group. 
		 * 
		 * Assumes that GDTF and not Unreal attributes are used.
		 */
		static const TMap<FName, TPair<FName, FName>> GDTFAttributeToFeatureMap;
	};
}
