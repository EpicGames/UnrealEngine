// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/DMXEditorFactoryNew.h"
#include "AssetTypeCategories.h"
#include "Library/DMXLibrary.h"
#include "DMXEditorLog.h"
#include "DMXEditorModule.h"

#include "Widgets/SWindow.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Styling/AppStyle.h"
#include "Input/Reply.h"

#include "Components/ListView.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SComboBox.h"

#include "Editor.h"

#define LOCTEXT_NAMESPACE "UDMXEditorFactoryNew"


// The whole class is deprecated 5.1, but we keep the implementation for the time being
PRAGMA_DISABLE_DEPRECATION_WARNINGS
UDMXEditorFactoryNew::UDMXEditorFactoryNew()
{
}

UObject * UDMXEditorFactoryNew::FactoryCreateNew(UClass * Class, UObject * InParent, FName Name, EObjectFlags Flags, UObject * Context, FFeedbackContext * Warn)
{
	UDMXLibrary* DMXLibrary = nullptr;
	if (ensure(SupportedClass == Class))
	{
		ensure(0 != (RF_Public & Flags));
		DMXLibrary = MakeNewEditor(InParent, Name, Flags);
	}
	else
	{
		UE_LOG_DMXEDITOR(Warning, TEXT("SupportedClass should be the same as New Object class"));
	}

	return DMXLibrary;
}

UDMXLibrary * UDMXEditorFactoryNew::MakeNewEditor(UObject * InParent, FName Name, EObjectFlags Flags)
{
	return NewObject<UDMXLibrary>(InParent, Name, Flags);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
