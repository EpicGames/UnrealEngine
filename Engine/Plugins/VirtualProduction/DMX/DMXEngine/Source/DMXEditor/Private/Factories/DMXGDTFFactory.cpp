// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/DMXGDTFFactory.h"

#include "Application/SlateApplicationBase.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "DMXEditorLog.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorReimportHandler.h"
#include "Factories/DMXGDTFImporter.h"
#include "Factories/DMXGDTFImportUI.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/FileManager.h"
#include "Interfaces/IMainFrameModule.h"
#include "Library/DMXGDTFAssetImportData.h"
#include "Library/DMXImportGDTF.h"
#include "Misc/Paths.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SDMXGDTFOptionWindow.h"

#define LOCTEXT_NAMESPACE "DMXGDTFFactory"

UDMXGDTFImportUI::UDMXGDTFImportUI()
	: bUseSubDirectory(false)
	, bImportXML(true)
	, bImportTextures(false)
	, bImportModels(false)
{}

void UDMXGDTFImportUI::ResetToDefault()
{
    bUseSubDirectory = false;
    bImportXML = true;
    bImportTextures = false;
    bImportModels = false;
}

UDMXGDTFFactory::UDMXGDTFFactory()
{
    SupportedClass = nullptr;
	Formats.Add(TEXT("gdtf;General Device Type Format"));

	bCreateNew = false;
	bText = false;
	bEditorImport = true;
	bOperationCanceled = false;
}

bool UDMXGDTFFactory::DoesSupportClass(UClass* Class)
{
    return Class == UDMXImportGDTF::StaticClass();
}

UClass* UDMXGDTFFactory::ResolveSupportedClass()
{
    return UDMXImportGDTF::StaticClass();
}

UObject* UDMXGDTFFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& InFilename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
    FString FileExtension = FPaths::GetExtension(InFilename);
    const TCHAR* Type = *FileExtension;

    if (!IFileManager::Get().FileExists(*InFilename))
    {
        UE_LOG_DMXEDITOR(Error, TEXT("Failed to load file '%s'"), *InFilename)
        return nullptr;
    }

    ParseParms(Parms);

    CA_ASSUME(InParent);

    if (bOperationCanceled)
    {
        bOutOperationCanceled = true;
        GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
        return nullptr;
    }

    GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, InParent, InName, Type);

    UObject* ExistingObject = nullptr;
    if (InParent)
    {
        ExistingObject = StaticFindObject(UObject::StaticClass(), InParent, *(InName.ToString()));
        if (ExistingObject)
        {
			bShowOptions = false;
        }
    }

	if (FParse::Param(FCommandLine::Get(), TEXT("NoDMXImportOption")))
	{
		bShowOptions = false;
	}

    // Create import options
	using namespace UE::DMX;
    FDMXGDTFImportArgs ImportArgs;

	const FString BaseFilename = FPaths::GetBaseFilename(InName.ToString());
    ImportArgs.Name = *BaseFilename;
    ImportArgs.Parent = InParent;
    ImportArgs.Filename = InFilename;
    ImportArgs.Flags = Flags;

    // Set Import UI
    bool bIsAutomated = IsAutomatedImport();
    bool bShowImportDialog = bShowOptions && !bIsAutomated;

	ImportUI = nullptr;
	if (bShowImportDialog)
	{
		ImportUI = GetMutableDefault<UDMXGDTFImportUI>();
		bOperationCanceled = GetOptionsFromDialog(InParent);
		bOutOperationCanceled = bOperationCanceled;
	}

    if (bImportAll)
    {
        // If the user chose to import all, we don't show the dialog again and use the same settings for each object until importing another set of files
		bShowOptions = false;
    }

    if (ImportUI && !ImportUI->bImportXML && !ImportUI->bImportModels && !ImportUI->bImportTextures)
    {
		const FNotificationInfo Info(LOCTEXT("NothingToimportInfo", "Skipping import of GDTF, nothing to import."));
		FSlateNotificationManager::Get().AddNotification(Info);
		return nullptr;
    }

	// Import to the Editor
	FText OutErrorReason;
	UDMXImportGDTF* GDTF = FDMXGDTFImporter::Import(*this, ImportArgs, OutErrorReason);
	if (!GDTF)
	{
		const FNotificationInfo Info(OutErrorReason);
		FSlateNotificationManager::Get().AddNotification(Info);

		return nullptr;
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, GDTF);

    return GDTF;
}

bool UDMXGDTFFactory::FactoryCanImport(const FString& Filename)
{
	const FString TargetExtension = FPaths::GetExtension(Filename);

	return TargetExtension == TEXT("gdtf");
}

bool UDMXGDTFFactory::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{
	UDMXImportGDTF* GDTF = Cast<UDMXImportGDTF>(Obj);
	if (GDTF && GDTF->GetGDTFAssetImportData())
	{
		const FString SourceFilename = GDTF->GetGDTFAssetImportData()->GetFilePathAndName();
		OutFilenames.Add(SourceFilename);
		return true;
	}
	return false;
}

void UDMXGDTFFactory::SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths)
{
	UDMXImportGDTF* GDTF = Cast<UDMXImportGDTF>(Obj);
	if (GDTF && GDTF->GetGDTFAssetImportData() && ensure(NewReimportPaths.Num() == 1))
	{
		GDTF->GetGDTFAssetImportData()->SetSourceFile(NewReimportPaths[0]);
	}
}

EReimportResult::Type UDMXGDTFFactory::Reimport(UObject* InObject)
{
	UDMXImportGDTF* GDTF = Cast<UDMXImportGDTF>(InObject);

	if (!GDTF || !GDTF->GetGDTFAssetImportData())
	{
		return EReimportResult::Failed;
	}

	const FString SourceFilename = GDTF->GetGDTFAssetImportData()->GetFilePathAndName();
	if (!FPaths::FileExists(SourceFilename))
	{
		return EReimportResult::Failed;
	}

	bool bOutCanceled = false;
	if (ImportObject(InObject->GetClass(), InObject->GetOuter(), *InObject->GetName(), RF_Public | RF_Standalone, SourceFilename, nullptr, bOutCanceled))
	{
		return EReimportResult::Succeeded;
	}

	return bOutCanceled ? EReimportResult::Cancelled : EReimportResult::Failed;
}

bool UDMXGDTFFactory::GetOptionsFromDialog(UObject* Parent)
{
	if (!ensureMsgf(Parent, TEXT("Trying to display import options for transient object. This is not expected.")))
	{
		return false;
	}

	TSharedPtr<SWindow> ParentWindow;
	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		ParentWindow = MainFrame.GetParentWindow();
	}

	// Compute centered window position based on max window size, which include when all categories are expanded
	const float ImportWindowWidth = 410.0f;
	const float ImportWindowHeight = 750.0f;
	const FVector2D ImportWindowSize = FVector2D(ImportWindowWidth, ImportWindowHeight); // Max window size it can get based on current slate

	const FSlateRect WorkAreaRect = FSlateApplicationBase::Get().GetPreferredWorkArea();
	const FVector2D DisplayTopLeft(WorkAreaRect.Left, WorkAreaRect.Top);
	const FVector2D DisplaySize(WorkAreaRect.Right - WorkAreaRect.Left, WorkAreaRect.Bottom - WorkAreaRect.Top);

	const FVector2D WindowPosition = (DisplayTopLeft + (DisplaySize - ImportWindowSize) / 2.0f);

	const TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("GDTFImportOpionsTitle", "GDTF Import Options"))
		.SizingRule(ESizingRule::Autosized)
		.AutoCenter(EAutoCenter::None)
		.ClientSize(ImportWindowSize)
		.ScreenPosition(WindowPosition);

	using namespace UE::DMX;
	TSharedPtr<SDMXGDTFOptionWindow> OptionWindow;
	Window->SetContent
	(
		SAssignNew(OptionWindow, SDMXGDTFOptionWindow)
		.ImportUI(ImportUI)
		.WidgetWindow(Window)
		.FullPath(FText::FromString(Parent->GetPathName()))
		.MaxWindowHeight(ImportWindowHeight)
		.MaxWindowWidth(ImportWindowWidth)
	);

	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);

	bImportAll = OptionWindow->ShouldImportAll();

	return OptionWindow->ShouldImport();
}

#undef LOCTEXT_NAMESPACE
