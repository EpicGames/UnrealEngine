// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "EditorReimportHandler.h"
#include "UObject/StrongObjectPtr.h"
#include "DMXGDTFFactory.generated.h"

class UDMXGDTFImportUI;

UCLASS(hidecategories=Object)
class UDMXGDTFFactory
    : public UFactory
	, public FReimportHandler
{
	GENERATED_BODY()

public:
	UDMXGDTFFactory();

	//~ Begin UFactory Interface
	virtual bool DoesSupportClass(UClass * Class) override;
	virtual UClass* ResolveSupportedClass() override;
	virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled);
	virtual bool FactoryCanImport(const FString& Filename) override;
	//~ End UFactory Interface

	//~ Begin FReimportHandler Interface
	virtual bool CanReimport(UObject* Obj, TArray<FString>& OutFilenames) override;
	virtual void SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths) override;
	virtual EReimportResult::Type Reimport(UObject* Obj) override;
	//~ End FReimportHandler Interface

private:
	/** Shows an options dialog that initializes members. If false is returned, the import should be canceled */
	[[nodiscard]] bool GetOptionsFromDialog(UObject* Parent);

	/** If true, shows an options dialog. Can be false for example when reimporting */
	bool bShowOptions = false;

	/** If true, all content of the GDTF should be imported */
	bool bImportAll = false;

	/** If true, importing was canceled */
	bool bOperationCanceled = false;

	UPROPERTY(transient)
	TObjectPtr<UDMXGDTFImportUI> ImportUI;
};
