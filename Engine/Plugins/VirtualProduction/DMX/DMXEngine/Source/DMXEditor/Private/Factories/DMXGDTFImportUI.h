// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "DMXGDTFImportUI.generated.h"

UCLASS(config = Editor, HideCategories=Object, MinimalAPI)
class UDMXGDTFImportUI 
    : public UObject
{
	GENERATED_BODY()

public:
    UDMXGDTFImportUI();

	void ResetToDefault();

public:
    UPROPERTY(EditAnywhere, Config, Category = "DMX")
    bool bUseSubDirectory = true;

    UPROPERTY(EditAnywhere, Config, Category = "DMX")
    bool bImportXML = true;

    UPROPERTY(EditAnywhere, Config, Category = "DMX")
    bool bImportTextures = true;

    UPROPERTY(EditAnywhere, Config, Category = "DMX")
    bool bImportModels = true;
};
