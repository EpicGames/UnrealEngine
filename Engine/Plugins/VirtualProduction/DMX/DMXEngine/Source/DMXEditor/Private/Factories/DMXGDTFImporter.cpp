// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/DMXGDTFImporter.h"

#include "DMXZipper.h"
#include "Editor.h"
#include "Factories/DMXGDTFFactory.h"
#include "Library/DMXGDTFAssetImportData.h"
#include "Library/DMXImportGDTF.h"
#include "Misc/Paths.h"

#define LOCTEXT_NAMESPACE "DMXGDTFImporter"

namespace UE::DMX
{
	UDMXImportGDTF* FDMXGDTFImporter::Import(const UDMXGDTFFactory& InImportFactory, const FDMXGDTFImportArgs& ImportArgs, FText& OutErrorReason)
	{
		FDMXGDTFImporter Instance(ImportArgs);
		return Instance.ImportInternal(InImportFactory, OutErrorReason);
	}

	FDMXGDTFImporter::FDMXGDTFImporter(const FDMXGDTFImportArgs& InImportArgs)
		: ImportArgs(InImportArgs)
	{}

	UDMXImportGDTF* FDMXGDTFImporter::ImportInternal(const UDMXGDTFFactory& InImportFactory, FText& OutErrorReason)
	{
		if (!ensureAlwaysMsgf(FPaths::FileExists(ImportArgs.Filename), TEXT("Cannot import GDTF File '%s'. File does not exist."), *ImportArgs.Filename))
		{
			return nullptr;
		}

		// Unzip GDTF
		Zip = MakeShared<FDMXZipper>();
		if (!Zip->LoadFromFile(ImportArgs.Filename))
		{
			OutErrorReason = LOCTEXT("InvalidGDTFError", "Cannot read GDTF.File is not a valid GDTF file.");
			return nullptr;
		}

		return CreateGDTF(InImportFactory, OutErrorReason);
	}

	UDMXImportGDTF* FDMXGDTFImporter::CreateGDTF(const UDMXGDTFFactory& InImportFactory, FText& OutErrorReason) const
	{
		if (!ensureMsgf(Zip.IsValid(), TEXT("Invalid zipper. Cannot create GDTF asset.")))
		{
			return nullptr;
		}

		// Unzip Description.xml
		constexpr TCHAR DescriptionXmlFilename[] = TEXT("description.xml");
		TArray64<uint8> DescriptionXmlData;
		if (!Zip->GetFileContent(DescriptionXmlFilename, DescriptionXmlData))
		{
			OutErrorReason = LOCTEXT("MissingDescriptionXmlGDTFError", "Cannot read GDTF. Cannot find Description.xml.");
			return nullptr;
		}

		// Create GDTF
		const FName UniqueName = IsUniqueObjectName(ImportArgs.Name, ImportArgs.Parent.Get()) ?
			ImportArgs.Name :
			MakeUniqueObjectName(ImportArgs.Parent.Get(), UDMXImportGDTF::StaticClass(), ImportArgs.Name);

		UDMXImportGDTF* NewGDTF = NewObject<UDMXImportGDTF>(ImportArgs.Parent.Get(), UniqueName, ImportArgs.Flags | RF_Public);

		// Set Asset Import Data
		UDMXGDTFAssetImportData* GDTFAssetImportData = NewGDTF->GetGDTFAssetImportData();
		if (!ensureAlwaysMsgf(GDTFAssetImportData, TEXT("Unexpected missing Asset Import Data for newly created GDTF %s"), *NewGDTF->GetName()))
		{
			return nullptr;
		}
		GDTFAssetImportData->SetSourceFile(ImportArgs.Filename);

		return NewGDTF;
	}
}

#undef LOCTEXT_NAMESPACE
