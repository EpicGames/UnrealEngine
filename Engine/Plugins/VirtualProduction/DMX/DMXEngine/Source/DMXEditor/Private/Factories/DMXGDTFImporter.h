// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtr.h"

class FDMXZipper;
class UDMXImportGDTF;
class UDMXGDTFFactory;

namespace UE::DMX
{
	struct FDMXGDTFImportArgs
	{
	public:
		FDMXGDTFImportArgs()
			: Parent(nullptr)
			, Name(NAME_None)
			, Flags(RF_NoFlags)
		{}

		TWeakObjectPtr<UObject> Parent;
		FName Name;
		FString Filename;
		EObjectFlags Flags;
	};


	/** Imports a GDTF asset. */
	class FDMXGDTFImporter
	{
	public:
		/** Tries to import a GDTF, using params from the import factory. Returns the resulting GDTF object or nullptr if no GDTF asset could be created. */
		[[nodiscard]] static UDMXImportGDTF* Import(const UDMXGDTFFactory& InImportFactory, const FDMXGDTFImportArgs& InImportArgs, FText& OutErrorReason);

	private:
		/** Private constructor */
		FDMXGDTFImporter(const FDMXGDTFImportArgs& InImportArgs);

		/** Non-static implementation */
		UDMXImportGDTF* ImportInternal(const UDMXGDTFFactory& InImportFactory, FText& OutErrorReason);

		/** Creates the GDTF asset */
		UDMXImportGDTF* CreateGDTF(const UDMXGDTFFactory& InImportFactory, FText& OutErrorReason) const;

		/** Args for this importer */
		const FDMXGDTFImportArgs& ImportArgs;

		/** The GDTF File as zip */
		TSharedPtr<FDMXZipper> Zip;
	};
}
