// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXGDTFToFixtureTypeConverter.h"

#include "Algo/Copy.h"
#include "Algo/Count.h"
#include "Algo/Find.h"
#include "Algo/MaxElement.h"
#include "Algo/MinElement.h"
#include "DMXEditorLog.h"
#include "DMXGDTF.h"
#include "DMXProtocolSettings.h"
#include "DMXRuntimeUtils.h"
#include "DMXZipper.h"
#include "GDTF/AttributeDefinitions/DMXGDTFAttribute.h"
#include "GDTF/DMXGDTFDescription.h"
#include "GDTF/DMXGDTFFixtureType.h"
#include "GDTF/DMXModes/DMXGDTFChannelFunction.h"
#include "GDTF/DMXModes/DMXGDTFDMXChannel.h"
#include "GDTF/DMXModes/DMXGDTFDMXMode.h"
#include "GDTF/DMXModes/DMXGDTFLogicalChannel.h"
#include "GDTF/Geometries/DMXGDTFGeometry.h"
#include "GDTF/Geometries/DMXGDTFGeometryBreak.h"
#include "GDTF/Geometries/DMXGDTFGeometryCollect.h"
#include "GDTF/Geometries/DMXGDTFGeometryReference.h"
#include "GDTF/Models/DMXGDTFModel.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/DMXGDTFAssetImportData.h"
#include "Library/DMXImportGDTF.h"
#include "XmlFile.h"
#include "XmlNode.h"
#include <limits>

#define LOCTEXT_NAMESPACE "DMXInitializeFixtureTypeFromGDTFHelper"

namespace UE::DMX::GDTF
{
	/** Helper to interprete a GDTF Channel */
	class FDMXGDTFChannelInterpreter
	{
	public:
		/** Properties of a channel */
		struct FChannelProperties
		{
			FString AttributeName;
			uint32 FirstChannel = 1;
			EDMXFixtureSignalFormat SignalFormat = EDMXFixtureSignalFormat::E8Bit;
			bool bLSBMode = false;
			uint32 NumCells = 1;
			uint32 DefaultValue = 0;

			EDMXGDTFPhysicalUnit PhysicalUnit = EDMXGDTFPhysicalUnit::None;
			double PhysicalFrom = 0.0;
			double PhysicalTo = 1.0;
		};

		static bool GetChannelProperties(
			const TSharedRef<FDMXGDTFDMXMode>& InDMXMode, 
			const TSharedRef<FDMXGDTFDMXChannel>& InDMXChannelNode, 
			FChannelProperties& OutChannelProperties)
		{
			FDMXGDTFChannelInterpreter Instance(InDMXMode, InDMXChannelNode);
			
			return Instance.GetChannelPropertiesInternal(OutChannelProperties);
		}

	private:
		FDMXGDTFChannelInterpreter(const TSharedRef<FDMXGDTFDMXMode>& InDMXModeNode, const TSharedRef<FDMXGDTFDMXChannel>& InDMXChannelNode)
			: DMXModeNode(InDMXModeNode)
			, DMXChannelNode(InDMXChannelNode)
		{
			LogicalChannel = InDMXChannelNode->LogicalChannelArray.IsValidIndex(0) ? InDMXChannelNode->LogicalChannelArray[0] : nullptr;
			ChannelFunction = InDMXChannelNode->ResolveInitialFunction();
		}

		/** Returns the properties of the channel. The root geometry collect is the geometry collect the mode references. */
		bool GetChannelPropertiesInternal(FChannelProperties& OutChannelProperties) const
		{
			if (!GetAttributeName(OutChannelProperties.AttributeName))
			{
				return false;
			}

			OutChannelProperties.DefaultValue = GetDefaultValue();
			GetDataType(OutChannelProperties.SignalFormat, OutChannelProperties.bLSBMode);

			OutChannelProperties.PhysicalUnit = GetPhysicalUnit();
			OutChannelProperties.PhysicalFrom = GetPhysicalFrom();
			OutChannelProperties.PhysicalTo = GetPhysicalTo();

			const int32 NumGeometries = CountGeometries();
			if (NumGeometries > 1)
			{				
				// Try to interprete as Unreal matrix channel
				const TArray<TSharedPtr<FDMXGDTFGeometryReference>> GeometryReferences = DMXChannelNode->ResolveGeometryReferences();
				if (!GeometryReferences.IsEmpty())
				{
					OutChannelProperties.NumCells = GeometryReferences.Num();

					if (!GetMatrixOffset(GeometryReferences, OutChannelProperties.FirstChannel))
					{
						return false;
					}
				}
			}
			else
			{				
				// Try to interprete as Unreal channel function
				if (!GetMinOffsetOfDMXChannel(OutChannelProperties.FirstChannel))
				{
					return false;
				}
			}

			return true;
		}

		/** Returns the number of geometries the DMX channel references */
		int32 CountGeometries() const
		{
			// Try to find a single geometry that is referenced
			const TSharedPtr<FDMXGDTFGeometry> Geometry = DMXChannelNode->ResolveGeometry();
			if (Geometry.IsValid())
			{
				return 1;
			}

			// Try to find geometry references
			const TArray<TSharedPtr<FDMXGDTFGeometryReference>> GeometryReferences = DMXChannelNode->ResolveGeometryReferences();
			if (!GeometryReferences.IsEmpty())
			{
				return GeometryReferences.Num();
			}

			// Legacy GDTFs may reference a model instead of a geometry
			const TSharedPtr<FDMXGDTFFixtureType> FixtureType = DMXChannelNode->GetFixtureType().Pin();
			if (!ensureMsgf(FixtureType.IsValid(), TEXT("Unexpected invalid fixture type for DMX Channel node.")))
			{
				return 0;
			}
			const TSharedPtr<FDMXGDTFModel>* ModelPtr = Algo::FindByPredicate(FixtureType->Models, [this](const TSharedPtr<FDMXGDTFModel>& Model)
				{
					return Model->Name == DMXChannelNode->Geometry;
				});
			if (ModelPtr)
			{
				return 1;
			}

			// Fall back to the mode geometry 
			if (!FixtureType->GeometryCollect.IsValid())
			{
				return 0;
			}
			
			const TSharedPtr<FDMXGDTFGeometry> ModeGeometry = FixtureType->GeometryCollect->FindGeometryByName(*DMXModeNode->Geometry.ToString());
			if (ModeGeometry.IsValid())
			{
				return 1;
			}

			// Accept even if the mode self only references a model
			const TSharedPtr<FDMXGDTFModel>* ModeModelPtr = Algo::FindByPredicate(FixtureType->Models, [this](const TSharedPtr<FDMXGDTFModel>& Model)
				{
					return Model->Name == DMXModeNode->Geometry;
				});

			return ModeModelPtr ? 1 : 0;
		}

		/** 
		 * Returns the attribute name of the channel node. 
		 * As per legacy considers only the first logical channel - FDMXFixtureFunction is not in support of more than one. 
		 */
		bool GetAttributeName(FString& OutName) const
		{
			if (!LogicalChannel.IsValid())
			{
				return false;
			}

			OutName = LogicalChannel->Attribute.ToString();
			return true;
		}

		/** 
		 * Gets the default value of the channel. Fails quietly if no default value present.
		 * Note, FDMXFixtureFunction only supports int32 default values currently.
		 */
		int64 GetDefaultValue() const
		{
			if (ChannelFunction.IsValid() &&
				ChannelFunction->Default.IsSet())
			{
				return static_cast<int64>(ChannelFunction->Default.GetChecked(DMXChannelNode));
			}

			return 0;
		}

		EDMXGDTFPhysicalUnit GetPhysicalUnit() const
		{
			TSharedPtr<FDMXGDTFAttribute> Attribute = ChannelFunction.IsValid() ? ChannelFunction->ResolveAttribute() : nullptr;
			return Attribute.IsValid() ? Attribute->PhysicalUnit : EDMXGDTFPhysicalUnit::None;
		} 

		double GetPhysicalFrom() const
		{
			return ChannelFunction.IsValid() ? ChannelFunction->PhysicalFrom : 0.0;
		}

		double GetPhysicalTo() const
		{
			return ChannelFunction.IsValid() ? ChannelFunction->PhysicalTo : 1.0;
		}

		bool GetMinOffsetOfDMXChannel(uint32& OutChannelOffset) const
		{
			if (DMXChannelNode->Offset.IsEmpty())
			{
				return false;
			}
			OutChannelOffset = *Algo::MinElement(DMXChannelNode->Offset);

			return true;
		}

		bool GetMaxOffsetOfDMXChannel(uint32& OutChannelOffset) const
		{
			if (DMXChannelNode->Offset.IsEmpty())
			{
				return false;
			}
			OutChannelOffset = *Algo::MaxElement(DMXChannelNode->Offset);

			return true;
		}

		bool GetMinDMXBreak(const TArray<TSharedPtr<FDMXGDTFGeometryBreak>>& InDMXBreakArray, uint8& OutMinBreak) const
		{
			if (InDMXBreakArray.IsEmpty())
			{
				return false;
			}
			const TSharedPtr<FDMXGDTFGeometryBreak>* MinBreakPtr = Algo::MinElementBy(InDMXBreakArray, &FDMXGDTFGeometryBreak::DMXBreak);

			OutMinBreak = (*MinBreakPtr)->DMXBreak;
			return true;
		}

		bool GetMaxDMXBreak(const TArray<TSharedPtr<FDMXGDTFGeometryBreak>>& InDMXBreakArray, uint8& OutMaxBreak) const
		{
			if (InDMXBreakArray.IsEmpty())
			{
				return false;
			}
			const TSharedPtr<FDMXGDTFGeometryBreak>* MaxBreakPtr = Algo::MaxElementBy(InDMXBreakArray, &FDMXGDTFGeometryBreak::DMXBreak);

			OutMaxBreak = (*MaxBreakPtr)->DMXBreak;
			return true;
		}

		/** Gets the matrix offset attribute value from a Channel Node. Returns false if no offset can be found. */
		bool GetMatrixOffset(const TArray<TSharedPtr<FDMXGDTFGeometryReference>>& GeometryReferences, uint32& OutOffset) const
		{
			const FName GeometryName = DMXChannelNode->Geometry;

			uint32 ChannelOffset;
			if (!GetMinOffsetOfDMXChannel(ChannelOffset))
			{
				return false;
			}
			TArray<uint32> OffsetArray;
			for (const TSharedPtr<FDMXGDTFGeometryReference>& GeometryReference : GeometryReferences)
			{
				uint8 DMXBreak;
				if (GeometryReference.IsValid() &&
					GetMinDMXBreak(GeometryReference->BreakArray, DMXBreak))
				{
					OffsetArray.Add(DMXBreak);
				}
			}

			if (OffsetArray.IsEmpty())
			{
				return false;
			}
			
			OutOffset = *Algo::MinElement(OffsetArray) + ChannelOffset - 1;
			return true;
		}

		/** Returns the data type offset array implies */
		void GetDataType(EDMXFixtureSignalFormat& OutSignalFormat, bool& OutLSBOrder) const
		{
			OutLSBOrder = false;

			// Compute number of used addresses
			uint32 OffsetMin;
			uint32 OffsetMax;
			if (!GetMinOffsetOfDMXChannel(OffsetMin) ||
				!GetMaxOffsetOfDMXChannel(OffsetMax))
			{
				return;
			}

			const uint32 NumUsedAddresses = FMath::Clamp(OffsetMax - OffsetMin + 1, 1, DMX_MAX_FUNCTION_SIZE);

			OutSignalFormat = static_cast<EDMXFixtureSignalFormat>(NumUsedAddresses - 1);

			// Offsets represent the channels in MSB order. If they are in reverse order, it means this Function uses LSB format.
			if (DMXChannelNode->Offset.Num() > 1)
			{
				OutLSBOrder = DMXChannelNode->Offset[0] > DMXChannelNode->Offset[1];
			}
		}

		TSharedPtr<FDMXGDTFLogicalChannel> LogicalChannel;
		TSharedPtr<FDMXGDTFChannelFunction> ChannelFunction;

		const TSharedRef<FDMXGDTFDMXMode> DMXModeNode;
		const TSharedRef<FDMXGDTFDMXChannel> DMXChannelNode; 
	};

	bool FDMXGDTFToFixtureTypeConverter::ConvertGDTF(UDMXEntityFixtureType& InOutFixtureType, const UDMXImportGDTF& InGDTF, const bool bUpdateFixtureTypeName)
	{
		FDMXGDTFToFixtureTypeConverter Instance;
		return Instance.ConvertGDTFInternal(InOutFixtureType, InGDTF, bUpdateFixtureTypeName);
	}

	bool FDMXGDTFToFixtureTypeConverter::ConvertGDTFInternal(UDMXEntityFixtureType& InOutFixtureType, const UDMXImportGDTF& InGDTF, const bool bUpdateFixtureTypeName) const
	{
		UDMXGDTFAssetImportData* GDTFAssetImportData = InGDTF.GetGDTFAssetImportData();
		if (!ensureMsgf(GDTFAssetImportData, TEXT("Found GDTF Asset that has no GDTF asset import data subobject.")))
		{
			return false;
		}

		const TSharedRef<FDMXZipper> GDTFZip = MakeShared<FDMXZipper>();
		TArray64<uint8> RawGDTFZip;
		if (!GDTFZip->LoadFromData(GDTFAssetImportData->GetRawSourceData()) ||
			!GDTFZip->GetData(RawGDTFZip))
		{
			return false;
		}

		UDMXGDTF* DMXGDTF = NewObject<UDMXGDTF>();
		DMXGDTF->InitializeFromData(RawGDTFZip);

		const TSharedPtr<FDMXGDTFFixtureType> GDTFFixtureType = DMXGDTF->GetDescription().IsValid() ? DMXGDTF->GetDescription()->GetFixtureType() : nullptr;
		if (!GDTFFixtureType.IsValid())
		{
			return false;
		}

		if (bUpdateFixtureTypeName && GDTFFixtureType->Name.GetStringLength() > 0)
		{
			InOutFixtureType.Name = GDTFFixtureType->Name.ToString();
		}

		InOutFixtureType.Modes.Reset();
		for (const TSharedPtr<FDMXGDTFDMXMode>& GDTFDMXMode : GDTFFixtureType->DMXModes)
		{
			FDMXFixtureMode Mode;
			if (GenerateMode(GDTFFixtureType, GDTFDMXMode, Mode))
			{
				InOutFixtureType.Modes.Add(Mode);
			}
		}

		for (int32 ModeIndex = 0; ModeIndex < InOutFixtureType.Modes.Num(); ModeIndex++)
		{
			InOutFixtureType.UpdateChannelSpan(ModeIndex);
		}
		InOutFixtureType.GetOnFixtureTypeChanged().Broadcast(&InOutFixtureType);

		CleanupAttributes(InOutFixtureType);

		return true;
	}

	bool FDMXGDTFToFixtureTypeConverter::GenerateMode(const TSharedPtr<FDMXGDTFFixtureType>& InFixtureTypeNode, const TSharedPtr<FDMXGDTFDMXMode>& InDMXModeNode, FDMXFixtureMode& OutMode) const
	{
		if (!ensureMsgf(InFixtureTypeNode.IsValid(), TEXT("Cannot create mode for GDTF Fixture Type. Fixture Type is invalid.")) ||
			!ensureMsgf(InDMXModeNode.IsValid(), TEXT("Cannot create mode for GDTF Fixture Type. DMX Mode is invalid.")))
		{
			return false;
		}

		FDMXFixtureMode Mode;
		Mode.ModeName = InDMXModeNode->Name.ToString();

		TMap<FName, int32> AttributeNameToCountMap;
		for (const TSharedPtr<FDMXGDTFDMXChannel>& DMXChannelNode : InDMXModeNode->DMXChannels)
		{
			FDMXGDTFChannelInterpreter::FChannelProperties ChannelProperties;
			if (!FDMXGDTFChannelInterpreter::GetChannelProperties(InDMXModeNode.ToSharedRef(), DMXChannelNode.ToSharedRef(), ChannelProperties))
			{
				continue;
			}

			const bool bIsMatrix = ChannelProperties.NumCells > 1;
			if (bIsMatrix)
			{
				if (!Mode.bFixtureMatrixEnabled)
				{
					Mode.FixtureMatrixConfig.FirstCellChannel = ChannelProperties.FirstChannel;
					Mode.bFixtureMatrixEnabled = true;

					// Fixture matrix has one cell attribute by default, clear it.
					Mode.FixtureMatrixConfig.CellAttributes.Reset();
				}

				FDMXFixtureCellAttribute MatrixAttribute;
				MatrixAttribute.Attribute = FDMXAttributeName(*ChannelProperties.AttributeName);
				MatrixAttribute.bUseLSBMode = ChannelProperties.bLSBMode;
				MatrixAttribute.DataType = ChannelProperties.SignalFormat;
				MatrixAttribute.DefaultValue = ChannelProperties.DefaultValue;

				Mode.FixtureMatrixConfig.CellAttributes.Add(MatrixAttribute);
				Mode.FixtureMatrixConfig.YCells = ChannelProperties.NumCells;
			}
			else
			{
				const int32* AttributeCountPtr = AttributeNameToCountMap.Find(*ChannelProperties.AttributeName);
				const FString Attribute = AttributeCountPtr ? 
					ChannelProperties.AttributeName + TEXT("_") + FString::FromInt(*AttributeCountPtr) :
					ChannelProperties.AttributeName;

				FDMXFixtureFunction Function;
				Function.FunctionName = ChannelProperties.AttributeName;
				Function.Attribute = FDMXAttributeName(*Attribute);
				Function.Channel = ChannelProperties.FirstChannel;
				Function.bUseLSBMode = ChannelProperties.bLSBMode;
				Function.DataType = ChannelProperties.SignalFormat;

				// Set physical properties (order matters here)
				Function.SetPhysicalUnit(ChannelProperties.PhysicalUnit);
				Function.SetPhysicalValueRange(ChannelProperties.PhysicalFrom, ChannelProperties.PhysicalTo);
				Function.DefaultValue = ChannelProperties.DefaultValue;
				Function.UpdatePhysicalDefaultValue();

				Mode.Functions.Add(Function);

				AttributeNameToCountMap.FindOrAdd(*ChannelProperties.AttributeName, 0)++;
			}
		}

		CleanupMode(Mode);

		OutMode = Mode;
		return true;
	}

	void FDMXGDTFToFixtureTypeConverter::CleanupMode(FDMXFixtureMode& InOutMode) const
	{
		TOptional<TRange<int32>> MatrixRange;
		if (InOutMode.bFixtureMatrixEnabled)
		{
			// Only one single, consecutive matrix is supported by the engine in this version.
			MatrixRange = TRange<int32>(InOutMode.FixtureMatrixConfig.FirstCellChannel, InOutMode.FixtureMatrixConfig.GetLastChannel() + 1);
			const FDMXFixtureFunction* OverlappingFunction = Algo::FindByPredicate(InOutMode.Functions, [MatrixRange](const FDMXFixtureFunction& Function)
				{
					const TRange<int32> FunctionRange(Function.Channel, Function.GetLastChannel() + 1);
					return FunctionRange.Overlaps(MatrixRange.GetValue());
				});

			if (OverlappingFunction)
			{
				UE_LOGF(LogDMXEditor, Warning, "Mode '%ls' contains many matrices, but this version of Unreal Engine only supports one matrix. Skipping import of mode.", *InOutMode.ModeName);
				InOutMode.Functions.Reset();
				InOutMode.bFixtureMatrixEnabled = false;
				InOutMode.FixtureMatrixConfig.CellAttributes.Reset();
				InOutMode.ModeName = FString::Printf(TEXT("n/a '%s' [not supported in this Engine Version]"), *InOutMode.ModeName);
			}
		}

		// Make sure functions are in consecutive order, insert 'reserved' channels where no channel is specified
		if (!InOutMode.Functions.IsEmpty())
		{
			Algo::SortBy(InOutMode.Functions, [](const FDMXFixtureFunction& Function)
				{
					return Function.Channel;
				});

			// Fill in blank functions
			const int32 LastFunctionChannel = InOutMode.Functions.Last().Channel;
			for (int32 Channel = 1; Channel < LastFunctionChannel; Channel++)
			{
				const FDMXFixtureFunction* FunctionOnChannelPtr = Algo::FindByPredicate(InOutMode.Functions, [Channel](const FDMXFixtureFunction& Function)
					{
						const TRange<int32> FunctionRange(Function.Channel, Function.GetLastChannel() + 1);
						return FunctionRange.Contains(Channel);
					});

				const bool bChannelHasFunction = FunctionOnChannelPtr != nullptr;
				const bool bChannelHasMatrix = MatrixRange.IsSet() && MatrixRange.GetValue().Contains(Channel);

				if (!bChannelHasFunction && !bChannelHasMatrix)
				{
					FDMXFixtureFunction EmptyFunction;
					EmptyFunction.Channel = Channel;
					EmptyFunction.FunctionName = TEXT("<empty>");
					InOutMode.Functions.Add(EmptyFunction);
				}
			}
		}

		Algo::SortBy(InOutMode.Functions, [](const FDMXFixtureFunction& Function)
			{
				return Function.Channel;
			});
	}

	void FDMXGDTFToFixtureTypeConverter::CleanupAttributes(UDMXEntityFixtureType& InOutFixtureType) const
	{
		// Get Protocol Setting's default attributes
		const UDMXProtocolSettings* ProtocolSettings = GetDefault<UDMXProtocolSettings>();
		TMap<FName, TArray<FString> > SettingsAttributeNameToKeywordsMap;
		for (const FDMXAttribute& Attribute : ProtocolSettings->Attributes)
		{
			TArray<FString> Keywords = Attribute.GetKeywords();

			SettingsAttributeNameToKeywordsMap.Emplace(Attribute.Name, Keywords);
		}

		for (FDMXFixtureMode& Mode : InOutFixtureType.Modes)
		{
			TArray<FName> AssignedAttributeNames;
			for (FDMXFixtureFunction& Function : Mode.Functions)
			{
				const TTuple<FName, TArray<FString>>* SettingsAttributeNameToKeywordPairPtr = Algo::FindByPredicate(SettingsAttributeNameToKeywordsMap, [Function](const TTuple<FName, TArray<FString>> AttributeNameToKeywordPair)
					{
						if (Function.Attribute.Name == AttributeNameToKeywordPair.Key ||
							AttributeNameToKeywordPair.Value.Contains(Function.Attribute.Name.ToString()))
						{
							return true;
						}

						return false;
					});

				if (SettingsAttributeNameToKeywordPairPtr && !AssignedAttributeNames.Contains(SettingsAttributeNameToKeywordPairPtr->Key))
				{
					Function.Attribute.Name = SettingsAttributeNameToKeywordPairPtr->Key;
					AssignedAttributeNames.Add(SettingsAttributeNameToKeywordPairPtr->Key);
				}
				else
				{
					AssignedAttributeNames.Add(Function.Attribute.Name);
				}
			}

			if (Mode.bFixtureMatrixEnabled)
			{
				TArray<FName> AssignedCellAttributeNames;
				for (FDMXFixtureCellAttribute& CellAttribute : Mode.FixtureMatrixConfig.CellAttributes)
				{
					const TTuple<FName, TArray<FString>>* SettingsAttributeNameToKeywordPairPtr = Algo::FindByPredicate(SettingsAttributeNameToKeywordsMap, [CellAttribute](const TTuple<FName, TArray<FString>> AttributeNameToKeywordPair)
						{
							if (CellAttribute.Attribute.Name == AttributeNameToKeywordPair.Key ||
								AttributeNameToKeywordPair.Value.Contains(CellAttribute.Attribute.Name.ToString()))
							{
								return true;
							}

							return false;
						});

					if (SettingsAttributeNameToKeywordPairPtr && !AssignedAttributeNames.Contains(SettingsAttributeNameToKeywordPairPtr->Key))
					{
						CellAttribute.Attribute.Name = SettingsAttributeNameToKeywordPairPtr->Key;
						AssignedAttributeNames.Add(SettingsAttributeNameToKeywordPairPtr->Key);
					}
					else
					{
						AssignedAttributeNames.Add(CellAttribute.Attribute.Name);
					}
				}
			}
		}
	}

	TArray<const FXmlNode*> FDMXGDTFToFixtureTypeConverter::GetChildrenRecursive(const FXmlNode& ParentNode) const
	{
		TArray<const FXmlNode*> Result;

		for (const FXmlNode* Child : ParentNode.GetChildrenNodes())
		{
			Result.Add(Child);
			Result.Append(GetChildrenRecursive(*Child));
		}

		return Result;
	}
}

#undef LOCTEXT_NAMESPACE
