// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UDMXEntityFixtureType;
struct FDMXFixtureFunction;
struct FDMXFixtureMode;
class FDMXZipper;
class FXmlNode;
class UDMXImportGDTF;

namespace UE::DMX::GDTF
{
	class FDMXGDTFFixtureType;
	class FDMXGDTFDMXMode;

	/**
	 * This is a 5.1 private editor only workaround to initialize Fixture Types from GDTF assets.
	 *
	 * Required since DMXImportGDTF does not hold required data, hence UDMXEntityFixtureType::SetModesFromDMXImport cannot create matrix fixture types.
	 *
	 * Opeted for this workaround since it would require to rework the entirety of the GDTF implementation to hold its data correctly.
	 * This exceeds planed features of 5.1 and is a planned feature for 5.2.
	 */
	class FDMXGDTFToFixtureTypeConverter
	{
	public:
		/** Imports the modes directly from the asset import data */
		static bool ConvertGDTF(UDMXEntityFixtureType& FixtureType, const UDMXImportGDTF& InGDTF, const bool bUpdateFixtureTypeName);

	private:
		/** Imports the modes directly from the asset import data. Non-static implementation */
		bool ConvertGDTFInternal(UDMXEntityFixtureType& InOutFixtureType, const UDMXImportGDTF& InGDTF, const bool bUpdateFixtureTypeName) const;

		/** Generates a Mode from specified Nodes. Returns true on success. */
		bool GenerateMode(const TSharedPtr<FDMXGDTFFixtureType>& InFixtureTypeNode, const TSharedPtr<FDMXGDTFDMXMode>& InDMXModeNode, FDMXFixtureMode& OutMode) const;

		/** Makes sure the mode can be used by the engine */
		void CleanupMode(FDMXFixtureMode& InOutMode) const;

		/** Makes sure that attributes are replaced with attributes specified in protocol settings where possible */
		void CleanupAttributes(UDMXEntityFixtureType& InOutFixtureType) const;

		/** Returns all nodes that are decendants of the node */
		TArray<const FXmlNode*> GetChildrenRecursive(const FXmlNode& ParentNode) const;
	};

}
