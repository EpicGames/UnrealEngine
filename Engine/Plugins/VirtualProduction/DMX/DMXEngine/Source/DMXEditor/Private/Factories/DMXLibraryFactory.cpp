// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/DMXLibraryFactory.h"

#include "DMXEditorLog.h"
#include "DMXEditorModule.h"
#include "Library/DMXLibrary.h"


UDMXLibraryFactory::UDMXLibraryFactory()
{
	SupportedClass = UDMXLibrary::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UDMXLibraryFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UDMXLibrary* DMXLibrary = NewObject<UDMXLibrary>(InParent, Name, Flags);

	return DMXLibrary;
}

uint32 UDMXLibraryFactory::GetMenuCategories() const
{
	return (uint32)FDMXEditorModule::GetAssetCategory();
}
