// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/DMXLibraryFromMVRFactory.h"

#include "DMXEditorSettings.h"
#include "DMXEditorLog.h"
#include "Factories/DMXLibraryFromMVRImporter.h"
#include "Factories/DMXLibraryFromMVRImportOptions.h"
#include "Library/DMXLibrary.h"
#include "MVR/DMXMVRAssetImportData.h"
#include "MVR/DMXMVRGeneralSceneDescription.h"

#include "Editor.h"
#include "EditorReimportHandler.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/Paths.h"
#include "Subsystems/ImportSubsystem.h"


#define LOCTEXT_NAMESPACE "DMXLibraryFromMVRFactory"

const FString UDMXLibraryFromMVRFactory::MVRFileExtension = TEXT("MVR");

UDMXLibraryFromMVRFactory::UDMXLibraryFromMVRFactory()
{
	bEditorImport = true;
	bEditAfterNew = true;
	SupportedClass = UDMXLibrary::StaticClass();

	Formats.Add(TEXT("mvr;My Virtual Rig"));
}

UObject* UDMXLibraryFromMVRFactory::FactoryCreateFile(UClass* InClass, UObject* Parent, FName InName, EObjectFlags Flags, const FString& InFilename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCancelled)
{
	CurrentFilename = InFilename;

	if (!FPaths::FileExists(InFilename))
	{
		UE_LOGF(LogDMXEditor, Error, "Failed to create DMX Library for MVR '%ls'. Cannot find file.", *InFilename);
		return nullptr;
	}
	UDMXEditorSettings* DMXEditorSettings = GetMutableDefault<UDMXEditorSettings>();
	check(DMXEditorSettings);
	DMXEditorSettings->LastMVRImportPath = FPaths::GetPath(InFilename);

	FDMXLibraryFromMVRImporter MVRImporter;
	if (!MVRImporter.LoadMVRFile(InFilename))
	{
		return nullptr;
	}

	UDMXLibraryFromMVRImportOptions* ImportOptions = NewObject<UDMXLibraryFromMVRImportOptions>();
	MVRImporter.UpdateImportOptionsFromModalWindow(ImportOptions, bOutOperationCancelled);
	if (ImportOptions->bCancelled)
	{
		return nullptr;
	}

	UImportSubsystem* ImportSubsystem = GEditor->GetEditorSubsystem<UImportSubsystem>();
	ImportSubsystem->BroadcastAssetPreImport(this, UDMXLibrary::StaticClass(), Parent, InName, *MVRFileExtension);

	UDMXLibrary* NewDMXLibrary = nullptr;
	TArray<UDMXImportGDTF*> GDTFs;
	MVRImporter.Import(Parent, InName, Flags | RF_Public, ImportOptions, NewDMXLibrary, GDTFs, bOutOperationCancelled);

	ImportSubsystem->BroadcastAssetPostImport(this, NewDMXLibrary);

	ImportOptions->ApplyOptions(NewDMXLibrary);

	return NewDMXLibrary;
}

bool UDMXLibraryFromMVRFactory::FactoryCanImport(const FString& Filename)
{
	const FString TargetExtension = FPaths::GetExtension(Filename);
	if (TargetExtension.Equals(UDMXLibraryFromMVRFactory::MVRFileExtension, ESearchCase::IgnoreCase))
	{
		return true;
	}

	return false;
}

bool UDMXLibraryFromMVRFactory::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{
	UDMXMVRAssetImportData* MVRAssetImportData = GetMVRAssetImportData(Obj);
	if (!MVRAssetImportData)
	{
		return false;
	}
	
	const FString SourceFilename = MVRAssetImportData->GetFilePathAndName();
	OutFilenames.Add(SourceFilename);
	return true;
}

void UDMXLibraryFromMVRFactory::SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths)
{
	UDMXMVRAssetImportData* MVRAssetImportData = GetMVRAssetImportData(Obj);
	if (!ensureMsgf(MVRAssetImportData, TEXT("Invalid MVR Asset Import Data for General Scene Description.")))
	{
		return;
	}

	if (!ensureMsgf(NewReimportPaths.Num() == 1, TEXT("Unexpected, more than one or no new reimport path when reimporting DMX Library from MVR.")))
	{
		return;
	}

	MVRAssetImportData->SetSourceFile(NewReimportPaths[0]);
}

EReimportResult::Type UDMXLibraryFromMVRFactory::Reimport(UObject* Obj)
{
	UDMXMVRAssetImportData* MVRAssetImportData = GetMVRAssetImportData(Obj);
	if (!ensureMsgf(MVRAssetImportData, TEXT("Invalid MVR Asset Import Data for General Scene Description.")))
	{
		return EReimportResult::Failed;
	}
	FString SourceFilename = MVRAssetImportData->GetFilePathAndName();

	UDMXLibrary* DMXLibrary = Cast<UDMXLibrary>(Obj);
	if (!ensureMsgf(DMXLibrary, TEXT("Invalid DMX Library when trying to reimport DMX Library.")))
	{
		return EReimportResult::Failed;
	}

	FDMXLibraryFromMVRImporter MVRImporter;
	if (!MVRImporter.LoadMVRFile(SourceFilename))
	{
		return EReimportResult::Failed;
	}

	UDMXLibraryFromMVRImportOptions* ImportOptions = NewObject<UDMXLibraryFromMVRImportOptions>();
	ImportOptions->ImportIntoDMXLibrary = CastChecked<UDMXLibrary>(Obj);
	ImportOptions->bIsReimport = true;
	ImportOptions->bCreateNewDMXLibrary = false;

	bool bOperationCancelled;
	MVRImporter.UpdateImportOptionsFromModalWindow(ImportOptions, bOperationCancelled);
	if (bOperationCancelled)
	{
		return EReimportResult::Cancelled;
	}

	if (ImportOptions->bCreateNewDMXLibrary)
	{		
		// Case where the user wants to create a new DMX Library instead of reimporting
		const FName NewDMXLibraryName = MakeUniqueObjectName(DMXLibrary->GetOuter(), UDMXLibrary::StaticClass(), *DMXLibrary->GetName(), EUniqueObjectNameOptions::GloballyUnique);

		UImportSubsystem* ImportSubsystem = GEditor->GetEditorSubsystem<UImportSubsystem>();
		ImportSubsystem->BroadcastAssetPreImport(this, UDMXLibrary::StaticClass(), DMXLibrary->GetOuter(), NewDMXLibraryName, *MVRFileExtension);

		UDMXLibrary* NewDMXLibrary = DuplicateObject(DMXLibrary, DMXLibrary->GetOuter());
		NewDMXLibrary->Rename(*NewDMXLibraryName.ToString());

		TArray<UDMXImportGDTF*> GDTFs;
		MVRImporter.Reimport(DMXLibrary, ImportOptions, GDTFs);

		ImportSubsystem->BroadcastAssetPostImport(this, NewDMXLibrary);
		FAssetRegistryModule::AssetCreated(NewDMXLibrary);

		ImportOptions->ApplyOptions(NewDMXLibrary);

		// Return 'Cancelled' so the reimport succes notification doesn't show
		return EReimportResult::Cancelled;
	}
	else
	{
		if (ImportOptions->ImportIntoDMXLibrary &&
			ImportOptions->ImportIntoDMXLibrary != DMXLibrary)
		{		
			// Case where user selected a different DMX Library to reimport into
			DMXLibrary = ImportOptions->ImportIntoDMXLibrary;
		}

		// Reimport
		TArray<UDMXImportGDTF*> GDTFs;
		MVRImporter.Reimport(DMXLibrary, ImportOptions, GDTFs);

		ImportOptions->ApplyOptions(DMXLibrary);

		return EReimportResult::Succeeded;
	}
}

int32 UDMXLibraryFromMVRFactory::GetPriority() const
{
	return ImportPriority;
}

UDMXMVRAssetImportData* UDMXLibraryFromMVRFactory::GetMVRAssetImportData(UObject* DMXLibraryObject) const
{
	UDMXLibrary* DMXLibrary = Cast<UDMXLibrary>(DMXLibraryObject);
	if (!DMXLibrary)
	{
		return nullptr;
	}

	UDMXMVRGeneralSceneDescription* GeneralSceneDescription = DMXLibrary->GetLazyGeneralSceneDescription();
	if (!ensureMsgf(GeneralSceneDescription, TEXT("Invalid General Scene Description in DMX Library.")))
	{
		return nullptr;
	}

	return GeneralSceneDescription->GetMVRAssetImportData();
}

#undef LOCTEXT_NAMESPACE
