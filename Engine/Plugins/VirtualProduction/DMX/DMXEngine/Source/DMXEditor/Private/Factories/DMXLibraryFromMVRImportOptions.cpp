// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/DMXLibraryFromMVRImportOptions.h"

#include "IO/DMXInputPort.h"
#include "IO/DMXInputPortConfig.h"
#include "IO/DMXPortManager.h"
#include "IO/DMXOutputPort.h"
#include "IO/DMXOutputPortConfig.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXLibrary.h"

#include "Algo/Find.h"
#include "Algo/MaxElement.h"
#include "Algo/MinElement.h"


void UDMXLibraryFromMVRImportOptions::ApplyOptions(UDMXLibrary* DMXLibrary)
{
	if (!ensureAlwaysMsgf(DMXLibrary, TEXT("Trying to apply MVR Import Options, but DMX Library is invalid.")))
	{
		return;
	}

	if (!ensureAlwaysMsgf(!bCancelled, TEXT("Trying to apply MVR Import Options, but import was Cancelled while the options were presented.")))
	{
		return;
	}

	const TArray<UDMXEntityFixturePatch*> TempFixturePatches = DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>();
	if (TempFixturePatches.IsEmpty())
	{
		return;
	}
	const UDMXEntityFixturePatch* const* FirstPatchPtr = Algo::MinElementBy(TempFixturePatches, [](const UDMXEntityFixturePatch* FixturePatch)
		{
			return FixturePatch->GetUniverseID();
		});
	const UDMXEntityFixturePatch* const* LastPatchPtr = Algo::MaxElementBy(TempFixturePatches, [](const UDMXEntityFixturePatch* FixturePatch)
		{
			return FixturePatch->GetUniverseID();
		});
	checkf(FirstPatchPtr && LastPatchPtr, TEXT("Unexpected, cannot find Min and Max element in an array that is not empty."));

	const int32 FirstUniverse = (*FirstPatchPtr)->GetUniverseID();
	const int32 NumUniverses = FMath::Max(1, FMath::Clamp((*LastPatchPtr)->GetUniverseID() - FirstUniverse + 1, 0, TNumericLimits<int32>::Max()));

	if (bUpdateInputPort)
	{
		UDMXProtocolSettings* ProtocolSettings = GetMutableDefault<UDMXProtocolSettings>();
		const TSharedPtr<FDMXInputPort> InputPort = FDMXPortManager::Get().FindInputPortByGuid(InputPortToUpdate.GetPortGuid());
		if (InputPort.IsValid())
		{
			FDMXInputPortConfig* InputPortConfigPtr = Algo::FindByPredicate(ProtocolSettings->InputPortConfigs, [&InputPort](const FDMXInputPortConfig& InputPortConfig)
				{
					return InputPortConfig.GetPortGuid() == InputPort->GetPortGuid();
				});
			if (ensureAlwaysMsgf(InputPortConfigPtr, TEXT("Cannot find config of Input Port '%s' in Project Settings."), *InputPort->GetPortName()))
			{
				FDMXInputPortConfig& PortConfig = *InputPortConfigPtr;

				FDMXInputPortConfigParams ConfigParams(PortConfig);
				ConfigParams.LocalUniverseStart = FMath::Min(PortConfig.GetLocalUniverseStart(), FirstUniverse);
				ConfigParams.NumUniverses = FMath::Max(PortConfig.GetNumUniverses(), NumUniverses);
				
				PortConfig = FDMXInputPortConfig(PortConfig.GetPortGuid(), ConfigParams);
			}
		}
		else
		{
			FDMXInputPortConfig NewConfig;
			FDMXInputPortConfigParams ConfigParams(NewConfig);
			ConfigParams.LocalUniverseStart = FMath::Min(NewConfig.GetLocalUniverseStart(), FirstUniverse);
			ConfigParams.NumUniverses = FMath::Max(NewConfig.GetNumUniverses(), NumUniverses);

			ProtocolSettings->PreEditChange(UDMXProtocolSettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UDMXProtocolSettings, InputPortConfigs)));
			ProtocolSettings->InputPortConfigs.Add(NewConfig);
			ProtocolSettings->PostEditChange();
		}
		ProtocolSettings->TryUpdateDefaultConfigFile();
	}

	if (bUpdateOutputPort)
	{
		UDMXProtocolSettings* ProtocolSettings = GetMutableDefault<UDMXProtocolSettings>();
		const TSharedPtr<FDMXOutputPort> OutputPort = FDMXPortManager::Get().FindOutputPortByGuid(OutputPortToUpdate.GetPortGuid());
		if (OutputPort.IsValid())
		{
			FDMXOutputPortConfig* OutputPortConfigPtr = Algo::FindByPredicate(ProtocolSettings->OutputPortConfigs, [&OutputPort](const FDMXOutputPortConfig& OutputPortConfig)
				{
					return OutputPortConfig.GetPortGuid() == OutputPort->GetPortGuid();
				});
			if (ensureAlwaysMsgf(OutputPortConfigPtr, TEXT("Cannot find config of Output Port '%s' in Project Settings."), *OutputPort->GetPortName()))
			{
				FDMXOutputPortConfig& PortConfig = *OutputPortConfigPtr;

				FDMXOutputPortConfigParams ConfigParams(PortConfig);
				ConfigParams.LocalUniverseStart = FMath::Min(PortConfig.GetLocalUniverseStart(), FirstUniverse);
				ConfigParams.NumUniverses = FMath::Max(PortConfig.GetNumUniverses(), NumUniverses);

				PortConfig = FDMXOutputPortConfig(PortConfig.GetPortGuid(), ConfigParams);
			}
		}
		else
		{
			FDMXOutputPortConfig NewConfig;
			FDMXOutputPortConfigParams ConfigParams(NewConfig);
			ConfigParams.LocalUniverseStart = FMath::Min(NewConfig.GetLocalUniverseStart(), FirstUniverse);
			ConfigParams.NumUniverses = FMath::Max(NewConfig.GetNumUniverses(), NumUniverses);

			ProtocolSettings->PreEditChange(UDMXProtocolSettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UDMXProtocolSettings, OutputPortConfigs)));
			ProtocolSettings->OutputPortConfigs.Add(NewConfig);
			ProtocolSettings->PostEditChange();
		}
		ProtocolSettings->TryUpdateDefaultConfigFile();
	}

	FDMXPortManager::Get().UpdateFromProtocolSettings();
}
