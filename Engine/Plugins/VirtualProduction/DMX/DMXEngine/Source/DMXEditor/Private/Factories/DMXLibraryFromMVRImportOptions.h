// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"

#include "IO/DMXInputPortReference.h"
#include "IO/DMXOutputPortReference.h"

#include "DMXLibraryFromMVRImportOptions.generated.h"

class UDMXLibrary;


/** 
 * Import Options when importing an MVR File as DMX Library.
 *
 * Note:
 * - A MVR File Path and Name is expected to be set.
 * - bCancelled has to be set to true when the user cancels import while Options here are presented.
 */
UCLASS(Config = EditorPerProjectUserSettings)
class UDMXLibraryFromMVRImportOptions
    : public UObject
{
	GENERATED_BODY()

public:
	/** Applies the options to the project (e.g. port changes). */
	void ApplyOptions(UDMXLibrary* DMXLibrary);

	/** If true, show options for Reimport */
	UPROPERTY()
	bool bIsReimport = false;

	/** Set to true if the user decided to cancel the import while being presented Options here */
	bool bCancelled = false;

	/** File Path and Name of the MVR file for which these options are displayed */
	FString Filename;

	/** If checked, creates a new DMX Library */
	UPROPERTY(EditAnywhere, Category = "Reimport")
	bool bCreateNewDMXLibrary = true;

	/** Imports the MVR into the specified DMX Library. Note this will not merge, but clear Patches in the DMX Library. */
	UPROPERTY(EditAnywhere, Category = "Reimport", Meta = (DisplayName = "Import into DMX Library", EditCondition = "!bCreateNewDMXLibrary", EditConditionHides))
	TObjectPtr<UDMXLibrary> ImportIntoDMXLibrary;

	/** If checked, reimports GDTF that were previously imported */
	UPROPERTY(EditAnywhere, Category = "Reimport")
	bool bReimportExisitingGDTFs = true;

	/** If checked, updates the specified Input Port to span the Universe range of all Patches in the MVR file, or creates a new Input Port if none exist. */
	UPROPERTY(Config, EditAnywhere, Category = "Ports")
	bool bUpdateInputPort = true;

	/** The Input Port which is updated */
	UPROPERTY(EditAnywhere, Category = "Ports", Meta = (ShowOnlyInnerProperties, EditCondition = "bUpdateInputPort", EditConditionHides))
	FDMXInputPortReference InputPortToUpdate;

	/** If checked, updates the specified Ouput Port to span the Universe range of all Patches in the MVR file, or creates a new Output Port if none exist. */
	UPROPERTY(Config, EditAnywhere, Category = "Ports")
	bool bUpdateOutputPort = true;

	/** The Output Port which is updated */
	UPROPERTY(EditAnywhere, Category = "Ports", Meta = (ShowOnlyInnerProperties, EditCondition = "bUpdateOutputPort", EditConditionHides))
	FDMXOutputPortReference OutputPortToUpdate;
};
