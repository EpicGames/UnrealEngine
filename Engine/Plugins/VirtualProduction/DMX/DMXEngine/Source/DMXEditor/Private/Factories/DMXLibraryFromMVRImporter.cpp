// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/DMXLibraryFromMVRImporter.h"

#include "Algo/Find.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "DMXEditorLog.h"
#include "DMXGDTFToFixtureTypeConverter.h"
#include "DMXZipper.h"
#include "Factories/DMXGDTFFactory.h"
#include "Factories/DMXLibraryFromMVRImportOptions.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IMainFrameModule.h"
#include "Layout/SlateRect.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/DMXGDTFAssetImportData.h"
#include "Library/DMXImportGDTF.h"
#include "Library/DMXLibrary.h"
#include "MVR/DMXMVRAssetImportData.h"
#include "MVR/DMXMVRGeneralSceneDescription.h"
#include "MVR/Types/DMXMVRFixtureNode.h"
#include "ObjectTools.h"
#include "Widgets/SDMXLibraryFromMVRImportOptions.h"
#include "XmlFile.h"

#define LOCTEXT_NAMESPACE "DMXLibraryFromMVRImporter"

namespace UE::DMX::DMXLibraryFromMVRImporter::Private
{
	/** Finds the file corresponding to the GDTF Spec in an MVR Zip file. Returns true if the file was found.  */
	[[nodiscard]] bool FindGDTFFilenameInMVRZip(const TSharedRef<FDMXZipper>& MVRZip, const FString& GDTFSpec, FString& OutGDTFFilename)
	{
		OutGDTFFilename.Reset();

		// Some softwares store the GDTF spec with, some without the .gdtf extension. 
		TArray<FString> Filenames = MVRZip->GetFiles();
		const FString* FilenamePtr = Algo::FindByPredicate(Filenames, [GDTFSpec](const FString& Filename)
			{
				if (GDTFSpec == Filename)
				{
					return true;
				}
				else if (GDTFSpec + TEXT(".gdtf") == Filename)
				{
					return true;
				}
				return false;
			});

		if (FilenamePtr)
		{
			OutGDTFFilename = *FilenamePtr;
			return true;
		}
		return false;
	}

	/** Window that displays the MVR import options */
	class FDMXLibraryFromMVROptionsWindow
	{
	public:
		/** Constructor */
		FDMXLibraryFromMVROptionsWindow(UDMXLibraryFromMVRImportOptions* InOptions)
		{
			if (!ensureAlwaysMsgf(InOptions, TEXT("Cannot create MVR import options window, options are invalid.")))
			{
				return;
			}

			TSharedPtr<SWindow> ParentWindow;
			if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
			{
				IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
				ParentWindow = MainFrame.GetParentWindow();
			}

			const FVector2D WindowSize = FVector2D(512.f, 338.f);

			const FSlateRect WorkAreaRect = FSlateApplicationBase::Get().GetPreferredWorkArea();
			const FVector2D DisplayTopLeft(WorkAreaRect.Left, WorkAreaRect.Top);
			const FVector2D DisplaySize(WorkAreaRect.Right - WorkAreaRect.Left, WorkAreaRect.Bottom - WorkAreaRect.Top);

			const FVector2D WindowPosition = (DisplayTopLeft + (DisplaySize - WindowSize) / 2.0f);
			
			const FText Caption = InOptions->bIsReimport ?
				LOCTEXT("ReimportWindowCaption", "MVR Reimport Options") :
				LOCTEXT("ImportWindowCaption", "MVR Import Options");

			const TSharedRef<SWindow> Window = SNew(SWindow)
				.Title(Caption)
				.SizingRule(ESizingRule::FixedSize)
				.AutoCenter(EAutoCenter::None)
				.ClientSize(WindowSize)
				.ScreenPosition(WindowPosition);

			Window->SetContent
			(
				SNew(SDMXLibraryFromMVRImportOptions, Window, InOptions)
				.MaxWidgetSize(WindowPosition)
			);

			FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);
		}
	};
}

bool FDMXLibraryFromMVRImporter::LoadMVRFile(const FString& InFilename)
{
	Filename = InFilename;
	if (!ensureAlwaysMsgf(FPaths::FileExists(InFilename), TEXT("Cannot import MVR File '%s'. File does not exist."), *InFilename))
	{
		return false;
	}

	// Unzip MVR
	Zip = MakeShared<FDMXZipper>();
	if (!Zip->LoadFromFile(InFilename))
	{
		UE_LOGF(LogDMXEditor, Error, "Cannot read MVR '%ls'. File is not a valid MVR.", *InFilename);
		return false;
	}

	// Create the General Scene Description
	GeneralSceneDescription = CreateGeneralSceneDescription(GetTransientPackage());
	if (!GeneralSceneDescription)
	{
		UE_LOGF(LogDMXEditor, Error, "Cannot read General Scene Description from MVR '%ls'. File is not a valid MVR.", *Filename);
		return false;
	}

	// Remember source data
	UDMXMVRAssetImportData* MVRAssetImportData = GeneralSceneDescription->GetMVRAssetImportData();
	MVRAssetImportData->SetSourceFile(Filename);

	return true;
}

void FDMXLibraryFromMVRImporter::UpdateImportOptionsFromModalWindow(UDMXLibraryFromMVRImportOptions* InOutImportOptions, bool& bOutCancelled)
{
	if (!ensureAlwaysMsgf(InOutImportOptions, TEXT("Invalid Import Options when trying to display an Import Options Window.")))
	{
		return;
	}

	InOutImportOptions->Filename = Filename;

	// The Modal Window blocks the Game Thread and allows users to edit the InOutImportOptions
	using namespace UE::DMX::DMXLibraryFromMVRImporter::Private;
	FDMXLibraryFromMVROptionsWindow OptionsWindow(InOutImportOptions);

	bOutCancelled = InOutImportOptions->bCancelled;
}

void FDMXLibraryFromMVRImporter::Import(UObject* InOuter, const FName& InName, EObjectFlags InFlags, UDMXLibraryFromMVRImportOptions* InImportOptions, UDMXLibrary*& OutDMXLibrary, TArray<UDMXImportGDTF*>& OutGDTFs, bool& bOutCancelled)
{
	check(GeneralSceneDescription);

	OutDMXLibrary = nullptr;
	OutGDTFs.Reset();

	UDMXLibrary* NewDMXLibrary = NewObject<UDMXLibrary>(InOuter, InName, InFlags | RF_Public);
	check(NewDMXLibrary);

	GeneralSceneDescription->Rename(nullptr, NewDMXLibrary);
	NewDMXLibrary->SetMVRGeneralSceneDescription(GeneralSceneDescription);

	const bool bReimportExisitingGDTFs = InImportOptions ? InImportOptions->bReimportExisitingGDTFs : true;
	const TArray<UDMXImportGDTF*> NewGDTFs = CreateGDTFs(InOuter, InFlags, bReimportExisitingGDTFs);

	InitializeDMXLibrary(NewDMXLibrary, NewGDTFs);

	OutDMXLibrary = NewDMXLibrary;
	OutGDTFs = NewGDTFs;
}

void FDMXLibraryFromMVRImporter::Reimport(UDMXLibrary* InDMXLibrary, UDMXLibraryFromMVRImportOptions* InImportOptions, TArray<UDMXImportGDTF*>& OutGDTFs)
{
	check(GeneralSceneDescription);

	OutGDTFs.Reset();

	GeneralSceneDescription->Rename(nullptr, InDMXLibrary);
	InDMXLibrary->SetMVRGeneralSceneDescription(GeneralSceneDescription);

	const bool bReimportExistingGDTFs = InImportOptions ? InImportOptions->bReimportExisitingGDTFs : true;
	const TArray<UDMXImportGDTF*> NewGDTFs = CreateGDTFs(InDMXLibrary->GetOuter(), InDMXLibrary->GetFlags(), bReimportExistingGDTFs);

	InitializeDMXLibrary(InDMXLibrary, NewGDTFs);

	OutGDTFs = NewGDTFs;
}

void FDMXLibraryFromMVRImporter::InitializeDMXLibrary(UDMXLibrary* DMXLibrary, const TArray<UDMXImportGDTF*>& GDTFAssets)
{
	using namespace UE::DMX::GDTF;

	check(DMXLibrary);
	check(Zip.IsValid());
	check(GeneralSceneDescription);

	// Move the General Scene Description to the DMX Library and set it
	GeneralSceneDescription->Rename(nullptr, DMXLibrary);
	DMXLibrary->SetMVRGeneralSceneDescription(GeneralSceneDescription);

	// Get or create a Fixture Type for each GDTF
	const TArray<UDMXEntityFixtureType*> FixtureTypes = DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixtureType>();
	TMap<FString, UDMXEntityFixtureType*> GDTFSpecToFixtureTypeMap;
	TMap<UDMXEntityFixtureType*, FLinearColor> FixtureTypeToColorMap;
	for (UDMXImportGDTF* GDTF : GDTFAssets)
	{
		UDMXGDTFAssetImportData* GDTFAssetImportData = GDTF->GetGDTFAssetImportData();
		if (!GDTFAssetImportData)
		{
			continue;
		}
		const FString GDTFSourceFilename = GDTFAssetImportData->GetFilePathAndName();
		const FString GDTFFilename = FPaths::GetCleanFilename(GDTFSourceFilename);

		UDMXEntityFixtureType* const* ExistingFixtureTypePtr = Algo::FindByPredicate(FixtureTypes, [GDTF](const UDMXEntityFixtureType* FixtureType)
			{
				return FixtureType->GDTFSource == GDTF;
			});
		if (ExistingFixtureTypePtr)
		{
			GDTFSpecToFixtureTypeMap.Add(GDTFFilename, *ExistingFixtureTypePtr);
		}
		else
		{
			FDMXEntityFixtureTypeConstructionParams FixtureTypeConstructionParams;
			FixtureTypeConstructionParams.DMXCategory = FDMXFixtureCategory(FDMXFixtureCategory::GetFirstValue());
			FixtureTypeConstructionParams.ParentDMXLibrary = DMXLibrary;

			UDMXEntityFixtureType* NewFixtureType = UDMXEntityFixtureType::CreateFixtureTypeInLibrary(FixtureTypeConstructionParams, FPaths::GetBaseFilename(GDTFFilename));
			NewFixtureType->GDTFSource = GDTF;

			constexpr bool bUpdateFixtureTypeName = true;
			FDMXGDTFToFixtureTypeConverter::ConvertGDTF(*NewFixtureType, *GDTF, bUpdateFixtureTypeName);

			GDTFSpecToFixtureTypeMap.Add(GDTFFilename, NewFixtureType);
		}
	}

	// Remove Fixture Patches that have no corresponding Fixture Node in the General Scene Description
	TArray<UDMXMVRFixtureNode*> FixtureNodes;
	GeneralSceneDescription->GetFixtureNodes(FixtureNodes);
	TArray<UDMXEntityFixturePatch*> FixturePatches = DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>();
	for (UDMXEntityFixturePatch* FixturePatch : FixturePatches)
	{
		UDMXMVRFixtureNode** CorrespondingFixtureNodePtr = Algo::FindByPredicate(FixtureNodes, [FixturePatch](const UDMXMVRFixtureNode* FixtureNode)
			{
				return FixturePatch->GetMVRFixtureUUID() == FixtureNode->UUID;
			});
		if (!CorrespondingFixtureNodePtr)
		{
			UDMXEntityFixturePatch::RemoveFixturePatchFromLibrary(FDMXEntityFixturePatchRef(FixturePatch));
		}
	}
	FixturePatches = DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>();

	// Update or create new Fixture Patches for the MVR Fixtures
	for (const UDMXMVRFixtureNode* FixtureNode : FixtureNodes)
	{
		if (!FixtureNode)
		{
			continue;
		}

		// Find the GDTF Filename, irregardless if it has the .gdtf extension
		FString GDTFFilename = FixtureNode->GDTFSpec;
		if (!GDTFSpecToFixtureTypeMap.Contains(GDTFFilename))
		{
			GDTFFilename = FixtureNode->GDTFSpec + TEXT(".gdtf");
		}
		if (!GDTFSpecToFixtureTypeMap.Contains(GDTFFilename))
		{
			continue;
		}

		UDMXEntityFixtureType* FixtureType = GDTFSpecToFixtureTypeMap[GDTFFilename];
		if (!FixtureTypeToColorMap.Contains(FixtureType))
		{
			FLinearColor FixtureTypeColor = FLinearColor::MakeRandomColor();

			// Avoid dominant red values for a bit more of a professional feel
			if (FixtureTypeColor.R > 0.75f)
			{
				FixtureTypeColor.R = FMath::Abs(FixtureTypeColor.R - 1.0f);
			}

			FixtureTypeToColorMap.Add(FixtureType, FixtureTypeColor);
		}

		int32 ActiveModeIndex = FixtureType->Modes.IndexOfByPredicate([FixtureNode](const FDMXFixtureMode& Mode)
			{
				const FString TrimmedModeName = Mode.ModeName.TrimStartAndEnd();
				const FString TrimmedGDTFModeName = FixtureNode->GDTFMode.TrimStartAndEnd();
				return TrimmedModeName.Equals(TrimmedGDTFModeName);
			});

		if (ActiveModeIndex != INDEX_NONE)
		{
			const int32 DesiredFixtureID = [FixtureNode]()
				{
					int32 Value;
					return LexTryParseString(Value, *FixtureNode->FixtureID) ? Value : 1;
				}();

			UDMXEntityFixturePatch* const* ExistingFixturePatchPtr = Algo::FindByPredicate(FixturePatches, [FixtureNode](const UDMXEntityFixturePatch* FixturePatch)
				{
					return FixturePatch->GetMVRFixtureUUID() == FixtureNode->UUID;
				});
			if (ExistingFixturePatchPtr)
			{
				// Update the existing patch (matching MVR UUID) if possible
				UDMXEntityFixturePatch& ExistingFixturePatch = **ExistingFixturePatchPtr;

				ExistingFixturePatch.SetFixtureType(FixtureType);
				ExistingFixturePatch.GenerateFixtureID(DesiredFixtureID);
				ExistingFixturePatch.SetActiveModeIndex(ActiveModeIndex);
				ExistingFixturePatch.SetUniverseID(FixtureNode->GetUniverseID());
				ExistingFixturePatch.SetStartingChannel(FixtureNode->GetStartingChannel());
				ExistingFixturePatch.EditorColor = FixtureTypeToColorMap.FindChecked(FixtureType);
			}
			else
			{
				// Create a new patch
				FDMXEntityFixturePatchConstructionParams FixturePatchConstructionParams;
				FixturePatchConstructionParams.ActiveMode = ActiveModeIndex;
				FixturePatchConstructionParams.FixtureTypeRef = FDMXEntityFixtureTypeRef(FixtureType);
				FixturePatchConstructionParams.UniverseID = FixtureNode->GetUniverseID();
				FixturePatchConstructionParams.StartingAddress = FixtureNode->GetStartingChannel();
				FixturePatchConstructionParams.MVRFixtureUUID = FixtureNode->UUID;
				FixturePatchConstructionParams.DefaultTransform = FixtureNode->GetTransformAbsolute();

				UDMXEntityFixturePatch* FixturePatch = UDMXEntityFixturePatch::CreateFixturePatchInLibrary(FixturePatchConstructionParams, FixtureNode->Name);
				FixturePatch->GenerateFixtureID(DesiredFixtureID);
				FixturePatch->EditorColor = FixtureTypeToColorMap.FindChecked(FixtureType);
			}
		}
		else
		{
			UE_LOGF(LogDMXEditor, Warning, "Skipped creating a Fixture Patch for '%ls', cannot find Mode '%ls' in '%ls'.", *FixtureNode->Name, *FixtureNode->GDTFMode, *FixtureNode->GDTFSpec)
		}
	}

	DMXLibrary->UpdateGeneralSceneDescription();
}

void FDMXLibraryFromMVRImporter::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(GeneralSceneDescription);
}

UDMXMVRGeneralSceneDescription* FDMXLibraryFromMVRImporter::CreateGeneralSceneDescription(UObject* Outer) const
{
	if (!Zip.IsValid())
	{
		return nullptr;
	}

	TArray64<uint8> XMLData;
	if (!Zip->GetFileContent(TEXT("GeneralSceneDescription.xml"), XMLData))
	{
		return nullptr;
	}

	// MVR implicitly adpots UTF-8 encoding of Xml Files by adopting the GDTF standard (DIN-15800).
	// Content is NOT null-terminated; we need to specify lengths here.
	const FUTF8ToTCHAR TCHARData(reinterpret_cast<const ANSICHAR*>(XMLData.GetData()), XMLData.Num());

	const TSharedRef<FXmlFile> GeneralSceneDescriptionXml = MakeShared<FXmlFile>(FString::ConstructFromPtrSize(TCHARData.Get(), TCHARData.Length()), EConstructMethod::ConstructFromBuffer);

	UDMXMVRGeneralSceneDescription* NewGeneralSceneDescription = UDMXMVRGeneralSceneDescription::CreateFromXmlFile(GeneralSceneDescriptionXml, Outer, NAME_None);

	return NewGeneralSceneDescription;
}

TArray<UDMXImportGDTF*> FDMXLibraryFromMVRImporter::CreateGDTFs(UObject* InParent, EObjectFlags InFlags, bool bReimportExisting)
{
	if (!GeneralSceneDescription || !Zip.IsValid())
	{
		return TArray<UDMXImportGDTF*>();
	}

	const FString Path = FPaths::GetPath(InParent->GetName()) + TEXT("/GDTFs");

	TArray<FAssetData> ExistingGDTFAssets;
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().GetAssetsByPath(FName(Path), ExistingGDTFAssets);

	TArray<UDMXImportGDTF*> ExistingGDTFs;
	for (const FAssetData& AssetData : ExistingGDTFAssets)
	{
		if (UDMXImportGDTF* GDTF = Cast<UDMXImportGDTF>(AssetData.GetAsset()))
		{
			ExistingGDTFs.AddUnique(GDTF);
		}
	}

	UDMXGDTFFactory* GDTFFactory = NewObject<UDMXGDTFFactory>();
	TArray<UDMXImportGDTF*> ImportedGDTFs;
	TArray<FString> ImportedGDTFNames;
	if (ExistingGDTFs.Num() > 0 && bReimportExisting)
	{
		for (UDMXImportGDTF* GDTFAsset : ExistingGDTFs)
		{
			if (GDTFFactory->Reimport(GDTFAsset) == EReimportResult::Succeeded)
			{
				ImportedGDTFs.Add(GDTFAsset);

				if (UDMXGDTFAssetImportData* GDTFAssetImportData = GDTFAsset->GetGDTFAssetImportData())
				{
					const FString SourceFilename = FPaths::GetCleanFilename(GDTFAssetImportData->GetFilePathAndName());
					ImportedGDTFNames.Add(SourceFilename);
				}
			}
		}
	}

	// Import GDTFs that aren't yet imported in the reimport procedure
	TArray<UDMXMVRFixtureNode*> FixtureNodes;
	GeneralSceneDescription->GetFixtureNodes(FixtureNodes);
	for (const UDMXMVRFixtureNode* FixtureNode : FixtureNodes)
	{
		// Don't import the same GDTF twice
		if (!ImportedGDTFNames.Contains(FixtureNode->GDTFSpec))
		{
			using namespace UE::DMX::DMXLibraryFromMVRImporter::Private;

			FString GDTFFilename;
			if (!FindGDTFFilenameInMVRZip(Zip.ToSharedRef(), FixtureNode->GDTFSpec, GDTFFilename))
			{
				UE_LOGF(LogDMXEditor, Warning, "Cannot find a GDTF file that corresponds to GDTF Spec '%ls' of Fixture Node '%ls'. Ignoring Fixture.", *FixtureNode->GDTFSpec, *FixtureNode->Name)
				{
					continue;
				}
			}

			const FDMXZipper::FDMXScopedUnzipToTempFile ScopedUnzipGDTF(Zip.ToSharedRef(), GDTFFilename);
			if (!ScopedUnzipGDTF.TempFilePathAndName.IsEmpty())
			{
				constexpr bool bRemovePathFromDesiredName = true;
				const FString BaseFileName = FPaths::GetBaseFilename(ScopedUnzipGDTF.TempFilePathAndName, bRemovePathFromDesiredName);
				FString AssetName = ObjectTools::SanitizeObjectName(BaseFileName);

				FString GDTFPackageName;
				FString GDTFName;
				FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
				AssetToolsModule.Get().CreateUniqueAssetName(Path / AssetName, TEXT(""), GDTFPackageName, GDTFName);

				UPackage* Package = CreatePackage(*GDTFPackageName);
				check(Package);
				Package->FullyLoad();

				bool bCancelled;
				UObject* NewGDTFObject = GDTFFactory->FactoryCreateFile(UDMXImportGDTF::StaticClass(), Package, *GDTFName, InFlags | RF_Public, ScopedUnzipGDTF.TempFilePathAndName, nullptr, GWarn, bCancelled);

				if (UDMXImportGDTF* NewGDTF = Cast<UDMXImportGDTF>(NewGDTFObject))
				{
					ImportedGDTFs.Add(NewGDTF);
					ImportedGDTFNames.Add(FixtureNode->GDTFSpec);

					FAssetRegistryModule::AssetCreated(NewGDTF);
					Package->MarkPackageDirty();
				}
			}
		}
	}

	return ImportedGDTFs;
}

#undef LOCTEXT_NAMESPACE
