// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/GCObject.h"

class UDMXImportGDTF;
class UDMXLibrary;
class UDMXMVRAssetImportData;
class UDMXMVRGeneralSceneDescription;
class UDMXLibraryFromMVRImportOptions;
class FDMXZipper;

enum EObjectFlags : uint32;


/**
 * Imports an MVR file into a DMX Library. 
 * 
 * 1. LoadMVRFile.
 * 2. CreateGDTFs
 * 3. InitializeDMXLibrary
 * 
 * Does not offer any merge options with an existing DMX Library.
 * To merge an MVR file into an existing DMX Library, import the MVR file into a new DMX Library, then merge it with the existing.
 */
class FDMXLibraryFromMVRImporter :
	public FGCObject
{
public:
	/** Loads the MVR File */
	[[nodiscard]] bool LoadMVRFile(const FString& InFilename);

	/** Creates a modal window that allows the user to change import options. */
	void UpdateImportOptionsFromModalWindow(UDMXLibraryFromMVRImportOptions* InOutImportOptions, bool& bOutCancelled);

	/** 
	 * Imports an MVR File into a newly created DMX Library. Creates GDTF assets and reimports if specified so in ImportOptions.
	 *
	 * @param InOuter			The outer in which the new objects will be created
	 * @param InName			The name of the DMX Library (GDTFs will be auto named)
	 * @param InFlag			Flags with which the new objects are constructed
	 * @param ImportOptions		Can be null. If set, displays a dialog where the user can configure the import.
	 * @param OutDMXLibrary		The resulting DMX Library
	 * @param OutGDTFs			The resulting GDTFs
	 * #param bOutCancelled		True if the import was cancelled
	 */
	void Import(UObject* InOuter, const FName& InName, EObjectFlags InFlags, UDMXLibraryFromMVRImportOptions* ImportOptions, UDMXLibrary*& OutDMXLibrary, TArray<UDMXImportGDTF*>& OutGDTFs, bool& bOutCancelled);

	/**
	 * Imports an MVR File into an existing DMX Library
	 *
	 * @param InOuter			The outer in which the new objects will be created
	 * @param InName			The name of the DMX Library (GDTFs will be auto named)
	 * @param InFlag			Flags with which the new objects are constructed
	 * @param ImportOptions		Can be null. If set, displays a dialog where the user can configure the import.
	 * @param OutDMXLibrary		The resulting DMX Library
	 * @param OutGDTFs			The resulting GDTFs
	 * #param bOutCancelled		True if the import was cancelled
	 */

	void Reimport(UDMXLibrary* InDMXLibrary, UDMXLibraryFromMVRImportOptions* InImportOptions, TArray<UDMXImportGDTF*>& OutGDTFs);

protected:
	//~ Begin FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FDMXLibraryFromMVRImporter");
	}
	//~ End FGCObject interface

private:
	/** Creates a General Scene Description from the MVR zip. Returns the General Scene Description or nullptr if not a valid MVR zip. */
	UDMXMVRGeneralSceneDescription* CreateGeneralSceneDescription(UObject* Outer) const;

	/** Creates GDTF assets from the MVR */
	TArray<UDMXImportGDTF*> CreateGDTFs(UObject* InParent, EObjectFlags InFlags, bool bReimportExisting);

	/** Initializes the DMX Library from the MVR data present and specified GDTFs */
	void InitializeDMXLibrary(UDMXLibrary* DMXLibrary, const TArray<UDMXImportGDTF*>& GDTFs);

	/** Temporary General Scene Description of the MVR File, or nullptr if not a valid MVR */
	TObjectPtr<UDMXMVRGeneralSceneDescription> GeneralSceneDescription = nullptr;

	/** Zip that holds the data */
	TSharedPtr<FDMXZipper> Zip;

	/** File name of the MVR File */
	FString Filename;
};
