// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/DMXMVRSceneActorFactory.h"

#include "Engine/Level.h"
#include "Engine/StaticMeshActor.h"
#include "Library/DMXLibrary.h"
#include "MVR/DMXMVRSceneActor.h"

#include "EngineUtils.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"


UDMXMVRSceneActorFactory::UDMXMVRSceneActorFactory()
{
	DisplayName = NSLOCTEXT("DMXMVRSceneActorFactory", "DMXMVRSceneActorFactoryDisplayName", "MVR Scene Actor");
	NewActorClass = ADMXMVRSceneActor::StaticClass();
}

bool UDMXMVRSceneActorFactory::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (AssetData.IsValid())
	{
		UClass* AssetClass = AssetData.GetClass();
		if (AssetClass && AssetClass == UDMXLibrary::StaticClass())
		{
			return true;
		}
	}

	return false;
}

AActor* UDMXMVRSceneActorFactory::SpawnActor(UObject* Asset, ULevel* InLevel, const FTransform& Transform, const FActorSpawnParameters& InSpawnParams)
{
	UDMXLibrary* DMXLibrary = Cast<UDMXLibrary>(Asset);

	if (!DMXLibrary)
	{
		return nullptr;
	}

	if (InSpawnParams.ObjectFlags & RF_Transient)
	{
		// This is a hack for drag and drop so that we don't spawn all the actors for the preview actor since it gets deleted right after.

		FActorSpawnParameters SpawnInfo(InSpawnParams);
		SpawnInfo.OverrideLevel = InLevel;
		AStaticMeshActor* DragActor = Cast<AStaticMeshActor>(InLevel->GetWorld()->SpawnActor(AStaticMeshActor::StaticClass(), &Transform, SpawnInfo));
		DragActor->GetStaticMeshComponent()->SetStaticMesh(Cast< UStaticMesh >(FSoftObjectPath(TEXT("StaticMesh'/Engine/EditorMeshes/EditorSphere.EditorSphere'")).TryLoad()));
		DragActor->SetActorScale3D(FVector(0.1f));

		return DragActor;
	}
	else 
	{
		if (!InLevel)
		{
			return nullptr;
		}

		UWorld* World = InLevel->GetWorld();
		if (!World)
		{
			return nullptr;
		}

		if(IsDMXLibraryAlreadySpawned(World, DMXLibrary))
		{ 
			NotifyDMXLibraryAlreadySpawned();

			return nullptr;
		}

		ADMXMVRSceneActor* MVRSceneActor = World->SpawnActor<ADMXMVRSceneActor>(ADMXMVRSceneActor::StaticClass());
		if (MVRSceneActor)
		{
			MVRSceneActor->SetDMXLibrary(DMXLibrary);

			return MVRSceneActor;
		}
	}

	return nullptr;
}

bool UDMXMVRSceneActorFactory::IsDMXLibraryAlreadySpawned(UWorld* World, UDMXLibrary* DMXLibrary) const
{
	if (World && DMXLibrary)
	{
		for (TActorIterator<ADMXMVRSceneActor> It(World, ADMXMVRSceneActor::StaticClass()); It; ++It)
		{
			ADMXMVRSceneActor* MVRSceneActor = *It;
			if (MVRSceneActor && MVRSceneActor->GetDMXLibrary() == DMXLibrary)
			{
				return true;
			}
		}
	}
	return false;
}

void UDMXMVRSceneActorFactory::NotifyDMXLibraryAlreadySpawned()
{
	if (!DMXLibraryAlreadySpawnedNotification.IsValid())
	{
		FNotificationInfo NotificationInfo(NSLOCTEXT("DMXMVRSceneActorFactory", "DMXLibraryAlreadySpawnedInSceneNotification", "DMX Library is already spawned in this scene."));
		NotificationInfo.ExpireDuration = 5.f;
		DMXLibraryAlreadySpawnedNotification = FSlateNotificationManager::Get().AddNotification(NotificationInfo);
	}
}
