// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "ActorFactories/ActorFactory.h"
#include "UObject/ObjectMacros.h"

#include "DMXMVRSceneActorFactory.generated.h"

class UDMXLibrary;

class AActor;
struct FAssetData;
class SNotificationItem;
class UWorld;


UCLASS()
class UDMXMVRSceneActorFactory
	: public UActorFactory
{
	GENERATED_BODY()

public:
	UDMXMVRSceneActorFactory();

	//~ Begin UActorFactory Interface
	virtual bool CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg) override;
	virtual AActor* SpawnActor(UObject* Asset, ULevel* InLevel, const FTransform& Transform, const FActorSpawnParameters& InSpawnParams) override;
	//~ End UActorFactory Interface

private:
	/** Returns true if the DMX Library is already spawned as an MVR Scene Actor in the world */
	bool IsDMXLibraryAlreadySpawned(UWorld* World, UDMXLibrary* DMXLibrary) const;

	/** Creates a notification that the DMX Library is already spawned as an MVR Scene Actor in the world */
	void NotifyDMXLibraryAlreadySpawned();

	/** Notification displayed when a DMX Library was already spawned */
	TWeakPtr<SNotificationItem> DMXLibraryAlreadySpawnedNotification;
};
