// Copyright Epic Games, Inc. All Rights Reserved.

#include "FixturePatchAutoAssignUtility.h"

#include "Algo/MaxElement.h"
#include "DMXEditor.h"
#include "DMXFixturePatchSharedData.h"
#include "DMXProtocolConstants.h"
#include "IO/DMXInputPort.h"
#include "IO/DMXOutputPort.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXLibrary.h"

#define LOCTEXT_NAMESPACE "FixturePatchAutoAssignUtility"

namespace UE::DMXEditor::AutoAssign::Private
{
	FAutoAssignElement::FAutoAssignElement(int64 LowerBoundInclusive, int64 UpperBoundExclusive)
		: AbsoluteRange(TRange<int64>(LowerBoundInclusive, UpperBoundExclusive))
	{}

	FAutoAssignElement::FAutoAssignElement(const TRange<int64>& InAbsoluteRange)
		: AbsoluteRange(InAbsoluteRange)
	{}

	FAutoAssignElement::FAutoAssignElement(UDMXEntityFixturePatch& FixturePatch)
		: WeakFixturePatch(&FixturePatch)
	{
		const int64 AbsoluteStartingChannel = FixturePatch.GetUniverseID() * DMX_UNIVERSE_SIZE + FixturePatch.GetStartingChannel() - 1;
		const int64 AbsoluteEndingChannel = AbsoluteStartingChannel + FixturePatch.GetChannelSpan();
		AbsoluteRange = TRange<int64>(AbsoluteStartingChannel, AbsoluteEndingChannel);
	}

	int32 FAutoAssignElement::GetUniverse() const
	{
		const int64 Universe = AbsoluteRange.GetLowerBoundValue() / DMX_UNIVERSE_SIZE;
		return FMath::Clamp(Universe, 1, std::numeric_limits<int32>::max());
	}

	int32 FAutoAssignElement::GetChannel() const
	{
		const int64 Channel = AbsoluteRange.GetLowerBoundValue() % DMX_UNIVERSE_SIZE;
		return FMath::Clamp(Channel, 1, std::numeric_limits<int32>::max());
	}

	int64 FAutoAssignElement::GetSize() const
	{
		return AbsoluteRange.Size<int64>();
	}

	int64 FAutoAssignElement::GetLowerBoundValue() const
	{
		return AbsoluteRange.GetLowerBoundValue();
	}

	int64 FAutoAssignElement::GetUpperBoundValue() const
	{
		return AbsoluteRange.GetUpperBoundValue();
	}

	void FAutoAssignElement::SetAbsoluteStartingChannel(int64 NewStartingChannel)
	{
		const int64 NewEndingAddress = NewStartingChannel + AbsoluteRange.GetUpperBoundValue() - AbsoluteRange.GetLowerBoundValue();
		AbsoluteRange = TRange<int64>(NewStartingChannel, NewEndingAddress);
	}

	void FAutoAssignElement::AlignAfter(const TSharedRef<FAutoAssignElement>& Other)
	{
		const bool bFitsCurrentUniverse = (Other->GetUpperBoundValue() + GetSize() - 1) / DMX_UNIVERSE_SIZE == Other->GetUpperBoundValue() / DMX_UNIVERSE_SIZE;
		if (bFitsCurrentUniverse)
		{
			SetAbsoluteStartingChannel(Other->GetUpperBoundValue());
		}
		else
		{
			// Assign to next universe
			const int64 NewStartingChannel = (Other->GetUpperBoundValue() + DMX_UNIVERSE_SIZE) / DMX_UNIVERSE_SIZE * DMX_UNIVERSE_SIZE;
			SetAbsoluteStartingChannel(NewStartingChannel);
		}
	}

	void FAutoAssignElement::ApplyToPatch()
	{
		if (UDMXEntityFixturePatch* Patch = WeakFixturePatch.Get())
		{
			// Transition back to the 1-512 range
			int64 Universe = (GetLowerBoundValue() + 1) / DMX_UNIVERSE_SIZE;
			int64 Channel =  (GetLowerBoundValue() + 1) % DMX_UNIVERSE_SIZE;
			if (Channel == 0)
			{
				Channel = 512;
				--Universe;
			}
			if (!ensureMsgf(Universe >= 1 && Channel >= 1 && Channel <= DMX_UNIVERSE_SIZE, TEXT("Invalid universe and channel resulting from auto assign. Channels not applied for '%s'."), *Patch->Name))
			{
				return;
			}

			const int32 UniverseSafe = FMath::Clamp(Universe, 1, std::numeric_limits<int32>::max());
			const int32 ChannelSafe = FMath::Clamp(Channel, 1, std::numeric_limits<int32>::max());

			Patch->SetUniverseID(UniverseSafe);
			Patch->SetStartingChannel(ChannelSafe);
		}
	}

	UDMXEntityFixturePatch* FAutoAssignElement::GetFixturePatch() const
	{
		return WeakFixturePatch.Get();
	}

	int32 FAutoAssignElementsUtility::Assign(TArray<TSharedRef<FAutoAssignElement>> ElementsToAssign, int32 AssignToUniverse, int32 AssignToChannel)
	{
		if (ElementsToAssign.IsEmpty())
		{
			return 1;
		}

		if (!EnsureValidPatchElements(ElementsToAssign))
		{
			return 1;
		}

		Algo::StableSortBy(ElementsToAssign, &FAutoAssignElement::GetLowerBoundValue);

		const int64 AssignToChannelAbsolute = FMath::Max(AssignToUniverse * DMX_UNIVERSE_SIZE + AssignToChannel - 1, (int64)DMX_UNIVERSE_SIZE);
		
		int64 Offset = AssignToChannelAbsolute - ElementsToAssign[0]->GetLowerBoundValue();
		for (const TSharedRef<FAutoAssignElement>& Element : ElementsToAssign)
		{
			const bool bFitsCurrentUniverse = (Element->GetLowerBoundValue() + Offset) / DMX_UNIVERSE_SIZE == (Element->GetUpperBoundValue() - 1 + Offset) / DMX_UNIVERSE_SIZE;
			if (bFitsCurrentUniverse)
			{
				const int64 NewStartingChannel = Element->GetLowerBoundValue() + Offset;
				Element->SetAbsoluteStartingChannel(NewStartingChannel);
			}
			else
			{
				// Assign to next universe
				const int64 OldStartingChannel = Element->GetLowerBoundValue() + Offset;
				const int64 NewStartingChannel = (Element->GetLowerBoundValue() + Offset + DMX_UNIVERSE_SIZE) / DMX_UNIVERSE_SIZE * DMX_UNIVERSE_SIZE;
				Element->SetAbsoluteStartingChannel(NewStartingChannel);

				Offset += NewStartingChannel - OldStartingChannel;
			}

			Element->ApplyToPatch();
		}

		const int64 FirstPatchedUniverse = ElementsToAssign[0]->GetLowerBoundValue() / DMX_UNIVERSE_SIZE;
		return FirstPatchedUniverse;
	}

	void FAutoAssignElementsUtility::Align(TArray<TSharedRef<FAutoAssignElement>> ElementsToAlign, bool bRetainStacks)
	{
		if (ElementsToAlign.IsEmpty())
		{
			return;
		}

		if (!EnsureValidPatchElements(ElementsToAlign))
		{
			return;
		}

		Algo::StableSortBy(ElementsToAlign, &FAutoAssignElement::GetLowerBoundValue);

		// Move all patches into valid universe range
		if (ElementsToAlign[0]->GetLowerBoundValue() < DMX_UNIVERSE_SIZE)
		{
			const int64 Offset = DMX_UNIVERSE_SIZE - ElementsToAlign[0]->GetLowerBoundValue();
			for (const TSharedRef<FAutoAssignElement>& ElementToAlign : ElementsToAlign)
			{
				ElementToAlign->SetAbsoluteStartingChannel(ElementToAlign->GetLowerBoundValue() + Offset);
			}
		}

		if (bRetainStacks)
		{
			// Create groups of stacked elements
			TArray<TArray<TSharedRef<FAutoAssignElement>>> StackedElements;
			for (int32 ElementIndex = 0; ElementIndex < ElementsToAlign.Num(); ElementIndex++)
			{
				const bool bIsStack = [&ElementsToAlign, ElementIndex]()
					{
						if (ElementsToAlign.IsValidIndex(ElementIndex - 1))
						{
							return AreElementsStacked(ElementsToAlign[ElementIndex - 1], ElementsToAlign[ElementIndex]);
						}
						else
						{
							return false;
						}
					}();

				if (bIsStack &&
					ensureMsgf(!StackedElements.IsEmpty(), TEXT("Trying to group auto assign elements into a stack, but cannot find the stack.")))
				{
					StackedElements.Last().Add(ElementsToAlign[ElementIndex]);
				}
				else
				{
					StackedElements.Add(TArray<TSharedRef<FAutoAssignElement>>({ ElementsToAlign[ElementIndex] }));
				}
			}

			// Align stacks of elements
			for (int32 StackIndex = 1; StackIndex < StackedElements.Num(); StackIndex++)
			{
				const TArray<TSharedRef<FAutoAssignElement>>& PreviousStack = StackedElements[StackIndex - 1];
				if (!ensureMsgf(!PreviousStack.IsEmpty(), TEXT("Cannot find elements in previous stack, cannot align elements retaining the stack.")))
				{
					continue;
				}
				const TSharedRef<FAutoAssignElement>& PreviousElement = PreviousStack[0];
				
				const TArray<TSharedRef<FAutoAssignElement>>& Stack = StackedElements[StackIndex];
				for (const TSharedRef<FAutoAssignElement>& Element : Stack)
				{
					Element->AlignAfter(PreviousElement);
					Element->ApplyToPatch();
				}
			}
		}
		else
		{
			// Align all elements
			for (int32 ElementIndex = 1; ElementIndex < ElementsToAlign.Num(); ElementIndex++)
			{
				const TSharedRef<FAutoAssignElement>& PreviousElement = ElementsToAlign[ElementIndex - 1];
				const TSharedRef<FAutoAssignElement>& Element = ElementsToAlign[ElementIndex];

				Element->AlignAfter(PreviousElement);
				Element->ApplyToPatch();
			}
		}
	}

	void FAutoAssignElementsUtility::Stack(TArray<TSharedRef<FAutoAssignElement>> ElementsToStack)
	{
		if (ElementsToStack.IsEmpty())
		{
			return;
		}

		if (!EnsureValidPatchElements(ElementsToStack))
		{
			return;
		}

		Algo::StableSortBy(ElementsToStack, &FAutoAssignElement::GetLowerBoundValue);

		// If the largest patch would exceed universe size, offset the stack so the largest patch fits into the universe as well
		const TSharedRef<FAutoAssignElement>* const LargestElement = Algo::MaxElementBy(ElementsToStack, &FAutoAssignElement::GetSize);

		const int64 EndingChannelForLargestExclusive = ElementsToStack[0]->GetLowerBoundValue() + (*LargestElement)->GetSize();
		const bool bOffsetToLargestPatch = (EndingChannelForLargestExclusive - 1) / DMX_UNIVERSE_SIZE != ElementsToStack[0]->GetLowerBoundValue() / DMX_UNIVERSE_SIZE;
		
		const int64 StackOnChannel = [&ElementsToStack, EndingChannelForLargestExclusive, bOffsetToLargestPatch]()
		{
			if (bOffsetToLargestPatch)
			{
				const int64 Offset = EndingChannelForLargestExclusive % DMX_UNIVERSE_SIZE;
				return FMath::Max(ElementsToStack[0]->GetLowerBoundValue() - Offset, (int64)DMX_UNIVERSE_SIZE);
			}
			else
			{
				return FMath::Max(ElementsToStack[0]->GetLowerBoundValue(), (int64)DMX_UNIVERSE_SIZE);
			}
		}();
	
		for (const TSharedRef<FAutoAssignElement>& Element : ElementsToStack)
		{
			Element->SetAbsoluteStartingChannel(StackOnChannel);
			Element->ApplyToPatch();
		}
	}

	void FAutoAssignElementsUtility::SpreadOverUniverses(TArray<TSharedRef<FAutoAssignElement>> ElementsToSpread)
	{
		if (ElementsToSpread.IsEmpty())
		{
			return;
		}

		if (!EnsureValidPatchElements(ElementsToSpread))
		{
			return;
		}
		Algo::StableSortBy(ElementsToSpread, &FAutoAssignElement::GetLowerBoundValue);

		// If the largest patch would exceed universe size, offset the patches so the largest patch fits into the universe as well
		const TSharedRef<FAutoAssignElement>* const LargestElement = Algo::MaxElementBy(ElementsToSpread, &FAutoAssignElement::GetSize);

		const int64 EndingChannelForLargestExclusive = ElementsToSpread[0]->GetLowerBoundValue() + (*LargestElement)->GetSize();
		const bool bOffsetToLargestPatch = (EndingChannelForLargestExclusive - 1) / DMX_UNIVERSE_SIZE != ElementsToSpread[0]->GetLowerBoundValue() / DMX_UNIVERSE_SIZE;

		int64 SpreadOnChannel = [&ElementsToSpread, EndingChannelForLargestExclusive, bOffsetToLargestPatch]()
		{
			if (bOffsetToLargestPatch)
			{
				const int64 Offset = EndingChannelForLargestExclusive % DMX_UNIVERSE_SIZE;
				return FMath::Max(ElementsToSpread[0]->GetLowerBoundValue() - Offset, (int64)DMX_UNIVERSE_SIZE);
			}
			else
			{
				return FMath::Max(ElementsToSpread[0]->GetLowerBoundValue(), (int64)DMX_UNIVERSE_SIZE);
			}
		}();

		for (const TSharedRef<FAutoAssignElement>& Element : ElementsToSpread)
		{
			Element->SetAbsoluteStartingChannel(SpreadOnChannel);
			SpreadOnChannel += DMX_UNIVERSE_SIZE;

			Element->ApplyToPatch();
		}
	}

	void FAutoAssignElementsUtility::AutoAssign(const TArray<TSharedRef<FAutoAssignElement>>& FreeElements, TArray<TSharedRef<FAutoAssignElement>> AutoAssignElements)
	{
		if (AutoAssignElements.IsEmpty())
		{
			return;
		}

		if (!EnsureValidPatchElements(AutoAssignElements))
		{
			return;
		}
		Algo::StableSortBy(AutoAssignElements, &FAutoAssignElement::GetLowerBoundValue);

		constexpr bool bRetainStacks = true;
		Align(AutoAssignElements, bRetainStacks);

		TArray<int64> StartingChannels;
		for (const TSharedRef<FAutoAssignElement>& FreeElement : FreeElements)
		{
			StartingChannels.Reset();
			const int64 AssignToChannelAbsolute = FMath::Max(FreeElement->GetLowerBoundValue(), (int64)DMX_UNIVERSE_SIZE);

			int64 Offset = AssignToChannelAbsolute - AutoAssignElements[0]->GetLowerBoundValue();
			for (const TSharedRef<FAutoAssignElement>& Element : AutoAssignElements)
			{
				const bool bFitsCurrentUniverse = (Element->GetLowerBoundValue() + Offset) / DMX_UNIVERSE_SIZE == (Element->GetUpperBoundValue() - 1 + Offset) / DMX_UNIVERSE_SIZE;
				if (bFitsCurrentUniverse)
				{
					const int64 NewStartingChannel = Element->GetLowerBoundValue() + Offset;
					StartingChannels.Add(NewStartingChannel);
				}
				else
				{
					// Assign to next universe
					const int64 OldStartingChannel = Element->GetLowerBoundValue() + Offset;
					const int64 NewStartingChannel = (Element->GetLowerBoundValue() + Offset + DMX_UNIVERSE_SIZE) / DMX_UNIVERSE_SIZE * DMX_UNIVERSE_SIZE;

					StartingChannels.Add(NewStartingChannel);

					Offset += NewStartingChannel - OldStartingChannel;
				}
			}

			const TRange<int64> AssignedRange(StartingChannels[0], StartingChannels.Last() + AutoAssignElements.Last()->GetSize() - 1);
			if (AssignedRange.Size<int64>() <= FreeElement->GetSize())
			{
				for (int64 ElementIndex = 0; ElementIndex < AutoAssignElements.Num(); ElementIndex++)
				{
					AutoAssignElements[ElementIndex]->SetAbsoluteStartingChannel(StartingChannels[ElementIndex]);
					AutoAssignElements[ElementIndex]->ApplyToPatch();
				}

				return;
			}
		}
	}

	bool FAutoAssignElementsUtility::AreElementsStacked(const TSharedRef<FAutoAssignElement>& FirstElement, const TSharedRef<FAutoAssignElement>& SecondElement)
	{
		if (FirstElement->GetLowerBoundValue() != SecondElement->GetLowerBoundValue() ||
			FirstElement->GetUpperBoundValue() != SecondElement->GetUpperBoundValue())
		{
			return false;
		}

		const UDMXEntityFixturePatch* FirstFixturePatch = FirstElement->GetFixturePatch();
		const UDMXEntityFixturePatch* SecondFixturePatch = SecondElement->GetFixturePatch();

		if (FirstFixturePatch && SecondFixturePatch)
		{
			return 
				FirstFixturePatch->GetFixtureType() &&
				FirstFixturePatch->GetFixtureType() == SecondFixturePatch->GetFixtureType();
		}
		else
		{
			return FirstElement->GetSize() == SecondElement->GetSize();
		}
	}

	bool FAutoAssignElementsUtility::EnsureValidPatchElements(const TArray<TSharedRef<FAutoAssignElement>>& Elements)
	{
		const bool bValidElements = Algo::FindByPredicate(Elements, [](const TSharedRef<FAutoAssignElement>& Element)
			{
				// Find an invalid element
				return 
					Element->GetLowerBoundValue() > Element->GetUpperBoundValue() || 
					Element->GetSize() < 0 || 
					Element->GetSize() > DMX_UNIVERSE_SIZE;
			}) == nullptr;
		if (!ensureMsgf(bValidElements, TEXT("Elements to assign are not in valid DMX range. Ignoring call.")))
		{
			return false;
		}
		return true;
	}
}


namespace UE::DMXEditor::AutoAssign
{
	int32 FAutoAssignUtility::Assign(TArray<UDMXEntityFixturePatch*> FixturePatches, int32 AssignToUniverse, int32 AssignToChannel)
	{
		FAutoAssignUtility Instance;

		using namespace Private;
		const TArray<TSharedRef<FAutoAssignElement>> PatchElements = Instance.CreatePatchElements(FixturePatches);
		return FAutoAssignElementsUtility::Assign(PatchElements, AssignToUniverse, AssignToChannel);
	}

	void FAutoAssignUtility::Align(TArray<UDMXEntityFixturePatch*> FixturePatches)
	{
		FAutoAssignUtility Instance;

		using namespace Private;
		const TArray<TSharedRef<FAutoAssignElement>> PatchElements = Instance.CreatePatchElements(FixturePatches);

		constexpr bool bRetainStacks = false;
		FAutoAssignElementsUtility::Align(PatchElements, bRetainStacks);
	}

	void FAutoAssignUtility::Stack(TArray<UDMXEntityFixturePatch*> FixturePatches)
	{
		FAutoAssignUtility Instance;

		using namespace Private;
		const TArray<TSharedRef<FAutoAssignElement>> PatchElements = Instance.CreatePatchElements(FixturePatches);
		FAutoAssignElementsUtility::Stack(PatchElements);
	}

	void FAutoAssignUtility::SpreadOverUniverses(TArray<UDMXEntityFixturePatch*> FixturePatches)
	{
		FAutoAssignUtility Instance;

		using namespace Private;
		const TArray<TSharedRef<FAutoAssignElement>> PatchElements = Instance.CreatePatchElements(FixturePatches);
		FAutoAssignElementsUtility::SpreadOverUniverses(PatchElements);
	}

	int32 FAutoAssignUtility::AutoAssign(EAutoAssignMode Mode, const TSharedRef<FDMXEditor>& DMXEditor, TArray<UDMXEntityFixturePatch*> FixturePatches, int32 UserDefinedUniverse, int32 UserDefinedChannel)
	{
		FAutoAssignUtility Instance;
		return Instance.AutoAssignInternal(Mode, DMXEditor, FixturePatches, UserDefinedUniverse, UserDefinedChannel);
	}

	int32 FAutoAssignUtility::AutoAssignInternal(EAutoAssignMode Mode, const TSharedRef<FDMXEditor>& DMXEditor, TArray<UDMXEntityFixturePatch*> FixturePatches, int32 UserDefinedUniverse, int32 UserDefinedChannel)
	{
		UDMXLibrary* DMXLibrary = GetDMXLibrary(FixturePatches);
		if (!DMXLibrary)
		{
			return 1;
		}

		using namespace Private;
		const int64 AbsoluteStartingChannel = FindAbsoluteStartingChannel(Mode, DMXEditor , *DMXLibrary, FixturePatches, UserDefinedUniverse, UserDefinedChannel);
		const TArray<TSharedRef<FAutoAssignElement>> FreeElements = CreateFreeElements(*DMXLibrary, FixturePatches, AbsoluteStartingChannel);
		const TArray<TSharedRef<FAutoAssignElement>> PatchElements = CreatePatchElements(FixturePatches);

		FAutoAssignElementsUtility::AutoAssign(FreeElements, PatchElements);

		const TSharedRef<FAutoAssignElement>* const FirstPatchElementPtr = Algo::FindByPredicate(PatchElements, [](const TSharedRef<FAutoAssignElement>& Element)
			{
				return Element->GetLowerBoundValue();
			});
		if (!ensureMsgf(FirstPatchElementPtr, TEXT("Unexpected FAutoAssignElementsUtility::AutoAssign cannot find first patch elements.")))
		{
			return 1;
		}

		return (*FirstPatchElementPtr)->GetLowerBoundValue() / DMX_UNIVERSE_SIZE;
	}

	int64 FAutoAssignUtility::FindAbsoluteStartingChannel(EAutoAssignMode Mode, const TSharedRef<FDMXEditor>& DMXEditor, const UDMXLibrary& DMXLibrary, const TArray<UDMXEntityFixturePatch*>& FixturePatches, int32 UniverseUnderMouse, int32 UserDefinedChannel) const
	{
		if (!ensureMsgf(!FixturePatches.IsEmpty() && !FixturePatches.Contains(nullptr), TEXT("Unexpected no fixture patches or null entry in Fixture Patches. Expected to be handled during FAutoAssignUtility construction.")))
		{
			return DMX_UNIVERSE_SIZE;
		}

		if (!ensureMsgf(
			Mode != EAutoAssignMode::UserDefinedChannel || 
			(UniverseUnderMouse > 0 && UniverseUnderMouse <= DMX_UNIVERSE_SIZE && UserDefinedChannel > 0 && UserDefinedChannel <= DMX_UNIVERSE_SIZE),
			TEXT("Trying to auto assign to channel under mouse, but provided channels are not valid. Skipping auto assign.")))
		{
			return DMX_UNIVERSE_SIZE;
		}

		if (Mode == EAutoAssignMode::UserDefinedChannel)
		{
			return (int64)UniverseUnderMouse * DMX_UNIVERSE_SIZE + UserDefinedChannel - 1;
		}
		else if (Mode == EAutoAssignMode::SelectedUniverse)
		{
			return DMXEditor->GetFixturePatchSharedData()->GetSelectedUniverse() * DMX_UNIVERSE_SIZE;
		}
		else if (Mode == EAutoAssignMode::FirstReachableUniverse)
		{
			const FDMXInputPortSharedRef* const FirstReachableInputPortPtr = Algo::MinElementBy(DMXLibrary.GetInputPorts(), &FDMXInputPort::GetLocalUniverseStart);
			const FDMXOutputPortSharedRef* const FirstReachableOutputPortPtr = Algo::MinElementBy(DMXLibrary.GetOutputPorts(), &FDMXOutputPort::GetLocalUniverseStart);

			if (FirstReachableInputPortPtr && FirstReachableOutputPortPtr)
			{
				return (int64)FMath::Max((*FirstReachableInputPortPtr)->GetLocalUniverseStart(), (*FirstReachableOutputPortPtr)->GetLocalUniverseStart()) * DMX_UNIVERSE_SIZE;
			}
			else if (FirstReachableInputPortPtr)
			{
				return  (int64)(*FirstReachableInputPortPtr)->GetLocalUniverseStart() * DMX_UNIVERSE_SIZE;
			}
			else if (FirstReachableOutputPortPtr)
			{
				return  (int64)(*FirstReachableOutputPortPtr)->GetLocalUniverseStart() * DMX_UNIVERSE_SIZE;
			}
			
			// If there's no valid ports, assign to 1.1
			return DMX_UNIVERSE_SIZE;
		}
		else if (Mode == EAutoAssignMode::AfterLastPatchedUniverse)
		{
			const TArray<UDMXEntityFixturePatch*> OtherPatches = FindOtherPatchesInLibrary(DMXLibrary, FixturePatches);

			const UDMXEntityFixturePatch* const* LastOtherFixturePatchPtr = Algo::MaxElementBy(OtherPatches, [](const UDMXEntityFixturePatch* OtherPatch)
				{
					return OtherPatch->GetUniverseID() * DMX_UNIVERSE_SIZE + OtherPatch->GetStartingChannel() + OtherPatch->GetChannelSpan();
				});

			if (LastOtherFixturePatchPtr)
			{
				return  (int64)(*LastOtherFixturePatchPtr)->GetUniverseID() * DMX_UNIVERSE_SIZE + (*LastOtherFixturePatchPtr)->GetStartingChannel() + (*LastOtherFixturePatchPtr)->GetChannelSpan() - 1;
			}
			else
			{
				return DMX_UNIVERSE_SIZE;
			}
		}
		else
		{
			checkf(0, TEXT("Unhandled enum value in FAutoAssignUtility::FindAbsoluteStartingChannel"));
			return DMX_UNIVERSE_SIZE;
		}
	}	
	
	UDMXLibrary* FAutoAssignUtility::GetDMXLibrary(TArray<UDMXEntityFixturePatch*> FixturePatches) const
	{
		if (FixturePatches.IsEmpty())
		{
			return nullptr;
		}
		FixturePatches.Remove(nullptr);

		// Ensure all patches reside in the same library
		UDMXLibrary* DMXLibrary = FixturePatches[0]->GetParentLibrary();
		if (!ensureMsgf(DMXLibrary, TEXT("Trying to auto assign fixture patches but the DMX Library of its patches is not valid.")))
		{
			return nullptr;
		}
		check(DMXLibrary);

		for (const UDMXEntityFixturePatch* FixturePatch : FixturePatches)
		{
			if (!ensureMsgf(FixturePatch->GetParentLibrary() == DMXLibrary, TEXT("Trying to auto assign fixture patches, but the patches don't share a common DMX Library. This is not supported")))
			{
				return nullptr;
			}
		}

		return DMXLibrary;
	}

	TArray<TSharedRef<Private::FAutoAssignElement>> FAutoAssignUtility::CreateFreeElements(UDMXLibrary& DMXLibrary, const TArray<UDMXEntityFixturePatch*>& FixturePatches, int64 AbsoluteStartingChannel) const
	{
		using namespace Private;
		TArray<UDMXEntityFixturePatch*> OtherPatchesInLibrary = FindOtherPatchesInLibrary(DMXLibrary, FixturePatches);
		Algo::StableSortBy(OtherPatchesInLibrary, [](const UDMXEntityFixturePatch* FixturePatch)
			{
				return FixturePatch->GetUniverseID() * DMX_UNIVERSE_SIZE + FixturePatch->GetStartingChannel();
			});

		TArray<FAutoAssignElement> OtherPatchElements;
		for (UDMXEntityFixturePatch* FixturePatch : OtherPatchesInLibrary)
		{
			if (IsValid(FixturePatch))
			{
				OtherPatchElements.Add(FAutoAssignElement(*FixturePatch));
			}
		}

		TArray<TSharedRef<FAutoAssignElement>> FreeElements;
		int64 StartingChannelOfFreeRange = FMath::Max(AbsoluteStartingChannel, (int64)DMX_UNIVERSE_SIZE);
		for (const FAutoAssignElement& OtherPatchElement : OtherPatchElements)
		{
			if (StartingChannelOfFreeRange < OtherPatchElement.GetUpperBoundValue())
			{
				const TRange<int64> FreeRange(StartingChannelOfFreeRange, OtherPatchElement.GetLowerBoundValue() - 1);
				FreeElements.Add(MakeShared<FAutoAssignElement>(FreeRange));

				StartingChannelOfFreeRange = OtherPatchElement.GetUpperBoundValue();
			}
		}
		const TSharedRef<FAutoAssignElement> LastFreeElement = MakeShared<FAutoAssignElement>(TRange<int64>(StartingChannelOfFreeRange, std::numeric_limits<int64>::max()));
		FreeElements.Add(LastFreeElement);

		return FreeElements;
	}

	TArray<TSharedRef<Private::FAutoAssignElement>> FAutoAssignUtility::CreatePatchElements(TArray<UDMXEntityFixturePatch*> FixturePatches) const
	{
		using namespace Private;
		TArray<TSharedRef<FAutoAssignElement>> PatchElements;
		if (!ensureMsgf(!FixturePatches.IsEmpty() && !FixturePatches.Contains(nullptr), TEXT("Unexpected no fixture patches or null entry in Fixture Patches. Expected to be handled during FAutoAssignUtility construction.")))
		{
			return PatchElements;
		}

		Algo::StableSortBy(FixturePatches, [](const UDMXEntityFixturePatch* FixturePatch)
			{
				return FixturePatch->GetUniverseID() * DMX_UNIVERSE_SIZE + FixturePatch->GetStartingChannel();
			});

		Algo::TransformIf(FixturePatches, PatchElements,
			[](UDMXEntityFixturePatch* FixturePatch)
			{
				return IsValid(FixturePatch);
			},
			[](UDMXEntityFixturePatch* FixturePatch)
			{
				return MakeShared<FAutoAssignElement>(*FixturePatch);
			});

		Algo::StableSortBy(PatchElements, &FAutoAssignElement::GetLowerBoundValue);

		return PatchElements;
	}

	TArray<UDMXEntityFixturePatch*> FAutoAssignUtility::FindOtherPatchesInLibrary(const UDMXLibrary& DMXLibrary, const TArray<UDMXEntityFixturePatch*>& FixturePatches) const
	{
		TArray<UDMXEntityFixturePatch*> Result = DMXLibrary.GetEntitiesTypeCast<UDMXEntityFixturePatch>();
		Result.RemoveAll([FixturePatches](const UDMXEntityFixturePatch* FixturePatch)
			{
				return
					!FixturePatch ||
					!FixturePatch->GetFixtureType() ||
					FixturePatches.Contains(FixturePatch);
			});


		Algo::StableSortBy(Result, [](UDMXEntityFixturePatch* FixturePatch)
			{
				return ((uint64)FixturePatch->GetUniverseID() * DMX_UNIVERSE_SIZE) + FixturePatch->GetStartingChannel();
			});

		return Result;
	}
}

#undef LOCTEXT_NAMESPACE
