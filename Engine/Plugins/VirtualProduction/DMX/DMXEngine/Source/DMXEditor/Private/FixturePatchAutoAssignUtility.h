// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Math/Range.h"
#include "UObject/WeakObjectPtr.h"

class FDMXEditor;
class FMenuBuilder;
class UDMXEntityFixturePatch;
class UDMXLibrary;


namespace UE::DMXEditor::AutoAssign::Private
{
	/** 
	 * An element to auto assign.
	 * Uses absolute DMX space, e.g. Universe 1 Channel 1 == 512.
	 */
	struct FAutoAssignElement
	{
		FAutoAssignElement(int64 LowerBoundInclusive, int64 UpperBoundExclusive);
		FAutoAssignElement(const TRange<int64>& InAbsoluteRange);
		FAutoAssignElement(UDMXEntityFixturePatch& FixturePatch);

		/** Returns the lower bound value, inclusive */
		int64 GetLowerBoundValue() const;

		/** Returns the upper bound value, exclusive */
		int64 GetUpperBoundValue() const;

		/** Returns the size */
		int64 GetSize() const;

		/** Returns the element as range */
		const TRange<int64>& GetRange() const { return AbsoluteRange; }

		/** Sets the absolute starting channel */
		void SetAbsoluteStartingChannel(int64 NewStartingAddress);

		/** Aligns this element after the other */
		void AlignAfter(const TSharedRef<FAutoAssignElement>& Other);

		/** If the element has a patch, writes the current range to the patch */
		void ApplyToPatch();

		/** Returns the fixture patch, or nullptr if the element has no patch */
		UDMXEntityFixturePatch* GetFixturePatch() const;

	private:
		int32 GetUniverse() const;
		int32 GetChannel() const;

		/** The absolute range, starting from universe 0 absolute channel 0 */
		TRange<int64> AbsoluteRange;

		/** The corresponding fixture patch */
		TWeakObjectPtr<UDMXEntityFixturePatch> WeakFixturePatch;
	};

	/** 
	 * Auto-assigns elements to free ranges.
	 *
	 * Uses absolute channel values, e.g. Universe 1 Channel 1 == 512 resp. DMX_UNIVERSE_SIZE.
	 * Applies results to patches on success.
	 */
	class FAutoAssignElementsUtility
	{
	public:
		/** Assigns the elements. Returns the universe of the first assigned patch */
		static int32 Assign(TArray<TSharedRef<FAutoAssignElement>> ElementsToAssign, int32 AssignToUniverse, int32 AssignToChannel);

		/** Aligns the elements. If bRetainStacks is true, retains stacks instead of aligning them too. */
		static void Align(TArray<TSharedRef<FAutoAssignElement>> ElementsToAlign, bool bRetainStacks);

		/** Stacks the elements */
		static void Stack(TArray<TSharedRef<FAutoAssignElement>> ElementsToStack);

		/** Spreads the elements over universes */
		static void SpreadOverUniverses(TArray<TSharedRef<FAutoAssignElement>> ElementsToSpread);

		/** Auto-assigns a set of elements to free elements. Note, free elements need be in the range of { DMX_UNIVERSE_SIZE, std::numeric_limits<int64>::Max() ), ensured. */
		static void AutoAssign(const TArray<TSharedRef<FAutoAssignElement>>& FreeElements, TArray<TSharedRef<FAutoAssignElement>> AutoAssignElements);

	private:
		/** Returns true if the elements are stacked. */
		static bool AreElementsStacked(const TSharedRef<FAutoAssignElement>& FirstElement, const TSharedRef<FAutoAssignElement>& SecondElement);

		/** Returns true if elments are in valid DMX range. Hits an ensure condition if elements are not valid. */
		static bool EnsureValidPatchElements(const TArray<TSharedRef<FAutoAssignElement>>& Elements);
	};
}


namespace UE::DMXEditor::AutoAssign
{
	enum class EAutoAssignMode : uint8
	{
		UserDefinedChannel,
		SelectedUniverse,
		FirstReachableUniverse,
		AfterLastPatchedUniverse
	};

	/** Utility to auto assign fixture patches. */
	class FAutoAssignUtility
	{
	public:
		/** Aligns FixturePatches */
		static int32 Assign(TArray<UDMXEntityFixturePatch*> FixturePatches, int32 AssignToUniverse, int32 AssignToChannel);

		/** Aligns FixturePatches */
		static void Align(TArray<UDMXEntityFixturePatch*> FixturePatches);

		/** Stacks FixturePatches */
		static void Stack(TArray<UDMXEntityFixturePatch*> FixturePatches);

		/** Auto assigns FixturePatches, one patch per Universe. Returns the Universe the first patch was auto assigned to.*/
		static void SpreadOverUniverses(TArray<UDMXEntityFixturePatch*> FixturePatches);

		/** Auto assigns FixturePatches. Returns the Universe the first patch was auto assigned to.*/
		static int32 AutoAssign(EAutoAssignMode Mode, const TSharedRef<FDMXEditor>& DMXEditor, TArray<UDMXEntityFixturePatch*> FixturePatches, int32 UniverseUnderMouse = 0, int32 UserDefinedChannel = 0);

	private:
		/** Non-static implementation of auto assign */
		int32 AutoAssignInternal(EAutoAssignMode Mode, const TSharedRef<FDMXEditor>& DMXEditor, TArray<UDMXEntityFixturePatch*> FixturePatches, int32 UniverseUnderMouse = 0, int32 UserDefinedChannel = 0);

		/** Finds the absolute starting channel from options */
		int64 FindAbsoluteStartingChannel(EAutoAssignMode Mode, const TSharedRef<FDMXEditor>& DMXEditor, const UDMXLibrary& DMXLibrary, const TArray<UDMXEntityFixturePatch*>& FixturePatches, int32 UniverseUnderMouse, int32 UserDefinedChannel) const;

		/** Gets the DMX Library from the patches, or returns nullptr if a valid DMX Library cannot be deduced from patches */
		UDMXLibrary* GetDMXLibrary(TArray<UDMXEntityFixturePatch*> FixturePatches) const;

		/** Creates elements for ranges that are not patched in the DMX Library */
		TArray<TSharedRef<Private::FAutoAssignElement>> CreateFreeElements(UDMXLibrary& DMXLibrary, const TArray<UDMXEntityFixturePatch*>& FixturePatches, int64 AbsoluteStartingChannel) const;

		/** Creates elements from patches. Note also creates elements for gaps between the patches to remember offsets */
		TArray<TSharedRef<Private::FAutoAssignElement>> CreatePatchElements(TArray<UDMXEntityFixturePatch*> FixturePatches) const;

		/** Helper to find other patches in the library */
		TArray<UDMXEntityFixturePatch*> FindOtherPatchesInLibrary(const UDMXLibrary& DMXLibrary, const TArray<UDMXEntityFixturePatch*>& FixturePatches) const;
	};
}
