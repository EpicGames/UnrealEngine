// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modes/DMXEditorApplicationMode.h"

#include "DMXEditor.h"
#include "DMXEditorTabNames.h"
#include "Tabs/DMXEditorTabFactories.h"
#include "Toolbars/DMXEditorToolbar.h"

#define LOCTEXT_NAMESPACE "DMXEditorApplicationMode"

const FName FDMXEditorApplicationMode::DefaultsMode(TEXT("DefaultsName"));

FText FDMXEditorApplicationMode::GetLocalizedMode(const FName InMode)
{
	static TMap< FName, FText > LocModes;

	if (LocModes.Num() == 0)
	{
		LocModes.Add(DefaultsMode, LOCTEXT("DMXDefaultsMode", "Defaults"));
	}

	check(InMode != NAME_None);
	return LocModes[InMode];
}

FDMXEditorDefaultApplicationMode::FDMXEditorDefaultApplicationMode(TSharedPtr<FDMXEditor> InDMXEditor)
	: FApplicationMode(FDMXEditorApplicationMode::DefaultsMode, FDMXEditorApplicationMode::GetLocalizedMode)
	, DMXEditorCachedPtr(InDMXEditor)
{
	using namespace UE::DMX;

	// 1. Create and register Tabs Factories
	DefaultsTabFactories.RegisterFactory(MakeShared<FDMXLibraryEditorTabSummoner>(InDMXEditor));
	DefaultsTabFactories.RegisterFactory(MakeShared<FDMXEditorFixtureTypesSummoner>(InDMXEditor));
	DefaultsTabFactories.RegisterFactory(MakeShared<FDMXEditorFixturePatchSummoner>(InDMXEditor));

	// 2. Register Tab Layouts
	TabLayout = FTabManager::NewLayout("Standalone_SimpleAssetEditor_Layout_v6")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewStack()
				->AddTab(FDMXEditorTabNames::DMXLibraryEditor, ETabState::OpenedTab)
				->AddTab(FDMXEditorTabNames::DMXFixtureTypesEditor, ETabState::OpenedTab)
				->AddTab(FDMXEditorTabNames::DMXFixturePatchEditor, ETabState::OpenedTab)
				->SetForegroundTab(FDMXEditorTabNames::DMXLibraryEditor)
			)
		);

	// 3. Setup Toolbar
	InDMXEditor->GetToolbarBuilder()->AddCompileToolbar(ToolbarExtender);
}

void FDMXEditorDefaultApplicationMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	if (TSharedPtr<FDMXEditor> DMXEditorPtr = DMXEditorCachedPtr.Pin())
	{
		DMXEditorPtr->RegisterToolbarTab(InTabManager.ToSharedRef());

		// Setup all tab factories
		DMXEditorPtr->PushTabFactories(DefaultsTabFactories);
	}
}

#undef LOCTEXT_NAMESPACE
