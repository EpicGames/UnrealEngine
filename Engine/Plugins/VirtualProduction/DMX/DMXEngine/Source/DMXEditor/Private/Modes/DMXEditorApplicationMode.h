// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorkflowOrientedApp/ApplicationMode.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

class FDMXEditor;

/**
 * Holds different DMX Editor modes identifiers
 * It could be more than one different layout for the editor
 */
struct FDMXEditorApplicationMode
{
	// Default layout for DMX editor
	static const FName DefaultsMode;

	static FText GetLocalizedMode(const FName InMode);

	/** No default constructor */
	FDMXEditorApplicationMode() = delete;
};

/**  Holds tabs layout and configuration for DMX default application mode */
class FDMXEditorDefaultApplicationMode : public FApplicationMode
{
public:
	FDMXEditorDefaultApplicationMode(TSharedPtr<FDMXEditor> InDMXEditor);

	//~ Begin FApplicationMode Interface
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
	//~ End FApplicationMode Interface

protected:
	TWeakPtr<FDMXEditor> DMXEditorCachedPtr;

	/** Set of spawnable tabs in Class Defaults mode */
	FWorkflowAllowedTabSet DefaultsTabFactories;
};
