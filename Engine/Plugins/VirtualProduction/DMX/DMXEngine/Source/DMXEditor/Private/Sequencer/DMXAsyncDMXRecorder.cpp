// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/DMXAsyncDMXRecorder.h"

#include "DMXProtocolCommon.h"
#include "IO/DMXRawListener.h"
#include "IO/DMXPort.h"
#include "Library/DMXLibrary.h"
#include "Library/DMXEntityFixtureType.h"
#include "Sequencer/MovieSceneDMXLibrarySection.h"

#include "TakeRecorderSettings.h"
#include "Misc/ScopeLock.h"
#include "Recorder/TakeRecorderParameters.h"


////////////////////////////////////////////////
// Helpers to add data to a DMXRecording

namespace
{
	void WriteSignalToFunctionChannelData(FDMXFunctionChannelData& MutableChannelData, const FFrameNumber& FrameNumber, const TArray<uint8>& UniverseData, bool bNormalizedValues)
	{
		check(UniverseData.Num() == DMX_UNIVERSE_SIZE);

		// Values we want to acquire
		FMovieSceneFloatValue MovieSceneFloatValue;
		MovieSceneFloatValue.InterpMode = ERichCurveInterpMode::RCIM_Linear;
		MovieSceneFloatValue.TangentMode = ERichCurveTangentMode::RCTM_None;

		bool bAcquiredValue = false;
		if (UDMXEntityFixturePatch* FixturePatch = MutableChannelData.GetFixturePatch().Get())
		{
			if (MutableChannelData.IsCellChannel())
			{
				FDMXFixtureMatrix MatrixProperties;
				if (!FixturePatch->GetMatrixProperties(MatrixProperties))
				{
					return;
				}

				const FDMXFixtureCellAttribute* CellAttributePtr = MatrixProperties.CellAttributes.FindByPredicate([&MutableChannelData](const FDMXFixtureCellAttribute& CellAttribute) {
					return CellAttribute.Attribute.Name == MutableChannelData.GetAttributeName();
					});

				if (CellAttributePtr)
				{
					TMap<FDMXAttributeName, int32> AttributeChannelMap;
					FixturePatch->GetMatrixCellChannelsAbsolute(MutableChannelData.GetCellCoordinate(), AttributeChannelMap);

					const int32* ChannelPtr = AttributeChannelMap.Find(MutableChannelData.GetAttributeName());

					if (ChannelPtr)
					{
						const int32 ChannelIndex = *ChannelPtr - 1;
						if (ensure(UniverseData.IsValidIndex(ChannelIndex)))
						{
							if (bNormalizedValues)
							{
								MovieSceneFloatValue.Value = UDMXEntityFixtureType::BytesToNormalizedValue(CellAttributePtr->DataType, CellAttributePtr->bUseLSBMode, &UniverseData[ChannelIndex]);
							}
							else
							{
								MovieSceneFloatValue.Value = UDMXEntityFixtureType::BytesToInt(CellAttributePtr->DataType, CellAttributePtr->bUseLSBMode, &UniverseData[ChannelIndex]);
							}
						}

						bAcquiredValue = true;
					}
				}
			}
			else
			{
				if (const FDMXFixtureMode* ModePtr = FixturePatch->GetActiveMode())
				{
					int32 AbsoluteStartingChannel = FixturePatch->GetStartingChannel();

					const FDMXFixtureFunction* FixtureFunctionPtr = ModePtr->Functions.FindByPredicate([&MutableChannelData](const FDMXFixtureFunction& Function) {
						return Function.Attribute.Name == MutableChannelData.GetAttributeName();
						});

					if (FixtureFunctionPtr)
					{
						const FDMXFixtureFunction& Function = *FixtureFunctionPtr;
						int32 AttributeIndex = AbsoluteStartingChannel + Function.Channel - 2; // FixturePatch Channel - 1, Function Channel - 1

						if (bNormalizedValues)
						{
							MovieSceneFloatValue.Value = UDMXEntityFixtureType::BytesToNormalizedValue(Function.DataType, Function.bUseLSBMode, &UniverseData[AttributeIndex]);
						}
						else
						{
							MovieSceneFloatValue.Value = UDMXEntityFixtureType::BytesToInt(Function.DataType, Function.bUseLSBMode, &UniverseData[AttributeIndex]);
						}

						bAcquiredValue = true;
					}
				}
			}
		}

		if (bAcquiredValue)
		{
			// Note: Generally to satisfy MovieSceneFloatChannel::AddKeys, Times and Values need be same sized arrays at all times
			bool bFirstKeyframe = MutableChannelData.Times.Num() == 0;

			if (bFirstKeyframe)
			{
				// Set the new keyframe
				MutableChannelData.Times.Add(FrameNumber);
				MutableChannelData.Values.Add(MovieSceneFloatValue);
			}
			else
			{
				if (MutableChannelData.Times.Last() == FrameNumber)
				{
					// The previous value sits on the same frame number, replace it with the new value
					MutableChannelData.Times.Last() = FrameNumber;
					MutableChannelData.Values.Last() = MovieSceneFloatValue;
				}
				else
				{
					FMovieSceneFloatValue PreviousValue = MutableChannelData.Values.Last();

					// Only record changed Values
					if (PreviousValue.Value != MovieSceneFloatValue.Value)
					{
						// Set the previous value anew one frame ahead of the new value
						MutableChannelData.Times.Add(FrameNumber - 1);
						MutableChannelData.Values.Add(PreviousValue);

						// Add the new Value
						MutableChannelData.Times.Add(FrameNumber);
						MutableChannelData.Values.Add(MovieSceneFloatValue);
					}
				}
			}
		}
	}
}


////////////////////////////////////////////////
// FDMXFunctionChannelData

FDMXFunctionChannelData::FDMXFunctionChannelData(UDMXEntityFixturePatch* InFixturePatch, FDMXFixtureFunctionChannel* InFunctionChannel)
	: FixturePatch(InFixturePatch)
	, FunctionChannel(InFunctionChannel)
	, LocalUniverseID(-1)
	, bCellChannel(false)
	, CellCoordinate(FIntPoint(0, 0))
{
	if (FunctionChannel &&
		FixturePatch.IsValid() &&
		IsValid(FixturePatch->GetFixtureType()))
	{
		LocalUniverseID = FixturePatch->GetUniverseID();
		AttributeName = FunctionChannel->AttributeName;
		bCellChannel = FunctionChannel->IsCellFunction();

		if (bCellChannel)
		{
			CellCoordinate = FunctionChannel->CellCoordinate;
		}
	}
}

FDMXFixtureFunctionChannel* FDMXFunctionChannelData::TryGetFunctionChannel(const UMovieSceneDMXLibrarySection* MovieSceneDMXLibrarySection)
{
	if (IsValid(MovieSceneDMXLibrarySection) && FixturePatch.IsValid() && FunctionChannel)
	{
		// Find the patch channel that is using the same patch
		const FDMXFixturePatchChannel* EqualPatchChannelPtr = MovieSceneDMXLibrarySection->GetFixturePatchChannels().FindByPredicate([this](const FDMXFixturePatchChannel& FixturePatchChannel) {
			return FixturePatchChannel.Reference.GetFixturePatch() == FixturePatch;
			});

		if (EqualPatchChannelPtr)
		{
			// Find the function channel at the same memory location
			const FDMXFixtureFunctionChannel* EqualFunctionChannelPtr = EqualPatchChannelPtr->FunctionChannels.FindByPredicate([this](const FDMXFixtureFunctionChannel& FixtureFunctionChannel) {
				return FunctionChannel == &FixtureFunctionChannel;
				});

			// The function still exists
			return FunctionChannel;
		}
	}

	return nullptr;
}


////////////////////////////////////////////////
// FDMXAsyncDMXRecorder

FDMXAsyncDMXRecorder::FDMXAsyncDMXRecorder(UDMXLibrary* InDMXLibrary, UMovieSceneDMXLibrarySection* InMovieSceneDMXLibrarySection)
	: DMXLibrary(InDMXLibrary)
	, MovieSceneDMXLibrarySection(InMovieSceneDMXLibrarySection)
	, bRecordNormalizedValues(true)
	, RecordStartTime(0.0)
	, RecordStartFrame()
	, TickResolution()
	, bStartAtCurrentTimecode(true)
	, NumSignalsToProccess(0)
	, Thread(nullptr)
	, bThreadStopping(false)
	, bListenersStopped(true)
{
	// Create Function Channel Data structs from the available channels.
	if (IsValid(DMXLibrary) && IsValid(MovieSceneDMXLibrarySection))
	{
		bRecordNormalizedValues = InMovieSceneDMXLibrarySection->bUseNormalizedValues;

		for (FDMXFixturePatchChannel& FixturePatchChannel : MovieSceneDMXLibrarySection->GetMutableFixturePatchChannels())
		{
			UDMXEntityFixturePatch* FixturePatch = FixturePatchChannel.Reference.GetFixturePatch();

			// Only valid patches
			if (IsValid(FixturePatch))
			{
				for (FDMXFixtureFunctionChannel& FixtureFunctionChannel : FixturePatchChannel.FunctionChannels)
				{
					FunctionChannelDataArr.Add(FDMXFunctionChannelData(FixturePatch, &FixtureFunctionChannel));
				}
			}
		}
	}

	// Sort by universe id ascending for faster writes
	FunctionChannelDataArr.Sort([](const FDMXFunctionChannelData& First, const FDMXFunctionChannelData& Second) {
		return First.GetLocalUniverseID() < Second.GetLocalUniverseID();
		});

	// Create Listeners
	for (const FDMXPortSharedRef& Port : DMXLibrary->GenerateAllPortsSet())
	{
		TSharedRef<FDMXRawListener> NewListener = MakeShared<FDMXRawListener>(Port);
		RawListeners.Add(NewListener);
	}

	// Create the thread
	FString ThreadName = TEXT("AsyncDMXRecorder");
	Thread = FRunnableThread::Create(this, *ThreadName, 0, TPri_Normal, FPlatformAffinity::GetPoolThreadMask());
}

FDMXAsyncDMXRecorder::~FDMXAsyncDMXRecorder()
{
	// We need to stop in any case
	for (const TSharedRef<FDMXRawListener>& Listener : RawListeners)
	{
		Listener->Stop();
	}

	if (Thread != nullptr)
	{
		Thread->Kill(true);
		delete Thread;
	}
}

bool FDMXAsyncDMXRecorder::Init()
{
	return true;
}

uint32 FDMXAsyncDMXRecorder::Run()
{
	while (!bThreadStopping)
	{
		Update();
		FPlatformProcess::Sleep(0.01f);
	}

	return 0;
}

void FDMXAsyncDMXRecorder::Stop()
{
	bThreadStopping = true;
}

void FDMXAsyncDMXRecorder::Tick()
{
	Update();
}

void FDMXAsyncDMXRecorder::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(DMXLibrary);
	Collector.AddReferencedObject(MovieSceneDMXLibrarySection);
}

FSingleThreadRunnable* FDMXAsyncDMXRecorder::GetSingleThreadInterface()
{
	return this;
}

void FDMXAsyncDMXRecorder::StartRecording(double InRecordStartTime, FFrameNumber InRecordStartFrame, FFrameRate InTickResolution)
{
	if (!IsValid(DMXLibrary))
	{
		return;
	}

	if (bListenersStopped)
	{
		// Reset the buffer, to make sure it doesn't contain data from the previous recording
		SignalToLocalUniverseMap.Reset();

		// Get relevant timing info
		FTakeRecorderParameters Parameters;
		Parameters.Project = GetDefault<UTakeRecorderProjectSettings>()->Settings;

		bStartAtCurrentTimecode = Parameters.Project.bStartAtCurrentTimecode;
		RecordStartTime = InRecordStartTime;
		RecordStartFrame = InRecordStartFrame;
		TickResolution = InTickResolution;

		// Recording started, minimize resource usage
		Thread->SetThreadPriority(TPri_Lowest);

		// Start the listeners
		for (const TSharedRef<FDMXRawListener>& Listener : RawListeners)
		{
			Listener->Start();
		}

		bListenersStopped = false;
	}
}

void FDMXAsyncDMXRecorder::StopRecording()
{
	if (!IsValid(DMXLibrary))
	{
		return;
	}

	if (!bListenersStopped)
	{
		// Stop and release the listeners
		for (const TSharedRef<FDMXRawListener>& Listener : RawListeners)
		{
			Listener->Stop();

			// Recording stopped, return to normal resource usage
			Thread->SetThreadPriority(TPri_Normal);
		}

		bListenersStopped = true;
	}
}

int32 FDMXAsyncDMXRecorder::GetNumSignalsToProccess() const
{
	// If this check is hit, data is still being processed
	check(bListenersStopped);

	return NumSignalsToProccess;
}

TArray<FDMXFunctionChannelData> FDMXAsyncDMXRecorder::GetRecordedData() const
{
	check(bListenersStopped);

	return FunctionChannelDataArr;
}

void FDMXAsyncDMXRecorder::Update()
{
	// Dequeue signals
	for (const TSharedRef<FDMXRawListener>& Listener : RawListeners)
	{
		FDMXSignalSharedPtr Signal;
		int32 LocalUniverseID;
		while (Listener->DequeueSignal(this, Signal, LocalUniverseID))
		{
			SignalToLocalUniverseMap.Add(Signal.ToSharedRef(), LocalUniverseID);
		}
	}

	// Update how many signals remain to be processed
	NumSignalsToProccess = SignalToLocalUniverseMap.Num();

	// Proccess data
	for (const TTuple<FDMXSignalSharedRef, int32>& SignalToLocalUniverseKvp : SignalToLocalUniverseMap)
	{
		int32 LocalUniverseID = SignalToLocalUniverseKvp.Value;

		for (FDMXFunctionChannelData& MutableChannelData : FunctionChannelDataArr)
		{
			// Find if data is relevant to the patch
			if (MutableChannelData.GetLocalUniverseID() == LocalUniverseID)
			{
				const FDMXSignalSharedRef& Signal = SignalToLocalUniverseKvp.Key;
				FFrameNumber FrameNumber = GetFrameNumberFromSignal(Signal);

				WriteSignalToFunctionChannelData(MutableChannelData, GetFrameNumberFromSignal(Signal), Signal->ChannelData, bRecordNormalizedValues);
			}

			// Patches are sorted by universe ID ascending. 
			// Skip to the next signal when the universe ID is higher to avoid needless testing
			if (MutableChannelData.GetLocalUniverseID() > LocalUniverseID)
			{
				break;
			}
		}

		NumSignalsToProccess--;
	}

	// Reset the buffer, but don't shrink
	SignalToLocalUniverseMap.Reset();
}

FFrameNumber FDMXAsyncDMXRecorder::GetFrameNumberFromSignal(const FDMXSignalSharedRef& Signal) const
{
	if (bStartAtCurrentTimecode)
	{
		return ((Signal->Timestamp - RecordStartTime) * TickResolution).FloorToFrame() + RecordStartFrame;
	}
	else
	{
		return ((Signal->Timestamp - RecordStartTime) * TickResolution).FloorToFrame();
	}
}
