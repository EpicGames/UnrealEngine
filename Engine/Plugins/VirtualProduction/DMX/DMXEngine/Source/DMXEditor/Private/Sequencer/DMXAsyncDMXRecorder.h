// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolCommon.h"
#include "Library/DMXEntityFixturePatch.h"

#include "CoreMinimal.h"
#include "HAL/CriticalSection.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Misc/SingleThreadRunnable.h"
#include "UObject/GCObject.h"

struct FDMXFixtureFunctionChannel;
class  FDMXRawListener;
class  FDMXSignal;
class  UDMXLibrary;
class  UMovieSceneDMXLibrarySection;

struct FMovieSceneFloatValue;


/** 
 * Can hold data for a specific channel from the FDMXRecorder. Does not rely on the channel remaining valid 
 * Note, it is required to do this by channel so we can set all keyframes at once via SetKeys method
 */
struct FDMXFunctionChannelData
{
public:
	FDMXFunctionChannelData() = delete;

	FDMXFunctionChannelData(UDMXEntityFixturePatch* InFixturePatch, FDMXFixtureFunctionChannel* InFunctionChannel);

	// Mutable times and values, in same size arrays, as per sequencer track requirements

	/** Time of each keyframe  */
	TArray<FFrameNumber> Times;

	/** Value of each keyframe */
	TArray<FMovieSceneFloatValue> Values;

	/** Returns a ptr to the function channel if it still exists, or nullptr if it turned invalid */
	FDMXFixtureFunctionChannel* TryGetFunctionChannel(const UMovieSceneDMXLibrarySection* InMovieSceneDMXLibrarySection);

	int32 GetLocalUniverseID() const { return LocalUniverseID; }

	const TWeakObjectPtr<UDMXEntityFixturePatch> GetFixturePatch() const { return FixturePatch; }

	const FName& GetAttributeName() const { return AttributeName; }

	bool IsCellChannel() const { return bCellChannel; }

	FIntPoint GetCellCoordinate() const { return CellCoordinate; }

private: 
	/** Raw ptr to the fixture patch of the channel, not guaranteed to remain valid */
	TWeakObjectPtr<UDMXEntityFixturePatch> FixturePatch;

	/** Raw ptr to the function channel, not guaranteed to remain valid */
	FDMXFixtureFunctionChannel* FunctionChannel;

	/** The local universe ID of the patch */
	int32 LocalUniverseID;

	/** The attribute name of teh function channel */
	FName AttributeName;

	/** True if the channel is a matrix cell channel */
	bool bCellChannel;

	/** The cell coordinate */
	FIntPoint CellCoordinate;
};


/** Records DMX asynchronously to optimize performance yet minimize processing of data after recroding */
class FDMXAsyncDMXRecorder
	: public FRunnable
	, public FSingleThreadRunnable
	, public FGCObject
	, public TSharedFromThis<FDMXAsyncDMXRecorder>
{
public:
	FDMXAsyncDMXRecorder(UDMXLibrary* InDMXLibrary, UMovieSceneDMXLibrarySection* InMovieSceneDMXLibrarySection);

	virtual ~FDMXAsyncDMXRecorder();

protected:
	// ~Begin FRunnable Interface
	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	// ~End FRunnable Interface

	// ~Begin FSingleThreadRunnable Interface
	virtual void Tick() override;
	virtual class FSingleThreadRunnable* GetSingleThreadInterface() override;
	// ~End FSingleThreadRunnable Interface

	// ~Begin FGCObject Interface
	virtual void AddReferencedObjects(FReferenceCollector & Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FDMXAsyncDMXRecorder");
	}
	// ~End FGCObject Interface

public:
	/** Starts to record incoming DMX */
	void StartRecording(double InRecordStartTime, FFrameNumber InRecordStartFrame, FFrameRate InTickResolution);

	/** Stops the recording */
	void StopRecording();

	/** Returns the number of signals that remain to be proccessed. */
	int32 GetNumSignalsToProccess() const;

	/** Returns the recorded data. */
	TArray<FDMXFunctionChannelData> GetRecordedData() const;

private:
	/** The actual recording, per fixutre function channel */
	TArray<FDMXFunctionChannelData> FunctionChannelDataArr;

private:
	/** Returns the frame number from a signal */
	FFrameNumber GetFrameNumberFromSignal(const FDMXSignalSharedRef& Signal) const;

	/** Processes newly received data */
	void Update();

	/** Raw listener for the relevant ports */
	TSet<TSharedRef<FDMXRawListener>> RawListeners;

	/** Cache used during recording */
	TMap<FDMXSignalSharedRef, int32 /** LocalUniverseID */> SignalToLocalUniverseMap;

	/** The library of the patches, gc'ed */
	TObjectPtr<UDMXLibrary> DMXLibrary;

	/** The library section to record, gc'ed */
	TObjectPtr<UMovieSceneDMXLibrarySection> MovieSceneDMXLibrarySection;

	/** If true, records normalized values */
	bool bRecordNormalizedValues;

	/** The time the recording started */
	double RecordStartTime;

	/** The frame the recording started */
	FFrameNumber RecordStartFrame;
	
	/** The tick resolution of the recording */
	FFrameRate TickResolution;

	/** True if recording needs to start at the current time code */
	bool bStartAtCurrentTimecode;

	/** The number of signals to proccess */
	TAtomic<int32> NumSignalsToProccess;

	/** Thread that processes the data */
	FRunnableThread* Thread;

	/** True while the thread wasn't stopped */
	bool bThreadStopping;

	/** True when all listeners are stopped */
	bool bListenersStopped;
};
