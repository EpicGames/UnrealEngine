// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/DMXLibrarySection.h"
#include "Sequencer/MovieSceneDMXLibrarySection.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXEntityFixtureType.h"

#include "ISequencer.h"
#include "MovieSceneSection.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "DMXLibrarySection"

FDMXLibrarySection::FDMXLibrarySection(UMovieSceneSection& InSection, TWeakPtr<ISequencer> InSequencer)
	: FSequencerSection(InSection), WeakSequencer(InSequencer)
{}

void FDMXLibrarySection::BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& InObjectBinding)
{
	UMovieSceneDMXLibrarySection* DMXSection = CastChecked<UMovieSceneDMXLibrarySection>(WeakSection.Get());
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();

	// Begin our Patches section for the context menu
	MenuBuilder.BeginSection(NAME_None, LOCTEXT("PatchesMenuSection", "Active Patches"));
	for (const FDMXFixturePatchChannel& PatchChannel : DMXSection->GetFixturePatchChannels())
	{
		UDMXEntityFixturePatch* Patch = PatchChannel.Reference.GetFixturePatch();
		if (Patch == nullptr || !Patch->IsValidLowLevelFast())
		{
			continue;
		}
		const FText&& PatchNameText = FText::FromString(Patch->GetDisplayName());
		
		auto MakeActiveModeUIAction = [=](const FDMXFixtureMode& InMode, int32 InModeIndex)->FUIAction
		{
			return FUIAction(
				FExecuteAction::CreateLambda([=]
				{
					// Set Patch active mode
					if (PatchChannel.ActiveMode == InModeIndex)
					{
						return;
					}

					const FScopedTransaction Transaction(FText::Format(
						LOCTEXT("SetActiveModeTransaction", "Set {0} as {1} track's Active Mode"),
						FText::FromString(InMode.ModeName),
						PatchNameText));

					DMXSection->Modify();
					DMXSection->SetFixturePatchActiveMode(Patch, InModeIndex);

					SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
				}),
				FCanExecuteAction::CreateLambda([=]
				{
					UDMXEntityFixtureType* FixtureType = Patch->GetFixtureType();
					if (!IsValid(Patch) || 
						!IsValid(FixtureType) ||
						!FixtureType->IsValidLowLevelFast() ||
						InModeIndex >= FixtureType->Modes.Num())
					{
						return false;
					}

					return FixtureType->Modes[InModeIndex].Functions.Num() > 0;
				}),
				FIsActionChecked::CreateLambda([=]
				{
					return PatchChannel.ActiveMode == InModeIndex;
				})
			);
		};

		auto MakeFunctionChannelUIAction = [=](const FDMXFixtureFunction& InFunction, int32 InFunctionIndex)->FUIAction
		{
			return FUIAction(
				FExecuteAction::CreateLambda([=]
				{
					// Toggle Function Channel display
					const FScopedTransaction Transaction(FText::Format(
						LOCTEXT("ToggleFunctionChannelTransaction", "Toggle {0} channel track on {1} track"),
						FText::FromString(InFunction.FunctionName),
						PatchNameText));

					DMXSection->Modify();
					DMXSection->ToggleFixturePatchChannel(Patch, InFunctionIndex);

					SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
				}),
				FCanExecuteAction(),
				FGetActionCheckState::CreateLambda([=]
				{
					return DMXSection->GetFixturePatchChannelEnabled(Patch, InFunctionIndex)
						? ECheckBoxState::Checked
						: ECheckBoxState::Unchecked;
				})
			);
		};

		// For each Patch added to the track, create a sub menu if its FixtureType is valid
		UDMXEntityFixtureType* FixtureType = Patch->GetFixtureType();
		if (!IsValid(FixtureType))
		{
			continue;
		}

		const FText&& Tooltip = FText::Format(LOCTEXT("PatchSubmenuTooltip", "Settings for the Fixture Functions under {0}"), PatchNameText);
		MenuBuilder.AddSubMenu(
			PatchNameText, Tooltip,
			FNewMenuDelegate::CreateLambda([=](FMenuBuilder& SubMenuBuilder) {
				// Patch sub menus will contain radio buttons to select the active mode
				// and toggles to display its function channels in the track
				
				// Modes section
				SubMenuBuilder.BeginSection(NAME_None, LOCTEXT("PatchActiveModeMenuSection", "Active Mode"));
				{
					int32 ModeIndex = 0;
					for (const FDMXFixtureMode& Mode : FixtureType->Modes)
					{
						const FText&& ModeNameText = FText::FromString(Mode.ModeName);
						const FText&& ModeTooltip = FText::Format(
							LOCTEXT("ActiveModeTooltip", "Animate Functions from {0} mode"),
							ModeNameText);
						SubMenuBuilder.AddMenuEntry(
							ModeNameText, ModeTooltip,
							FSlateIcon(),
							MakeActiveModeUIAction(Mode, ModeIndex++),
							NAME_None, EUserInterfaceActionType::RadioButton);
					}
				}
				SubMenuBuilder.EndSection();

				// Functions from Active Mode section
				SubMenuBuilder.BeginSection(NAME_None, LOCTEXT("PatchFunctionsMenuSection", "Functions"));
				{
					if (PatchChannel.ActiveMode < FixtureType->Modes.Num())
					{
						const FDMXFixtureMode& Mode = FixtureType->Modes[PatchChannel.ActiveMode];
						int32 FunctionIndex = 0;
						for (const FDMXFixtureFunction& Function : Mode.Functions)
						{
							const FText&& FunctionNameText = FText::FromString(Function.FunctionName);
							const FText&& FunctionTooltip = FText::Format(
								LOCTEXT("FunctionChannelTooltip", "Toggle animation track for {0}"),
								FunctionNameText);

							SubMenuBuilder.AddMenuEntry(
								FunctionNameText, FunctionTooltip,
								FSlateIcon(),
								MakeFunctionChannelUIAction(Function, FunctionIndex++),
								NAME_None, EUserInterfaceActionType::ToggleButton);
						}
					}
				}
				SubMenuBuilder.EndSection();
			}),
			FUIAction(),
			NAME_None,
			EUserInterfaceActionType::None);
	}
	MenuBuilder.EndSection();
}

bool FDMXLibrarySection::RequestDeleteCategory(const TArray<FName>& CategoryNamePath)
{
	UMovieSceneDMXLibrarySection* DMXSection = CastChecked<UMovieSceneDMXLibrarySection>(WeakSection.Get());
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();

	const FName& PatchName = CategoryNamePath[CategoryNamePath.Num() - 1];

	const FScopedTransaction Transaction(FText::Format(
		LOCTEXT("DeletePatchTrack", "Delete {0} from track"),
		FText::FromName(PatchName)));

	if (DMXSection->TryModify())
	{
		DMXSection->RemoveFixturePatch(PatchName);
		SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
		return true;
	}

	return false;
}

bool FDMXLibrarySection::RequestDeleteKeyArea(const TArray<FName>& KeyAreaNamePath)
{
	UMovieSceneDMXLibrarySection* DMXSection = CastChecked<UMovieSceneDMXLibrarySection>(WeakSection.Get());
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();

	const FName& PatchName = KeyAreaNamePath[0];

	// Get the channel name from the full path
	const FString&& ChannelNamePath = KeyAreaNamePath[KeyAreaNamePath.Num() - 1].ToString();
	TArray<FString> ChannelNamePathParts;
	FName ChannelName = NAME_None;
	if (ChannelNamePath.ParseIntoArray(ChannelNamePathParts, TEXT(".")))
	{
		ChannelName = *ChannelNamePathParts[ChannelNamePathParts.Num() - 1];
	}

	if (ChannelName == NAME_None)
	{
		return false;
	}

	const FScopedTransaction Transaction(FText::Format(
		LOCTEXT("DeletePatchTrack", "Delete {0} from track"),
		FText::FromName(PatchName)));

	if (DMXSection->TryModify())
	{
		DMXSection->ToggleFixturePatchChannel(PatchName, ChannelName);
		SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
		return true;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
