// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISequencerSection.h"

struct FGuid;
class FMenuBuilder;
class ISequencer;
class UMovieSceneSection;

/** Editor implementation of DMX Library track sections */
class FDMXLibrarySection
	: public FSequencerSection
{
public:

	FDMXLibrarySection(UMovieSceneSection& InSection, TWeakPtr<ISequencer> InSequencer);

public:

	//~ ISequencerSection interface start

	/** Called when the user right clicks the keyframes area of the track */
	virtual void BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& InObjectBinding) override;

	/** Called when the user deletes a Patch section */
	virtual bool RequestDeleteCategory(const TArray<FName>& CategoryNamePath) override;

	/** Called when the user deletes a Patch's Function Channel section */
	virtual bool RequestDeleteKeyArea(const TArray<FName>& KeyAreaNamePath) override;

	//~ ISequencerSection interface end

protected:

	/** The sequencer which is controlling this section */
	TWeakPtr<ISequencer> WeakSequencer;
};
