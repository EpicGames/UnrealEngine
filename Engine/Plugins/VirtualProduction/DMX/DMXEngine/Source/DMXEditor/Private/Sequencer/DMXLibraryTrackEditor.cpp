// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXLibraryTrackEditor.h"

#include "Algo/Sort.h"
#include "ContentBrowserModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IContentBrowserSingleton.h"
#include "ISequencer.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXLibrary.h"
#include "MovieSceneSection.h"
#include "MVVM/Views/ViewUtilities.h"
#include "Sequencer/DMXLibrarySection.h"
#include "Sequencer/MovieSceneDMXLibrarySection.h"
#include "Sequencer/MovieSceneDMXLibraryTrack.h"
#include "SequencerSettings.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "DMXLibraryTrackEditor"

DECLARE_LOG_CATEGORY_CLASS(DMXLibraryTrackEditorLog, Log, All);


FDMXLibraryTrackEditor::FDMXLibraryTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer)
{}

TSharedRef<ISequencerTrackEditor> FDMXLibraryTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer)
{
	return MakeShared<FDMXLibraryTrackEditor>(OwningSequencer);
}

namespace UE::DMX
{
	TSharedRef<SWidget> CreateAssetPicker(FOnAssetSelected OnAssetSelected, FOnAssetEnterPressed OnAssetEnterPressed, TWeakPtr<ISequencer> InSequencer)
	{
		UMovieSceneSequence* Sequence = InSequencer.IsValid() ? InSequencer.Pin()->GetFocusedMovieSceneSequence() : nullptr;

		FAssetPickerConfig AssetPickerConfig;
		{
			AssetPickerConfig.OnAssetSelected = OnAssetSelected;
			AssetPickerConfig.OnAssetEnterPressed = OnAssetEnterPressed;
			AssetPickerConfig.bAllowNullSelection = false;
			AssetPickerConfig.bAddFilterUI = true;
			AssetPickerConfig.bShowTypeInColumnView = false;
			AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
			AssetPickerConfig.Filter.bRecursiveClasses = true;
			AssetPickerConfig.Filter.ClassPaths.Add(UDMXLibrary::StaticClass()->GetClassPathName());
			AssetPickerConfig.SaveSettingsName = TEXT("SequencerAssetPicker");
			AssetPickerConfig.AdditionalReferencingAssets.Add(FAssetData(Sequence));
		}

		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

		TSharedPtr<ISequencer> Sequencer = InSequencer.Pin();
		const float WidthOverride = Sequencer.IsValid() ? Sequencer->GetSequencerSettings()->GetAssetBrowserWidth() : 500.f;
		const float HeightOverride = Sequencer.IsValid() ? Sequencer->GetSequencerSettings()->GetAssetBrowserHeight() : 400.f;

		return SNew(SBox)
			.WidthOverride(WidthOverride)
			.HeightOverride(HeightOverride)
			[
				ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
			];
	}
}

void FDMXLibraryTrackEditor::BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track)
{
	UMovieSceneDMXLibraryTrack* DMXTrack = Cast<UMovieSceneDMXLibraryTrack>(Track);

	auto AssignAsset = [DMXTrack](const FAssetData& InAssetData)
	{
		UDMXLibrary* DMXLibrary = Cast<UDMXLibrary>(InAssetData.GetAsset());

		if (DMXLibrary)
		{
			if (DMXTrack->GetDMXLibrary() != DMXLibrary)
			{
				FScopedTransaction Transaction(LOCTEXT("SetDMXAssetTransaction", "Assign DMX Library to track"));
				DMXTrack->Modify();
				for (UMovieSceneSection* Section : DMXTrack->GetAllSections())
				{
					if (UMovieSceneDMXLibrarySection* DMXLibrarySection = Cast<UMovieSceneDMXLibrarySection>(Section))
					{
						DMXLibrarySection->Modify();
					}
				}

				DMXTrack->SetDMXLibrary(DMXLibrary);
			}
		}

		FSlateApplication::Get().DismissAllMenus();
	};

	auto AssignAssetEnterPressed = [AssignAsset](const TArray<FAssetData>& InAssetData)
	{
		if (InAssetData.Num() > 0)
		{
			AssignAsset(InAssetData[0].GetAsset());
		}
	};

	auto SubMenuCallback = [this, AssignAsset, AssignAssetEnterPressed](FMenuBuilder& SubMenuBuilder)
	{
		SubMenuBuilder.AddWidget(UE::DMX::CreateAssetPicker(FOnAssetSelected::CreateLambda(AssignAsset), FOnAssetEnterPressed::CreateLambda(AssignAssetEnterPressed), GetSequencer()), FText::GetEmpty(), true);
	};

	MenuBuilder.AddSubMenu(
		LOCTEXT("SetAsset", "Set Asset"),
		LOCTEXT("SetAsset_ToolTip", "Sets the DMX Library that this track animates."),
		FNewMenuDelegate::CreateLambda(SubMenuCallback)
	);
}

void FDMXLibraryTrackEditor::BuildAddTrackMenu(FMenuBuilder& MenuBuilder)
{
	auto SubMenuCallback = [this](FMenuBuilder& SubMenuBuilder)
	{
		SubMenuBuilder.AddWidget(
			UE::DMX::CreateAssetPicker(
				FOnAssetSelected::CreateRaw(this, &FDMXLibraryTrackEditor::AddDMXLibraryTrackToSequence),
				FOnAssetEnterPressed::CreateRaw(this, &FDMXLibraryTrackEditor::AddDMXLibraryTrackToSequenceEnterPressed),
				GetSequencer()
			),
			FText::GetEmpty(),
			true);
	};

	MenuBuilder.AddSubMenu(
		LOCTEXT("AddDMXTrack", "DMX Library Track"),
		LOCTEXT("AddDMXTrackToolTip", "Adds a new track that controls a DMX Library's Patches functions."),
		FNewMenuDelegate::CreateLambda(SubMenuCallback),
		false,
		FSlateIconFinder::FindIconForClass(UDMXLibrary::StaticClass())
	);
}

bool FDMXLibraryTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return Type == UMovieSceneDMXLibraryTrack::StaticClass();
}

bool FDMXLibraryTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	// DMX Library tracks can be added to level sequences and widget sequences
	static UClass* LevelSequenceClass = FindObject<UClass>(nullptr, TEXT("/Script/LevelSequence.LevelSequence"), EFindObjectFlags::ExactClass);
	static UClass* WidgetAnimationClass = FindObject<UClass>(nullptr, TEXT("/Script/UMG.WidgetAnimation"), EFindObjectFlags::ExactClass);
	return InSequence != nullptr &&
		((LevelSequenceClass != nullptr && InSequence->GetClass()->IsChildOf(LevelSequenceClass)) ||
		(WidgetAnimationClass != nullptr && InSequence->GetClass()->IsChildOf(WidgetAnimationClass)));
}

const FSlateBrush* FDMXLibraryTrackEditor::GetIconBrush() const
{
	// TODO Add DMX Library icon
	//return FSlateIconFinder::FindIconForClass(UMaterialParameterCollection::StaticClass()).GetIcon();
	return FMovieSceneTrackEditor::GetIconBrush();
}

TSharedPtr<SWidget> FDMXLibraryTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	UMovieSceneDMXLibraryTrack* DMXTrack = Cast<UMovieSceneDMXLibraryTrack>(Track);

	// Create combo button "+ Patch" to pick an asset to add
	// sub menu content callback
	FOnGetContent AddPatchMenuContent = FOnGetContent::CreateSP(this, &FDMXLibraryTrackEditor::OnGetAddPatchMenuContent, DMXTrack);

	constexpr bool bEnabled = true;
	return UE::Sequencer::MakeAddButton(LOCTEXT("AddPatchButton", "Patch"), AddPatchMenuContent, Params.ViewModel);
}

TSharedRef<SWidget> FDMXLibraryTrackEditor::OnGetAddPatchMenuContent(UMovieSceneDMXLibraryTrack* DMXTrack)
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection(NAME_None);
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("AddAllPatches", "Add all Patches"),
			LOCTEXT("AddAllPatchesTooltip", "Adds all Patches with at least one Fixture Function"),
			FSlateIcon(),
			FExecuteAction::CreateSP(this, &FDMXLibraryTrackEditor::HandleAddAllPatchesClicked, DMXTrack)
		);
	}
	MenuBuilder.EndSection();

	// Add an entry for each Library's Patch
	MenuBuilder.BeginSection(NAME_None, LOCTEXT("FixturePatchesHeading", "Fixture Patches"));
	{
		TArray<UDMXEntityFixturePatch*> Patches = DMXTrack->GetDMXLibrary()->GetEntitiesTypeCast<UDMXEntityFixturePatch>();
		Algo::SortBy(Patches, &UDMXEntityFixturePatch::Name, [](const FString& A, const FString& B)->bool
			{
				if (A.IsNumeric() && B.IsNumeric())
				{
					return FCString::Atof(*A) < FCString::Atof(*B);
				}
				return A < B;
			});

		for (UDMXEntityFixturePatch* Patch : Patches)
		{
			const FText PatchName = FText::FromString(Patch->GetDisplayName());
			MenuBuilder.AddMenuEntry(
				PatchName,
				FText::Format(LOCTEXT("AddPatchTrackTooltip", "Add track for {0}"), PatchName),
				FSlateIcon(),
				FExecuteAction::CreateSP(this, &FDMXLibraryTrackEditor::HandlePatchSelectedFromAddMenu, DMXTrack, Patch)
			);
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

/**
 * Searches for a Mode with at least one Fixture Function in the Patch giving
 * preference to the Patch's active Mode.
 * If no valid mode is found, returns INDEX_NONE.
 */
int32 GetValidPatchMode(UDMXEntityFixturePatch* InPatch)
{
	// Checking just for good measure. The functions that call this one would already have checked it
	if (InPatch == nullptr)
	{
		return INDEX_NONE;
	}

	UDMXEntityFixtureType* FixtureType = InPatch->GetFixtureType();
	if (!IsValid(FixtureType))
	{
		UE_LOGF(DMXLibraryTrackEditorLog, Warning, "%s: Fixture Patch has null parent type", __FUNCTION__);
		return INDEX_NONE;
	}

	// We'll search for a Mode with functions in it. Preferably, the current one from the Patch
	int32 ValidActiveMode = INDEX_NONE;

	for (int32 ModeIndex = 0; ModeIndex < FixtureType->Modes.Num(); ++ModeIndex)
	{
		const FDMXFixtureMode& Mode = FixtureType->Modes[ModeIndex];
		if (Mode.Functions.Num() > 0 ||
			Mode.FixtureMatrixConfig.CellAttributes.Num() > 0)
		{
			if (ModeIndex == InPatch->GetActiveModeIndex())
			{
				// This is the preferred mode. It's valid, so stop searching.
				ValidActiveMode = ModeIndex;
				break;
			}
		}
	}

	if (ValidActiveMode == INDEX_NONE)
	{
		UE_LOGF(DMXLibraryTrackEditorLog, Warning, "%s: No active mode set in %ls. Patch will not be recorded.", __FUNCTION__, *InPatch->Name);
	}

	return ValidActiveMode;
}

void FDMXLibraryTrackEditor::HandlePatchSelectedFromAddMenu(UMovieSceneDMXLibraryTrack* Track, UDMXEntityFixturePatch* Patch)
{
	if (Track->GetDMXLibrary() == nullptr || Patch == nullptr || !Patch->IsValidLowLevelFast())
	{
		return;
	}

	UMovieScene* MovieScene = GetSequencer()->GetFocusedMovieSceneSequence()->GetMovieScene();
	if (MovieScene->IsReadOnly())
	{
		return;
	}
	
	const FName PatchFName(*Patch->GetDisplayName());

	UMovieSceneSection* const* DMXLibrarySectionPtr = Track->GetAllSections().FindByPredicate([](UMovieSceneSection* Section)
		{
			return Section && Section->IsA(UMovieSceneDMXLibrarySection::StaticClass());
		});
	if (!DMXLibrarySectionPtr)
	{
		return;
	}

	UMovieSceneDMXLibrarySection* DMXSection = CastChecked<UMovieSceneDMXLibrarySection>(*DMXLibrarySectionPtr);

	// Don't add repeated Fixture Patches
	if (DMXSection->ContainsFixturePatch(Patch))
	{
		return;
	}

	// Only use Patches which have any mode with at least one Fixture Function
	const int32 ValidActiveMode = GetValidPatchMode(Patch);

	// Do we have no valid Modes at all?
	if (ValidActiveMode == INDEX_NONE)
	{
		UE_LOGF(DMXLibraryTrackEditorLog, Warning, "%s: Fixture Patch has no Modes with functions in it", __FUNCTION__);
		return;
	}

	const FScopedTransaction Transaction(FText::Format(LOCTEXT("AddPatchToTrackTransaction", "Add Patch {0} to track"), FText::FromName(PatchFName)));
	DMXSection->Modify();
	DMXSection->AddFixturePatch(Patch);

	// Do we need to use an Active Mode for the Patch that's not its active one?
	if (ValidActiveMode != Patch->GetActiveModeIndex())
	{
		DMXSection->SetFixturePatchActiveMode(Patch, ValidActiveMode);
	}

	GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
}

void FDMXLibraryTrackEditor::HandleAddAllPatchesClicked(UMovieSceneDMXLibraryTrack* Track)
{
	if (Track->GetDMXLibrary() == nullptr)
	{
		return;
	}

	UMovieScene* MovieScene = GetSequencer()->GetFocusedMovieSceneSequence()->GetMovieScene();
	if (MovieScene->IsReadOnly())
	{
		return;
	}

	UMovieSceneSection* const* DMXLibrarySectionPtr = Track->GetAllSections().FindByPredicate([](UMovieSceneSection* Section)
		{
			return Section && Section->IsA(UMovieSceneDMXLibrarySection::StaticClass());
		});
	if (!DMXLibrarySectionPtr)
	{
		return;
	}

	UMovieSceneDMXLibrarySection* DMXSection = CastChecked<UMovieSceneDMXLibrarySection>(*DMXLibrarySectionPtr);

	const TArray<UDMXEntityFixturePatch*>&& LibraryPatches = Track->GetDMXLibrary()->GetEntitiesTypeCast<UDMXEntityFixturePatch>();
	// Cache existing Patches in the section to not add repeated ones
	const TArray<UDMXEntityFixturePatch*>&& SectionPatches = DMXSection->GetFixturePatches();

	if (SectionPatches.Num() >= LibraryPatches.Num())
	{
		// Don't need to add any Patches.
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddAllPatchesToTrackTransaction", "Add all Patches to track"));
	DMXSection->Modify();

	// Add all non-repeated valid Patches
	for (UDMXEntityFixturePatch* Patch : LibraryPatches)
	{
		if (Patch == nullptr || !Patch->IsValidLowLevelFast())
		{
			continue;
		}

		// Don't add repeated Fixture Patches
		if (SectionPatches.Contains(Patch))
		{
			continue;
		}

		// Only use Patches which have any mode with at least one Fixture Function
		int32 ValidActiveMode = GetValidPatchMode(Patch);

		// Do we have no valid Modes at all?
		if (ValidActiveMode == INDEX_NONE)
		{
			UE_LOGF(DMXLibraryTrackEditorLog, Warning, "%s: Fixture Patch has no Modes with functions in it", __FUNCTION__);
			continue;
		}

		DMXSection->AddFixturePatch(Patch);

		// Do we need to use an Active Mode for the Patch that's not its active one?
		if (ValidActiveMode != Patch->GetActiveModeIndex())
		{
			DMXSection->SetFixturePatchActiveMode(Patch, ValidActiveMode);
		}
	}

	GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);

	return;
}

void FDMXLibraryTrackEditor::AddDMXLibraryTrackToSequence(const FAssetData& InAssetData)
{
	FSlateApplication::Get().DismissAllMenus();

	UDMXLibrary* Library = Cast<UDMXLibrary>(InAssetData.GetAsset());
	UMovieScene* MovieScene = GetFocusedMovieScene();
	if (Library == nullptr || MovieScene == nullptr)
	{
		return;
	}

	if (MovieScene->IsReadOnly())
	{
		return;
	}

	// Attempt to find an existing DMX track that animates this object
	for (UMovieSceneTrack* Track : MovieScene->GetTracks())
	{
		if (auto* DMXTrack = Cast<UMovieSceneDMXLibraryTrack>(Track))
		{
			if (DMXTrack->GetDMXLibrary() == Library)
			{
				return;
			}
		}
	}

	const FScopedTransaction Transaction(LOCTEXT("AddDMXTrackTransaction", "Add DMX Library Track"));

	MovieScene->Modify();
	UMovieSceneDMXLibraryTrack* Track = MovieScene->AddTrack<UMovieSceneDMXLibraryTrack>();
	check(Track);

	Track->SetDMXLibrary(Library);
	UMovieSceneSection* DMXSection = Track->CreateNewSection();
	DMXSection->SetRange(MovieScene->GetPlaybackRange());
	Track->AddSection(*DMXSection);

	if (GetSequencer().IsValid())
	{
		GetSequencer()->OnAddTrack(Track, FGuid());
	}
}

void FDMXLibraryTrackEditor::AddDMXLibraryTrackToSequenceEnterPressed(const TArray<FAssetData>& InAssetData)
{
	if (InAssetData.Num() > 0)
	{
		AddDMXLibraryTrackToSequence(InAssetData[0].GetAsset());
	}
}

bool FDMXLibraryTrackEditor::HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid)
{
	if (UDMXLibrary* Library = Cast<UDMXLibrary>(Asset))
	{
		AddDMXLibraryTrackToSequence(FAssetData(Library));
	}

	return false;
}

TSharedRef<ISequencerSection> FDMXLibraryTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	check(SupportsType(SectionObject.GetOuter()->GetClass()));
	return MakeShared<FDMXLibrarySection>(SectionObject, GetSequencer());
}

#undef LOCTEXT_NAMESPACE
