// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneTrackEditor.h"

struct FAssetData;
class UDMXEntityFixturePatch;
class UMovieSceneDMXLibraryTrack;
class UMovieSceneSection;

/**
 * Track editor for DMX Libraries.
 */
class FDMXLibraryTrackEditor
	: public FMovieSceneTrackEditor
{
public:

	/** Constructor. */
	FDMXLibraryTrackEditor(TSharedRef<ISequencer> InSequencer);

	/** Factory function */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

public:

	//~ ISequencerTrackEditor interface

	/** Provides the context menu custom options when right-clicking a DMX Library track */
	virtual void BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track) override;

	/** Adds the DMX Library track option to Sequencer's "Add Track" menu */
	virtual void BuildAddTrackMenu(FMenuBuilder& MenuBuilder) override;

	/** Returns whether this track editor is suitable for the passed in type of track */
	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override;

	/** Returns what types of Sequence the DMX Library track can be added to */
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;

	/** Gets the icon for the DMX Library track */
	virtual const FSlateBrush* GetIconBrush() const override;

	/** Builds the widget that goes on the right of the track node. In this case, the "+ Patch" menu. */
	virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;

	/** Handles when a DMX Library asset is dragged into the Sequence editor */
	virtual bool HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid) override;

	/** Returns editor customization of the sections */
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;

private:

	/** Provides the contents of the add Patch menu, displaying all Patches from this DMX Library. */
	TSharedRef<SWidget> OnGetAddPatchMenuContent(UMovieSceneDMXLibraryTrack* DMXTrack);

	/** Creates a Patch section using the selected Fixture Patch from the "+ Patch" menu */
	void HandlePatchSelectedFromAddMenu(UMovieSceneDMXLibraryTrack* Track, UDMXEntityFixturePatch* Patch);

	/** Adds all Library's Patches as channels of the DMX Library track */
	void HandleAddAllPatchesClicked(UMovieSceneDMXLibraryTrack* Track);

	/** Creates a new DMX Library track on the active Sequence, using the selected DMX Library asset */
	void AddDMXLibraryTrackToSequence(const FAssetData& InAssetData);
	void AddDMXLibraryTrackToSequenceEnterPressed(const TArray<FAssetData>& InAssetData);
};