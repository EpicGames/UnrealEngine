// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/MovieSceneDMXLibraryTrackRecorder.h"

#include "DMXEditorLog.h"
#include "DMXProtocolSettings.h"
#include "DMXStats.h"
#include "DMXSubsystem.h"
#include "DMXTypes.h"
#include "IO/DMXInputPort.h"
#include "IO/DMXOutputPort.h"
#include "IO/DMXRawListener.h"
#include "Library/DMXLibrary.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXEntityFixtureType.h"
#include "Sequencer/DMXAsyncDMXRecorder.h"
#include "Sequencer/MovieSceneDMXLibrarySection.h"
#include "Sequencer/MovieSceneDMXLibraryTrack.h"

#include "TakeRecorderSettings.h"
#include "Recorder/TakeRecorderParameters.h"
#include "Engine/Engine.h"
#include "Engine/TimecodeProvider.h"
#include "Features/IModularFeatures.h"
#include "Misc/App.h"
#include "Misc/ScopedSlowTask.h" 
#include "Modules/ModuleManager.h"
#include "MovieSceneFolder.h"


DECLARE_CYCLE_STAT(TEXT("Take recorder record sample"), STAT_DMXTakeRecorderRecordSample, STATGROUP_DMX);

#define LOCTEXT_NAMESPACE "MovieSceneDMXLibraryTrackRecorder"


TWeakObjectPtr<UMovieSceneDMXLibraryTrack> UMovieSceneDMXLibraryTrackRecorder::CreateTrack(UMovieScene* InMovieScene, UDMXLibrary* Library, const TArray<FDMXEntityFixturePatchRef>& InFixturePatchRefs, bool bInDiscardSamplesBeforeStart, bool bRecordNormalizedValues)
{
	check(Library);

	FixturePatchRefs = InFixturePatchRefs;

	// Only keep alive fixture patch refs
	FixturePatchRefs.RemoveAll([](const FDMXEntityFixturePatchRef& Ref) {
		UDMXEntityFixturePatch* FixturePatch = Ref.GetFixturePatch();
		return
			!IsValid(FixturePatch) ||
			!IsValid(FixturePatch->GetFixtureType());
		});

	MovieScene = InMovieScene;
	bDiscardSamplesBeforeStart = bInDiscardSamplesBeforeStart;

	DMXLibraryTrack = MovieScene->AddTrack<UMovieSceneDMXLibraryTrack>();
	DMXLibraryTrack->SetDMXLibrary(Library);

	DMXLibrarySection = Cast<UMovieSceneDMXLibrarySection>(DMXLibraryTrack->CreateNewSection());
	if (DMXLibrarySection.IsValid())
	{
		DMXLibrarySection->SetIsActive(false);
		DMXLibraryTrack->AddSection(*DMXLibrarySection);

		DMXLibrarySection->AddFixturePatches(FixturePatchRefs);

		DMXLibrarySection->bUseNormalizedValues = bRecordNormalizedValues;

		// Erase existing data in the track related to the Fixture Patches we're going to record.
		// This way, the user can record different Patches incrementally, one at a time.
		// TODO: Legacy code, I'm not sure if this ever made sense
		for (FDMXFixturePatchChannel& FixturePatchChannel : DMXLibrarySection->GetMutableFixturePatchChannels())
		{
			if (FixturePatchRefs.Contains(FixturePatchChannel.Reference.GetFixturePatch()))
			{
				for (FDMXFixtureFunctionChannel& FunctionChannel : FixturePatchChannel.FunctionChannels)
				{
					FunctionChannel.Channel.Reset();
				}
			}
		}

		// Resize the section to either it's remaining	keyframes range or 0
		DMXLibrarySection->SetRange(DMXLibrarySection->GetAutoSizeRange().Get(TRange<FFrameNumber>(0, 0)));

		// Make sure it starts at frame 0, in case Auto Size removed a piece of the start
		DMXLibrarySection->ExpandToFrame(0);

		// Mark the track as recording. Also prevents its evaluation from sending DMX data
		DMXLibrarySection->SetIsRecording(true);

		// Create an async dmx recorder
		AsyncDMXRecorder = MakeShared<FDMXAsyncDMXRecorder>(Library, DMXLibrarySection.Get());
	}
	
	return DMXLibraryTrack;
}

void UMovieSceneDMXLibraryTrackRecorder::SetSectionStartTimecodeImpl(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame)
{
	check(AsyncDMXRecorder.IsValid());

	if (DMXLibrarySection.IsValid() && DMXLibraryTrack.IsValid())
	{
		DMXLibrarySection->TimecodeSource = FMovieSceneTimecodeSource(InSectionStartTimecode);

		FTakeRecorderParameters Parameters;
		Parameters.User = GetDefault<UTakeRecorderUserSettings>()->Settings;
		Parameters.Project = GetDefault<UTakeRecorderProjectSettings>()->Settings;

		FFrameRate TickResolution = MovieScene->GetTickResolution();
		FFrameRate DisplayRate = MovieScene->GetDisplayRate();

		RecordStartTime = FApp::GetCurrentTime();
		RecordStartFrame = Parameters.Project.bStartAtCurrentTimecode ? FFrameRate::TransformTime(FFrameTime(InSectionStartTimecode.ToFrameNumber(DisplayRate)), DisplayRate, TickResolution).FloorToFrame() : MovieScene->GetPlaybackRange().GetLowerBoundValue();
	
		AsyncDMXRecorder->StartRecording(RecordStartTime, RecordStartFrame, TickResolution);
	}
}

UMovieSceneSection* UMovieSceneDMXLibraryTrackRecorder::GetMovieSceneSection() const
{
	return Cast<UMovieSceneSection>(DMXLibrarySection.Get());
}

void UMovieSceneDMXLibraryTrackRecorder::StopRecordingImpl()
{
	check(AsyncDMXRecorder.IsValid());

	// Stop the async recorder
	AsyncDMXRecorder->StopRecording();

	// Flag the section as no longer recording
	DMXLibrarySection->SetIsRecording(false);
}

void UMovieSceneDMXLibraryTrackRecorder::FinalizeTrackImpl()
{
	check(AsyncDMXRecorder.IsValid());

	if(UMovieSceneDMXLibrarySection* LibrarySection = DMXLibrarySection.Get())
	{
		// Write keyframes
		{
			TArray<FDMXFunctionChannelData> FunctionChannelData = AsyncDMXRecorder->GetRecordedData();
			int32 TotalNumChannels = FunctionChannelData.Num();

			FScopedSlowTask WriteKeyframesTask(TotalNumChannels, FText::Format(LOCTEXT("WriteRecordedDMXData", "Writing Sequencer Channels for {0} DMX Attributes"), TotalNumChannels));
			const bool bShowCancelButton = true;
			const bool bAllowInPie = true;
			WriteKeyframesTask.MakeDialog(bShowCancelButton, bAllowInPie);

			for (FDMXFunctionChannelData& SingleChannelData : FunctionChannelData)
			{
				if (FDMXFixtureFunctionChannel* FunctionChannel = SingleChannelData.TryGetFunctionChannel(DMXLibrarySection.Get()))
				{
					// If bDiscardSamplesBeforeStart is set, remove samples before start
					if (bDiscardSamplesBeforeStart)
					{
						const int32 LastDiscardSampleIndex = SingleChannelData.Times.IndexOfByPredicate([this](const FFrameNumber& FrameNumber)
							{
								return FrameNumber < DMXLibrarySection->GetInclusiveStartFrame();
							});

						for (int32 DiscardIndex = 0; DiscardIndex < LastDiscardSampleIndex; DiscardIndex++)
						{
							SingleChannelData.Times.RemoveAt(DiscardIndex);
							SingleChannelData.Values.RemoveAt(DiscardIndex);
						}
					}

					FunctionChannel->Channel.AddKeys(SingleChannelData.Times, SingleChannelData.Values);
				}

				WriteKeyframesTask.EnterProgressFrame();
			}
		}

		if (DMXLibrarySection.IsValid())
		{
			// Set the final range 
			TOptional<TRange<FFrameNumber> > DefaultSectionLength = DMXLibrarySection->GetAutoSizeRange();
			if (DefaultSectionLength.IsSet())
			{
				DMXLibrarySection->SetRange(DefaultSectionLength.GetValue());
			}

			// Rebuild the section's cache so it can be played back right away
			DMXLibrarySection->RebuildPlaybackCache();

			// Activate the section
			DMXLibrarySection->SetIsActive(true);
		}
	}
}

void UMovieSceneDMXLibraryTrackRecorder::RecordSampleImpl(const FQualifiedFrameTime& CurrentFrameTime)
{
	SCOPE_CYCLE_COUNTER(STAT_DMXTakeRecorderRecordSample);

	// Expand the section to the new length
	FFrameRate	 TickResolution = MovieScene->GetTickResolution();
	FFrameNumber CurrentFrame = CurrentFrameTime.ConvertTo(TickResolution).FloorToFrame();
	DMXLibrarySection->ExpandToFrame(CurrentFrame);

	// Recording happens in AsyncDMXRecorder
}

bool UMovieSceneDMXLibraryTrackRecorder::LoadRecordedFile(const FString& FileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap, TFunction<void()> InCompletionCallback)
{
	UE_LOG_DMXEDITOR(Warning, TEXT("Loading recorded file for DMX Library tracks is not supported."));
	return false;
}

#undef LOCTEXT_NAMESPACE
