// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TrackRecorders/MovieSceneTrackRecorder.h"

#include "Library/DMXEntityReference.h"

#include "CoreMinimal.h"
#include "MovieScene.h"
#include "Channels/MovieSceneFloatChannel.h"

#include "MovieSceneDMXLibraryTrackRecorder.generated.h"

struct FDMXFixtureFunctionChannel;
class  FDMXAsyncDMXRecorder;
class  FDMXRawListener;
class  UDMXSubsystem;
class  UMovieSceneDMXLibraryTrack;
class  UMovieSceneDMXLibrarySection;

struct FFrameNumber;
class  UMovieSceneSection;
class  UMovieSceneTrackRecorderSettings;


/**
* Track recorder implementation for DMX libraries
* Reuses logic of Animation/LiveLink Plugin in many areas.
*/
UCLASS(BlueprintType)
class DMXEDITOR_API UMovieSceneDMXLibraryTrackRecorder
	: public UMovieSceneTrackRecorder
{
	GENERATED_BODY()

public:
	// UMovieSceneTrackRecorder Interface
	virtual void RecordSampleImpl(const FQualifiedFrameTime& CurrentFrameTime) override;
	virtual void FinalizeTrackImpl() override;
	virtual void SetSectionStartTimecodeImpl(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame) override;
	virtual UMovieSceneSection* GetMovieSceneSection() const override;
	virtual void StopRecordingImpl() override;
	virtual bool LoadRecordedFile(const FString& InFileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap, TFunction<void()> InCompletionCallback) override;
	// ~UMovieSceneTrackRecorder Interface

public:
	/** Creates a track. We don't call UMovieSceneTrackRecorder::CreateTrack or CreateTrackImpl since that expects an  ObjectToRecord and a GUID which isn't needed. */
	TWeakObjectPtr<UMovieSceneDMXLibraryTrack> CreateTrack(UMovieScene* InMovieScene, UDMXLibrary* Library, const TArray<FDMXEntityFixturePatchRef>& InFixturePatchRefs, bool bDiscardSamplesBeforeStart, bool bRecordNormalizedValues);

private:
	/** Asnyc Recorder for DMX */
	TSharedPtr<FDMXAsyncDMXRecorder> AsyncDMXRecorder;

	/** The raw listeners for the ports of the library */
	TSet<TSharedPtr<FDMXRawListener>> RawListeners;

	/** Whether or not we use timecode time or world time*/
	bool bUseSourceTimecode;

	UPROPERTY()
	TArray<FDMXEntityFixturePatchRef> FixturePatchRefs;

	/** The DMXLibrary Track to record onto */
	TWeakObjectPtr<UMovieSceneDMXLibraryTrack> DMXLibraryTrack;

	/** Sections to record to on each track*/
	TWeakObjectPtr<UMovieSceneDMXLibrarySection> DMXLibrarySection;

	/** The time at the start of this recording section */
	double RecordStartTime;

	/** The frame at the start of this recording section */
	FFrameNumber RecordStartFrame;

	/** If true, discards samples that were recorded before the actual recording start frame */
	bool bDiscardSamplesBeforeStart;
};

