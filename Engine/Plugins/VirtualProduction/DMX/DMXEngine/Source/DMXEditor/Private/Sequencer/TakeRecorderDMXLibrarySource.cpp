// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/TakeRecorderDMXLibrarySource.h"

#include "DMXEditorLog.h"
#include "Library/DMXLibrary.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Sequencer/MovieSceneDMXLibrarySection.h"
#include "Sequencer/MovieSceneDMXLibraryTrack.h"
#include "Sequencer/MovieSceneDMXLibraryTrackRecorder.h"

#include "LevelSequence.h"
#include "MovieSceneFolder.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneChannelTraits.h"
#include "Framework/Notifications/NotificationManager.h" 
#include "Widgets/Notifications/SNotificationList.h"


#define LOCTEXT_NAMESPACE "TakeRecorderDMXLibrarySource"

UTakeRecorderDMXLibrarySource::UTakeRecorderDMXLibrarySource(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
	, DMXLibrary(nullptr)
	, bRecordNormalizedValues(true)
	, bDiscardSamplesBeforeStart(true)
{
	// DMX Tracks are blue
	TrackTint = FColor(0.0f, 125.0f, 255.0f, 65.0f);
}

void UTakeRecorderDMXLibrarySource::AddAllPatches()
{
	if (DMXLibrary == nullptr || !DMXLibrary->IsValidLowLevelFast())
	{
		return;
	}

	TArray<UDMXEntityFixturePatch*> FixturePatches = DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>();
	
	// Sort the patches by universe and starting channel
	FixturePatches.Sort([](UDMXEntityFixturePatch& FirstPatch, UDMXEntityFixturePatch& SecondPatch) {
		if (FirstPatch.GetUniverseID() < SecondPatch.GetUniverseID())
		{
			return true;
		}
		else if (FirstPatch.GetUniverseID() > SecondPatch.GetUniverseID())
		{
			return false;
		}

		return FirstPatch.GetStartingChannel() <= SecondPatch.GetStartingChannel();
	});

	// Update fixture patch refs
	FixturePatchRefs.Reset();
	for (UDMXEntityFixturePatch* FixturePatch : FixturePatches)
	{
		FixturePatchRefs.Add(FDMXEntityFixturePatchRef(FixturePatch));
	}
}

TArray<UTakeRecorderSource*> UTakeRecorderDMXLibrarySource::PreRecording(ULevelSequence* InSequence, FMovieSceneSequenceID InSequenceID, ULevelSequence* InRootSequence, FManifestSerializer* InManifestSerializer)
{
	if (DMXLibrary)
	{
		UMovieScene* MovieScene = InRootSequence->GetMovieScene();
		TrackRecorder = NewObject<UMovieSceneDMXLibraryTrackRecorder>();
		CachedDMXLibraryTrack = TrackRecorder->CreateTrack(MovieScene, DMXLibrary, FixturePatchRefs, bDiscardSamplesBeforeStart, bRecordNormalizedValues);
	}
	else
	{
		UE_LOGF(LogDMXProtocol, Error, "No library specified for DMX Track Recorder.");
	}

	return TArray<UTakeRecorderSource*>();
}

void UTakeRecorderDMXLibrarySource::TickRecording(const FQualifiedFrameTime& CurrentTime)
{
	if (TrackRecorder)
	{
		TrackRecorder->RecordSample(CurrentTime);
	}
}

void UTakeRecorderDMXLibrarySource::StartRecording(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame, class ULevelSequence* InSequence)
{
	if (TrackRecorder)
	{
		TrackRecorder->SetSectionStartTimecode(InSectionStartTimecode, InSectionFirstFrame);
	}
}

void UTakeRecorderDMXLibrarySource::StopRecording(class ULevelSequence* InSequence)
{
	if (TrackRecorder)
	{
		TrackRecorder->StopRecording();
	}
}

TArray<UTakeRecorderSource*> UTakeRecorderDMXLibrarySource::PostRecording(class ULevelSequence* InSequence, ULevelSequence* InRootSequence, const bool bCancelled)
{
	if (TrackRecorder)
	{
		TrackRecorder->FinalizeTrack();
	}

	TrackRecorder = nullptr;
	return TArray<UTakeRecorderSource*>();
}

void UTakeRecorderDMXLibrarySource::AddContentsToFolder(UMovieSceneFolder* InFolder)
{
	if (CachedDMXLibraryTrack.IsValid())
	{
		InFolder->AddChildTrack(CachedDMXLibraryTrack.Get());
	}
}

FText UTakeRecorderDMXLibrarySource::GetDisplayTextImpl() const
{
	if (DMXLibrary != nullptr && DMXLibrary->IsValidLowLevelFast())
	{
		return FText::FromString(DMXLibrary->GetName());
	}

	return LOCTEXT("Display Text", "Null DMX Library");
}

FText UTakeRecorderDMXLibrarySource::GetAddSourceDisplayTextImpl() const
{
    return LOCTEXT("TakeRecorderDisplayName", "DMX Library");
}

void UTakeRecorderDMXLibrarySource::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTakeRecorderDMXLibrarySource, FixturePatchRefs))
	{
		// Prevent from selecting the same patch twice
		int32 ChangedIndex = PropertyChangedEvent.GetArrayIndex(PropertyName.ToString());
		if (FixturePatchRefs.IsValidIndex(ChangedIndex))
		{
			UDMXEntityFixturePatch* SelectedPatch = FixturePatchRefs[ChangedIndex].GetFixturePatch();

			if (SelectedPatch)
			{
				const bool bIsDuplicate = [this, ChangedIndex, SelectedPatch]() -> const bool
				{
					for (int32 IdxPatchRef = 0; IdxPatchRef < FixturePatchRefs.Num(); IdxPatchRef++)
					{
						if (IdxPatchRef == ChangedIndex)
						{
							continue;
						}

						if (FixturePatchRefs[IdxPatchRef].GetFixturePatch() == SelectedPatch)
						{
							return true;
						}
					}

					return false;
				}();

				if (bIsDuplicate)
				{
					FNotificationInfo Info(FText::Format(LOCTEXT("SetFixturePatchTwice", "{0} already has a track."), FText::FromString(SelectedPatch->GetDisplayName())));
					Info.ExpireDuration = 5.f;
					FSlateNotificationManager::Get().AddNotification(Info);

					// Reset the patch ref
					FixturePatchRefs[ChangedIndex].SetEntity(nullptr);
				}
			}
		}

		// Fix DMX Library reference on new Patch refs
		ResetPatchesLibrary();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UTakeRecorderDMXLibrarySource, AddAllPatchesDummy))
	{
		// Fix DMX Library reference on new Patch refs
		ResetPatchesLibrary();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UTakeRecorderDMXLibrarySource, DMXLibrary))
	{
		// If the library has changed, delete all Patch refs. They won't be accessible
		// anymore since they were children of a different DMX Library
		if (FixturePatchRefs.Num() > 0)
		{
			if (FixturePatchRefs[0].DMXLibrary != DMXLibrary)
			{
				FixturePatchRefs.Empty();
			}
		}
	}
}

void UTakeRecorderDMXLibrarySource::PostLoad()
{
	Super::PostLoad();

	// Make sure the Refs don't display the DMX Library picker.
	ResetPatchesLibrary();
}

void UTakeRecorderDMXLibrarySource::ResetPatchesLibrary()
{
	for (FDMXEntityFixturePatchRef& PatchRef : FixturePatchRefs)
	{
		PatchRef.bDisplayLibraryPicker = false;
		PatchRef.DMXLibrary = DMXLibrary;
	}
}

#undef LOCTEXT_NAMESPACE
