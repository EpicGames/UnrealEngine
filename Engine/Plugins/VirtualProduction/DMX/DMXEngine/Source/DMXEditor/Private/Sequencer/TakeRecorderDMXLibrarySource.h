// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TakeRecorderSource.h"
#include "Library/DMXEntityReference.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "TakeRecorderDMXLibrarySource.generated.h"

class ULevelSequence;
class UMovieSceneFolder;
class UMovieSceneDMXLibraryTrack;
class UMovieSceneDMXLibraryTrackRecorder;

/**
 * Empty struct to have it customized in DetailsView to display a button on
 * the DMX Take Recorder properties. This is a required hack to customize the
 * properties in the TakeRecorder DetailsView because it has a customization
 * that overrides any class customization. So we need to tackle individual
 * property types instead.
 */
USTRUCT()
struct FAddAllPatchesButton
{
	GENERATED_BODY()
};

/** A recording source for DMX data related to a DMX Library */
UCLASS(Category = "DMX")
class UTakeRecorderDMXLibrarySource
	: public UTakeRecorderSource
{
public:
	GENERATED_BODY()

	UTakeRecorderDMXLibrarySource(const FObjectInitializer& ObjInit);

	/** DMX Library to record Patches' Fixture Functions from */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Source", meta = (DisplayName = "DMX Library"))
	TObjectPtr<UDMXLibrary> DMXLibrary;

	/**
	 * Dummy property to be replaced with the "Add all patches" button.
	 * @see FAddAllPatchesButton
	 */
	UPROPERTY(EditAnywhere, Transient, Category = "My Category", meta = (DisplayName = ""))
	FAddAllPatchesButton AddAllPatchesDummy;

	/** The Fixture Patches to record from the selected Library */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Source", meta = (DisplayName = "Fixture Patches"))
	TArray<FDMXEntityFixturePatchRef> FixturePatchRefs;
		
	/** 
	 * If true, all values are recorded as normalized values (0.0 to 1.0).
	 * 
	 * If false, values are recorded as absolute values, depending on the data type of a patch:
	 * 0-255 for 8bit, 0-65'536 for 16bit, 0-16'777'215 for 24bit. 32bit is not fully supported in this mode.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Scene")
	bool bRecordNormalizedValues;

protected:
	// ~Begin UTakeRecorderSource Interface
	virtual bool SupportsSubscenes() const override { return false; }
	// ~End UTakeRecorderSource Interface

public:
	/** Adds all Patches from the active DMX Library as recording sources */
	UFUNCTION(BlueprintCallable, Category = "DMX")
	void AddAllPatches();

private:
	// ~Begin UTakeRecorderSource Interface
	virtual TArray<UTakeRecorderSource*> PreRecording(ULevelSequence* InSequence, FMovieSceneSequenceID InSequenceID, ULevelSequence* InRootSequence, FManifestSerializer* InManifestSerializer) override;
	virtual void StartRecording(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame, class ULevelSequence* InSequence) override;
	virtual void StopRecording(class ULevelSequence* InSequence) override;
	virtual void TickRecording(const FQualifiedFrameTime& CurrentTime) override;
	virtual TArray<UTakeRecorderSource*> PostRecording(class ULevelSequence* InSequence, ULevelSequence* InRootSequence, const bool bCancelled) override;
	virtual void AddContentsToFolder(UMovieSceneFolder* InFolder) override;
	virtual FText GetDisplayTextImpl() const override;
	virtual FText GetAddSourceDisplayTextImpl() const override;	
	// ~End UTakeRecorderSource

	// ~Begin UObject Interface
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostLoad() override;
	// ~End UObject Interface

	/**
	 * Make sure all EntityRefs don't display their Library property and
	 * that they use this source's DMX Library as their library.
	 */
	void ResetPatchesLibrary();

private:
	/** Whether to discard samples with timecode that occurs before the start of recording*/
	bool bDiscardSamplesBeforeStart;

	/** Track recorder used by this source */
	UPROPERTY()
	TObjectPtr<UMovieSceneDMXLibraryTrackRecorder> TrackRecorder;

	/**
	 * Stores an existing DMX Library track in the Sequence to be recorded or
	 * a new one created for recording. Set during PreRecording.
	 */
	TWeakObjectPtr<UMovieSceneDMXLibraryTrack> CachedDMXLibraryTrack;
};
