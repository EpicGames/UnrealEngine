// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tabs/DMXEditorTabFactories.h"

#include "DMXEditor.h"
#include "DMXEditorTabNames.h"
#include "DMXEditorStyle.h"
#include "Widgets/LibrarySettings/SDMXLibraryEditorTab.h"
#include "Widgets/FixtureType/SDMXFixtureTypeEditor.h"
#include "Widgets/FixturePatch/SDMXFixturePatchEditor.h"

#include "Textures/SlateIcon.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Docking/SDockTab.h"


#define LOCTEXT_NAMESPACE "DMXWorkflowTabFactory"

TSharedRef<SDockTab> FDMXEditorPropertyTabSummoner::SpawnTab(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedRef<SDockTab> NewTab = FWorkflowTabFactory::SpawnTab(Info);
	NewTab->SetTag(GetIdentifier());

	TSharedPtr<FDMXEditor> DMXEditor = StaticCastSharedPtr<FDMXEditor>(HostingApp.Pin());

	return NewTab;
}

FDMXLibraryEditorTabSummoner::FDMXLibraryEditorTabSummoner(TSharedPtr<FAssetEditorToolkit> InHostingApp)
	: FDMXEditorPropertyTabSummoner(UE::DMX::FDMXEditorTabNames::DMXLibraryEditor, InHostingApp)
{
	TabLabel = LOCTEXT("DMXLibraryEditorTabName", "Library Settings");
	TabIcon = FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.DMXLibrarySettings");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("DMXLibraryEditorTabView", "Library Settings");
	ViewMenuTooltip = LOCTEXT("DMXLibraryEditorTabMenuTooltip", "Show the Library Settings view");
}

TSharedRef<SWidget> FDMXLibraryEditorTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FDMXEditor> DMXEditor = StaticCastSharedPtr<FDMXEditor>(HostingApp.Pin());

	return DMXEditor->GetDMXLibraryEditorTab();
}

FText FDMXLibraryEditorTabSummoner::GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const
{
	return LOCTEXT("DMXLibraryEditorTabTooltip", "Settings specific to this Library");
}

FDMXEditorFixtureTypesSummoner::FDMXEditorFixtureTypesSummoner(TSharedPtr<FAssetEditorToolkit> InHostingApp)
	: FDMXEditorPropertyTabSummoner(UE::DMX::FDMXEditorTabNames::DMXFixtureTypesEditor, InHostingApp)
{
	TabLabel = LOCTEXT("DMXFixtureTypesTabLabel", "Fixture Types");
	TabIcon = FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.FixtureType");
	
	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("DMXFixtureTypesView", "Fixture Types");
	ViewMenuTooltip = LOCTEXT("DMXFixtureTypesViewTooltip", "Show the fixture types view");
}

TSharedRef<SWidget> FDMXEditorFixtureTypesSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FDMXEditor> DMXEditor = StaticCastSharedPtr<FDMXEditor>(HostingApp.Pin());

	return DMXEditor->GetFixtureTypeEditor();
}

FDMXEditorFixturePatchSummoner::FDMXEditorFixturePatchSummoner(TSharedPtr<FAssetEditorToolkit> InHostingApp)
	: FDMXEditorPropertyTabSummoner(UE::DMX::FDMXEditorTabNames::DMXFixturePatchEditor, InHostingApp)
{
	TabLabel = LOCTEXT("DMXFixturePatchTabLabel", "Fixture Patch");
	TabIcon = FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.FixturePatch");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("DMXFixturePatchView", "Fixture Patch");
	ViewMenuTooltip = LOCTEXT("DMXFixturePatchViewTooltip", "Show the fixture patch view");
}

TSharedRef<SWidget> FDMXEditorFixturePatchSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FDMXEditor> DMXEditor = StaticCastSharedPtr<FDMXEditor>(HostingApp.Pin());

	return DMXEditor->GetFixturePatchEditor();
}

#undef LOCTEXT_NAMESPACE
