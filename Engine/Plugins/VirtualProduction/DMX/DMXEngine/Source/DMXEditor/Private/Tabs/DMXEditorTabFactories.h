// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/SWidget.h"
#include "Textures/SlateIcon.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

#define LOCTEXT_NAMESPACE "DMXEditorTabFactories"

class FAssetEditorToolkit;
class SDockTab;

struct FDMXEditorPropertyTabSummoner : public FWorkflowTabFactory
{
public:
	FDMXEditorPropertyTabSummoner(const FName& InIdentifier, TSharedPtr<class FAssetEditorToolkit> InHostingApp)
		: FWorkflowTabFactory(InIdentifier, InHostingApp)
	{
	}

	virtual TSharedRef<SDockTab> SpawnTab(const FWorkflowTabSpawnInfo& Info) const override;
};

struct FDMXLibraryEditorTabSummoner : public FDMXEditorPropertyTabSummoner
{
public:
	FDMXLibraryEditorTabSummoner(TSharedPtr<FAssetEditorToolkit> InHostingApp);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override;
};

struct FDMXEditorFixtureTypesSummoner : public FDMXEditorPropertyTabSummoner
{
public:
	FDMXEditorFixtureTypesSummoner(TSharedPtr<FAssetEditorToolkit> InHostingApp);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override
	{
		return LOCTEXT("DMXFixtureTypesTab", "Fixture Types");
	}
};

struct FDMXEditorFixturePatchSummoner : public FDMXEditorPropertyTabSummoner
{
public:
	FDMXEditorFixturePatchSummoner(TSharedPtr<FAssetEditorToolkit> InHostingApp);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override
	{
		return LOCTEXT("DMXFixturePatchTab", "Fixture Patch");
	}
};

#undef LOCTEXT_NAMESPACE
