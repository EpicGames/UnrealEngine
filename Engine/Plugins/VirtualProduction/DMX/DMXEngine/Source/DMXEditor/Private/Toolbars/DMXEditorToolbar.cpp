// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolbars/DMXEditorToolbar.h"

#include "DMXEditor.h"
#include "DMXEditorStyle.h"
#include "Commands/DMXEditorCommands.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "KismetToolbar"

void FDMXEditorToolbar::AddCompileToolbar(TSharedPtr<FExtender> Extender)
{
	TSharedPtr<FDMXEditor> DMXEditorPtr = DMXEditor.Pin();

	Extender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		DMXEditorPtr->GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateSP(this, &FDMXEditorToolbar::FillDMXLibraryToolbar));
}

FSlateIcon FDMXEditorToolbar::GetStatusImage() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Status.Good");
}

void FDMXEditorToolbar::FillDMXLibraryToolbar(FToolBarBuilder& ToolbarBuilder)
{
	ToolbarBuilder.BeginSection("DMXLibraryToolbar");
	{
		ToolbarBuilder.AddToolBarButton(
			FDMXEditorCommands::Get().ImportDMXLibrary,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.DMXLibraryToolbar.Import"));

		ToolbarBuilder.AddToolBarButton(
			FDMXEditorCommands::Get().ExportDMXLibrary,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.DMXLibraryToolbar.Export"));
	}
	ToolbarBuilder.EndSection();
}

#undef LOCTEXT_NAMESPACE
