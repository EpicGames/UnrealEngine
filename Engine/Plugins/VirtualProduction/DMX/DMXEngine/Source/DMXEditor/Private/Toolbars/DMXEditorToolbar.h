// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/SWidget.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/Commands.h"

class FDMXEditor;
class FExtender;
class FMenuBuilder;
class FToolBarBuilder;

class FDMXEditorToolbar : public TSharedFromThis<FDMXEditorToolbar>
{
public:
	FDMXEditorToolbar(TSharedPtr<FDMXEditor> InDMXEditor)
		: DMXEditor(InDMXEditor) {}

	void AddCompileToolbar(TSharedPtr<FExtender> Extender);

	/** Returns the current status icon for the blueprint being edited */
	FSlateIcon GetStatusImage() const;

private:
	void FillDMXLibraryToolbar(FToolBarBuilder& ToolbarBuilder);

protected:
	/** Pointer back to the blueprint editor tool that owns us */
	TWeakPtr<FDMXEditor> DMXEditor;
};

