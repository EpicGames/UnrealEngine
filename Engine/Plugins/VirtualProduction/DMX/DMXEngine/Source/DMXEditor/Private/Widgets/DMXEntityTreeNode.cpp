// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/DMXEntityTreeNode.h"

#include "DMXRuntimeUtils.h"
#include "Library/DMXEntity.h"


#define LOCTEXT_NAMESPACE "FDMXEntityTreeNodeBase"

FDMXEntityTreeNodeBase::FDMXEntityTreeNodeBase(FDMXEntityTreeNodeBase::ENodeType InNodeType)
	: NodeType(InNodeType)
{}

FDMXEntityTreeNodeBase::ENodeType FDMXEntityTreeNodeBase::GetNodeType() const
{
	return NodeType;
}

void FDMXEntityTreeNodeBase::AddChild(TSharedPtr<FDMXEntityTreeNodeBase> InChildPtr)
{
	if (InChildPtr.IsValid())
	{
		// unparent from previous parent
		if (InChildPtr->GetParent().IsValid())
		{
			InChildPtr->RemoveFromParent();
		}

		InChildPtr->ParentNodePtr = SharedThis(this);
		Children.Add(InChildPtr);
	}
}

void FDMXEntityTreeNodeBase::RemoveChild(TSharedPtr<FDMXEntityTreeNodeBase> InChildPtr)
{
	if (InChildPtr.IsValid())
	{
		if (Children.Contains(InChildPtr))
		{
			Children.Remove(InChildPtr);
			InChildPtr->ParentNodePtr = nullptr;
		}
	}
}

void FDMXEntityTreeNodeBase::RemoveFromParent()
{
	if (GetParent().IsValid())
	{
		GetParent().Pin()->RemoveChild(SharedThis(this));
	}
}

const TArray<TSharedPtr<FDMXEntityTreeNodeBase>>& FDMXEntityTreeNodeBase::GetChildren() const
{
	return Children;
}

void FDMXEntityTreeNodeBase::ClearChildren()
{
	Children.Empty(0);
}

void FDMXEntityTreeNodeBase::SortChildren()
{
	Children.HeapSort([](const TSharedPtr<FDMXEntityTreeNodeBase>& Left, const TSharedPtr<FDMXEntityTreeNodeBase>& Right)->bool
		{
			if (Left.IsValid() && Right.IsValid())
			{
				return *Left < *Right;
			}
			return false;
		});
}

void FDMXEntityTreeNodeBase::SortChildren(TFunction<bool (const TSharedPtr<FDMXEntityTreeNodeBase>&, const TSharedPtr<FDMXEntityTreeNodeBase>&)> Predicate)
{
	Children.Sort(Predicate);
}

bool FDMXEntityTreeNodeBase::IsFlaggedForFiltration() const
{
	// if Unknown, throws an error message
	return ensureMsgf(FilterFlags != (uint8)EFilteredState::Unknown, TEXT("Querying a bad filtration state.")) 
		? (FilterFlags & (uint8)EFilteredState::FilteredInMask) == 0 
		: false;
}

void FDMXEntityTreeNodeBase::UpdateCachedFilterState(bool bMatchesFilter, bool bUpdateParent)
{
	bool bFlagsChanged = false;
	if ((FilterFlags & EFilteredState::Unknown) == EFilteredState::Unknown)
	{
		FilterFlags = EFilteredState::FilteredOut;
		bFlagsChanged = true;
	}

	if (bMatchesFilter)
	{
		bFlagsChanged |= (FilterFlags & EFilteredState::MatchesFilter) == 0;
		FilterFlags |= EFilteredState::MatchesFilter;
	}
	else
	{
		bFlagsChanged |= (FilterFlags & EFilteredState::MatchesFilter) != 0;
		FilterFlags &= ~EFilteredState::MatchesFilter;
	}

	const bool bHadChildMatch = (FilterFlags & EFilteredState::ChildMatches) != 0;
	// refresh the cached child state (don't update the parent, we'll do that below if it's needed)
	RefreshCachedChildFilterState(/*bUpdateParent =*/false);

	bFlagsChanged |= bHadChildMatch != ((FilterFlags & EFilteredState::ChildMatches) != 0);
	if (bUpdateParent && bFlagsChanged)
	{
		ApplyFilteredStateToParent();
	}
}

void FDMXEntityTreeNodeBase::SetExpansionState(bool bNewExpansionState)
{
	bShouldBeExpanded = bNewExpansionState;
}

void FDMXEntityTreeNodeBase::SetWarningStatus(const FText& InWarningToolTip)
{
	WarningToolTip = InWarningToolTip;
}

void FDMXEntityTreeNodeBase::SetErrorStatus(const FText& InErrorToolTip)
{
	ErrorToolTip = InErrorToolTip;
}

bool FDMXEntityTreeNodeBase::operator<(const FDMXEntityTreeNodeBase& Other) const
{
	const FString& ThisName = GetDisplayNameString();
	const FString& OtherName = Other.GetDisplayNameString();
	if (ThisName.IsNumeric() && OtherName.IsNumeric())
	{
		return FCString::Atoi(*ThisName) < FCString::Atoi(*OtherName);
	}
	
	// If the names are strings with numbers at the end, separate them to compare name then number
	FString NameOnlyThis;
	FString NameOnlyOther;
	int32 NumberThis = 0;
	int32 NumberOther = 0;
	if (FDMXRuntimeUtils::GetNameAndIndexFromString(ThisName, NameOnlyThis, NumberThis) && 
		FDMXRuntimeUtils::GetNameAndIndexFromString(OtherName, NameOnlyOther, NumberOther) && 
		NameOnlyThis.Equals(NameOnlyOther))
	{
		return NumberThis < NumberOther;
	}

	return ThisName < OtherName;
}

void FDMXEntityTreeNodeBase::RefreshCachedChildFilterState(bool bUpdateParent)
{
	const bool bContainedMatch = !IsFlaggedForFiltration();

	FilterFlags &= ~EFilteredState::ChildMatches;
	for (TSharedPtr<FDMXEntityTreeNodeBase> Child : Children)
	{
		if (!Child->IsFlaggedForFiltration())
		{
			FilterFlags |= EFilteredState::ChildMatches;
			break;
		}
	}
	const bool bContainsMatch = !IsFlaggedForFiltration();

	const bool bStateChange = bContainedMatch != bContainsMatch;
	if (bUpdateParent && bStateChange)
	{
		ApplyFilteredStateToParent();
	}
}

void FDMXEntityTreeNodeBase::ApplyFilteredStateToParent()
{
	FDMXEntityTreeNodeBase* Child = this;

	while (Child->ParentNodePtr.IsValid())
	{
		FDMXEntityTreeNodeBase* Parent = Child->ParentNodePtr.Pin().Get();

		if (!IsFlaggedForFiltration())
		{
			if ((Parent->FilterFlags & EFilteredState::ChildMatches) == 0)
			{
				Parent->FilterFlags |= EFilteredState::ChildMatches;
			}
			else
			{
				// all parents from here on up should have the flag
				break;
			}
		}
		// have to see if this was the only child contributing to this flag
		else if (Parent->FilterFlags & EFilteredState::ChildMatches)
		{
			Parent->FilterFlags &= ~EFilteredState::ChildMatches;
			for (const TSharedPtr<FDMXEntityTreeNodeBase>& Sibling : Parent->Children)
			{
				if (Sibling.Get() == Child)
				{
					continue;
				}

				if (Sibling->FilterFlags & EFilteredState::FilteredInMask)
				{
					Parent->FilterFlags |= EFilteredState::ChildMatches;
					break;
				}
			}

			if (Parent->FilterFlags & EFilteredState::ChildMatches)
			{
				// another child added the flag back
				break;
			}
		}
		Child = Parent;
	}
}

FDMXEntityTreeRootNode::FDMXEntityTreeRootNode()
	: FDMXEntityTreeNodeBase(FDMXEntityTreeNodeBase::ENodeType::RootNode)
{}

FString FDMXEntityTreeRootNode::GetDisplayNameString() const
{
	return FString(TEXT("Root Node"));
}

FText FDMXEntityTreeRootNode::GetDisplayNameText() const
{
	return LOCTEXT("RootNodeName", "Root Node");
}

FDMXEntityTreeEntityNode::FDMXEntityTreeEntityNode(UDMXEntity* InEntity)
	: FDMXEntityTreeNodeBase(FDMXEntityTreeNodeBase::ENodeType::EntityNode)
{
	DMXEntity = InEntity;
}

FString FDMXEntityTreeEntityNode::GetDisplayNameString() const
{
	if (UDMXEntity* Entity = GetEntity())
	{
		return Entity->GetDisplayName();
	}
	return TEXT("null entity");
}

FText FDMXEntityTreeEntityNode::GetDisplayNameText() const
{
	if (UDMXEntity* Entity = GetEntity())
	{
		return FText::FromString(Entity->GetDisplayName());
	}

	return LOCTEXT("NullEntityError", "Entity is null");
}

UDMXEntity* FDMXEntityTreeEntityNode::GetEntity() const
{
	return DMXEntity.Get();
}


FDMXEntityTreeCategoryNode::FDMXEntityTreeCategoryNode(ECategoryType InCategoryType, FText InCategoryName, const FText& InToolTip /*= FText::GetEmpty()*/)
	: FDMXEntityTreeNodeBase(FDMXEntityTreeNodeBase::ENodeType::CategoryNode)
	, CategoryType(InCategoryType)
	, CategoryName(InCategoryName)
	, ToolTip(InToolTip)
	, bCanDropOntoCategory(false)
	, IntValue(0)
{}

FDMXEntityTreeCategoryNode::FDMXEntityTreeCategoryNode(ECategoryType InCategoryType, FText InCategoryName, int32 Value, const FText& InToolTip /*= FText::GetEmpty()*/)
	: FDMXEntityTreeNodeBase(FDMXEntityTreeNodeBase::ENodeType::CategoryNode)
	, CategoryType(InCategoryType)
	, CategoryName(InCategoryName)
	, ToolTip(InToolTip)
	, bCanDropOntoCategory(true)
	, IntValue(Value)
{}

FDMXEntityTreeCategoryNode::FDMXEntityTreeCategoryNode(ECategoryType InCategoryType, FText InCategoryName, const FDMXFixtureCategory& Value, const FText& InToolTip /*= FText::GetEmpty()*/)
	: FDMXEntityTreeNodeBase(FDMXEntityTreeNodeBase::ENodeType::CategoryNode)
	, CategoryType(InCategoryType)
	, CategoryName(InCategoryName)
	, ToolTip(InToolTip)
	, bCanDropOntoCategory(true)
	, IntValue(0)
	, CategoryValue(Value)
{}

FString FDMXEntityTreeCategoryNode::GetDisplayNameString() const
{
	return CategoryName.ToString();
}

FText FDMXEntityTreeCategoryNode::GetDisplayNameText() const
{
	return CategoryName;
}

FDMXEntityTreeCategoryNode::ECategoryType FDMXEntityTreeCategoryNode::GetCategoryType() const
{
	return CategoryType;
}

#undef LOCTEXT_NAMESPACE
