// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DMXProtocolTypes.h"

#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FDMXEditor;
class UDMXEntity;


/** Abstract base class for nodes in a list of entities */
class FDMXEntityTreeNodeBase
	: public TSharedFromThis<FDMXEntityTreeNodeBase>
{
	/** Delegate for when the context menu requests a rename */
	DECLARE_DELEGATE(FOnRenameRequested);

public:
	enum class ENodeType : uint8
	{
		CategoryNode,
		EntityNode,
		RootNode
	};

	/**  Constructor */
	FDMXEntityTreeNodeBase(FDMXEntityTreeNodeBase::ENodeType InNodeType);
	
	/** Destructor */
	virtual ~FDMXEntityTreeNodeBase() {}

	/**  @return Whether or not this object represents a node that can be renamed from the entities tree. */
	virtual bool CanRename() const = 0;

	/**  @return The string to be used in the tree display. */
	virtual FString GetDisplayNameString() const = 0;
	
	/**  @return The name of this node in text. */
	virtual FText GetDisplayNameText() const = 0;
	
	/**  @return The type of this node. */
	virtual ENodeType GetNodeType() const;

	/** Add a child node to this node */
	virtual void AddChild(TSharedPtr<FDMXEntityTreeNodeBase> InChildPtr);
	
	/** Remove a child node from this node */
	virtual void RemoveChild(TSharedPtr<FDMXEntityTreeNodeBase> InChildPtr);
	
	/** Remove this node from its parent one */
	virtual void RemoveFromParent();
	
	/**  @return Child nodes for this object. */
	virtual const TArray<TSharedPtr<FDMXEntityTreeNodeBase>>& GetChildren() const;
	
	/**  Remove all child nodes from this node. */
	virtual void ClearChildren();
	
	/**  Sort children by name */
	virtual void SortChildren();
	
	/**  Sort children using custom predicate */
	virtual void SortChildren(TFunction<bool (const TSharedPtr<FDMXEntityTreeNodeBase>&, const TSharedPtr<FDMXEntityTreeNodeBase>&)> Predicate);
	
	/**  @return This object's parent node (or an invalid reference if no parent is assigned). */
	virtual TWeakPtr<FDMXEntityTreeNodeBase> GetParent() const { return ParentNodePtr; }

	/** Query that determines if this item should be filtered out or not */
	virtual bool IsFlaggedForFiltration() const;

	/** Refreshes this item's filtration state. Use bUpdateParent to make sure the parent's EFilteredState::ChildMatches flag is properly updated based off the new state */
	void UpdateCachedFilterState(bool bMatchesFilter, bool bUpdateParent);

	/** Update this node's desired expansion state for when there're no filters */
	void SetExpansionState(bool bNewExpansionState);

	/** This node's desired expansion state for when there're no filters */
	bool GetExpansionState() const { return bShouldBeExpanded; }

	/** If the warning tool tip is not empty, the node will display a warning icon with said tool tip. */
	void SetWarningStatus(const FText& InWarningToolTip);
	
	const FText& GetWarningStatus() const { return WarningToolTip; }
	
	/** If the error tool tip is not empty, the node will display an error icon with said tool tip. */
	void SetErrorStatus(const FText& InErrorToolTip);
	
	const FText& GetErrorStatus() const { return ErrorToolTip; }

	/** Operator used when sorting categories by name/number */
	bool operator<(const FDMXEntityTreeNodeBase& Other) const;

protected:
	/** Updates the EFilteredState::ChildMatches flag, based off of children's current state */
	void RefreshCachedChildFilterState(bool bUpdateParent);
	
	/** Used to update the EFilteredState::ChildMatches flag for parent nodes, when this item's filtration state has changed */
	void ApplyFilteredStateToParent();

	FText WarningToolTip;
	
	FText ErrorToolTip;

private:
	/** The type of the node */
	ENodeType NodeType;

	// Actual tree structure
	TWeakPtr<FDMXEntityTreeNodeBase> ParentNodePtr;
	
	/** The children of the node */
	TArray<TSharedPtr<FDMXEntityTreeNodeBase>> Children;

	/** Register whether the node should be expanded when there's no search filter text */
	bool bShouldBeExpanded = false;

	/** Filtered states the node can take */
	enum EFilteredState
	{
		FilteredOut = 0x00,
		MatchesFilter = (1 << 0),
		ChildMatches = (1 << 1),

		FilteredInMask = (MatchesFilter | ChildMatches),
		Unknown = 0xFC // ~FilteredInMask
	};
	
	uint8 FilterFlags = static_cast<uint8>(EFilteredState::Unknown);
};


/** A root node in an an entity list */
class FDMXEntityTreeRootNode
	: public FDMXEntityTreeNodeBase
{
public:
	FDMXEntityTreeRootNode();
	 
	// ~ start FDMXEntityTreeNodeBase interface
	virtual bool CanRename() const override { return false; }
	virtual FString GetDisplayNameString() const override;
	virtual FText GetDisplayNameText() const override;
	// ~ end FDMXEntityTreeNodeBase interface
};


/** An entity node in an an entity list */
class FDMXEntityTreeEntityNode
	: public FDMXEntityTreeNodeBase
{
public:
	FDMXEntityTreeEntityNode(UDMXEntity* InEntity);

	// ~ start FDMXEntityTreeNodeBase interface
	virtual bool CanRename() const override { return true; }
	virtual FString GetDisplayNameString() const override;
	virtual FText GetDisplayNameText() const override;
	// ~ end FDMXEntityTreeNodeBase interface

	/** Returns the entity of this node */
	virtual UDMXEntity* GetEntity() const;

private:
	/** The DMXEntity of the node */
	TWeakObjectPtr<UDMXEntity> DMXEntity;
};


/** A category node in an an entity list */
class FDMXEntityTreeCategoryNode
	: public FDMXEntityTreeNodeBase
{
public:
	enum class ECategoryType : uint8
	{
		DMXCategory,
		FixtureAssignmentState,
		UniverseID,
		NONE
	};

	FDMXEntityTreeCategoryNode(ECategoryType InCategoryType, FText InCategoryName, const FText& ToolTip = FText::GetEmpty());
	FDMXEntityTreeCategoryNode(ECategoryType InCategoryType, FText InCategoryName, int32 Value, const FText& ToolTip = FText::GetEmpty());
	FDMXEntityTreeCategoryNode(ECategoryType InCategoryType, FText InCategoryName, const FDMXFixtureCategory& Value, const FText& ToolTip = FText::GetEmpty());

	// ~ start FDMXEntityTreeNodeBase interface
	virtual bool CanRename() const override { return false; }
	virtual FString GetDisplayNameString() const override;
	virtual FText GetDisplayNameText() const override;
	ECategoryType GetCategoryType() const;
	// ~ end FDMXEntityTreeNodeBase interface

	const FText& GetToolTip() const { return ToolTip; }

	bool CanDropOntoCategory() const { return bCanDropOntoCategory; }
	int32 GetIntValue() const { return IntValue; }
	const FDMXFixtureCategory& GetCategoryValue() { return CategoryValue; }

private:
	/** This node's category type */
	ECategoryType CategoryType;
	FText CategoryName;
	FText ToolTip;

	bool bCanDropOntoCategory;

	/** The int value, if of UniverseID type */
	int32 IntValue = 0;

	/** The fixture category, if of FixtureCategory type */
	FDMXFixtureCategory CategoryValue;
};
