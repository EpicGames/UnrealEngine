// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "DMXAddFixturePatchMenuData.generated.h"


class UDMXEntityFixtureType;


UCLASS(Config = DMXEditor)
class UDMXAddFixturePatchMenuData
	: public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(Config)
	TSoftObjectPtr<UDMXEntityFixtureType> SoftFixtureType;

	UPROPERTY(Config)
	int32 ActiveModeIndex = 0;

	UPROPERTY(Config)
	int32 NumPatches = 0;

	UPROPERTY(Config)
	bool bIncrementChannelAfterPatching = true;
};
