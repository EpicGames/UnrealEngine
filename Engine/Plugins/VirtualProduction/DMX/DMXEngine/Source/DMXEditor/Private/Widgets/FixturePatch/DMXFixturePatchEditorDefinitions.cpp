// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXFixturePatchEditorDefinitions.h"


const int32 FDMXChannelGridSpecs::NumColumns = 32;

const int32 FDMXChannelGridSpecs::ChannelIDOffset = 1;
