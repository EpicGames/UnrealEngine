// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


/** Specifications of channels in a universe displayed in UI */
struct FDMXChannelGridSpecs
{
	/** Num Columns in the grid */
	static const int32 NumColumns;

	/** The offset of the Channel ID, from 0 to the first channel ID */
	static const int32 ChannelIDOffset;
};
