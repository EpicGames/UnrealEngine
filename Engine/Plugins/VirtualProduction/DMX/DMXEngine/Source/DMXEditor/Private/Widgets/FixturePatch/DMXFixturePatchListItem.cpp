// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXFixturePatchListItem.h"

#include "Algo/MinElement.h"
#include "DMXEditor.h"
#include "DMXFixturePatchSharedData.h"
#include "DMXRuntimeUtils.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXEntityReference.h"
#include "Library/DMXLibrary.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "DMXFixturePatchListItem"

FDMXFixturePatchListItem::FDMXFixturePatchListItem(TWeakPtr<FDMXEditor> InDMXEditor, UDMXEntityFixturePatch* InFixturePatch)
	: WeakFixturePatch(InFixturePatch)
	, WeakDMXEditor(InDMXEditor)
{}

FGuid FDMXFixturePatchListItem::GetMVRUUID() const
{
	return WeakFixturePatch.IsValid() ? WeakFixturePatch->GetMVRFixtureUUID() : FGuid();
}

FLinearColor FDMXFixturePatchListItem::GetBackgroundColor() const
{
	if (!ErrorStatusText.IsEmpty())
	{
		return FLinearColor::Red;
	}

	if (UDMXEntityFixturePatch* FixturePatch = WeakFixturePatch.Get())
	{
		return FixturePatch->EditorColor;
	}

	return FLinearColor::Red;
}

FString FDMXFixturePatchListItem::GetFixturePatchName() const
{
	if (const UDMXEntityFixturePatch* FixturePatch = GetFixturePatch())
	{
		return FixturePatch->Name;
	}

	return FString();
}

void FDMXFixturePatchListItem::SetFixturePatchName(const FString& InDesiredName, FString& OutNewName)
{
	const TGuardValue<bool> ChangeFixturePatchGuard(bChangingFixturePatch, true);

	if (UDMXEntityFixturePatch* FixturePatch = WeakFixturePatch.Get())
	{
		if (FixturePatch->Name == InDesiredName)
		{
			OutNewName = InDesiredName;
			return;
		}

		const FScopedTransaction SetFixturePatchNameTransaction(LOCTEXT("SetFixturePatchNameTransaction", "Set Fixture Patch Name"));
		FixturePatch->PreEditChange(UDMXEntityFixturePatch::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UDMXEntity, Name)));

		// Fixture Patches require to have a unique name to work with sequencer.
		UDMXLibrary* DMXLibrary = FixturePatch->GetParentLibrary();
		OutNewName = DMXLibrary ? FDMXRuntimeUtils::FindUniqueEntityName(DMXLibrary, UDMXEntityFixturePatch::StaticClass(), InDesiredName) : TEXT("Invalid Fixture Patch");

		FixturePatch->SetName(OutNewName);

		FixturePatch->PostEditChange();
	}
}

FString FDMXFixturePatchListItem::GetFixtureID() const
{
	return WeakFixturePatch.IsValid() ? FString::FromInt(WeakFixturePatch->GetFixtureID()) : TEXT("Invalid");
}

void FDMXFixturePatchListItem::SetFixtureID(int32 InFixtureID)
{
	const TGuardValue<bool> ChangeFixturePatchGuard(bChangingFixturePatch, true);

	if (UDMXEntityFixturePatch* FixturePatch = WeakFixturePatch.Get())
	{
		const FScopedTransaction SetFixturePatchNameTransaction(LOCTEXT("SetFixturePatchFixtureIDTransaction", "Set Fixture ID"));
		FixturePatch->PreEditChange(UDMXEntityFixturePatch::StaticClass()->FindPropertyByName(UDMXEntityFixturePatch::GetFixtureIDPropertyNameChecked()));

		FixturePatch->GenerateFixtureID(InFixtureID);

		FixturePatch->PostEditChange();
	}
}

UDMXEntityFixtureType* FDMXFixturePatchListItem::GetFixtureType() const
{
	if (UDMXEntityFixturePatch* FixturePatch = WeakFixturePatch.Get())
	{
		return FixturePatch->GetFixtureType();
	}

	return nullptr;
}

void FDMXFixturePatchListItem::SetFixtureType(UDMXEntityFixtureType* FixtureType)
{
	const TGuardValue<bool> ChangeFixturePatchGuard(bChangingFixturePatch, true);

	UDMXLibrary* DMXLibrary = GetDMXLibrary();
	UDMXEntityFixturePatch* FixturePatch = WeakFixturePatch.Get();

	if (DMXLibrary && FixturePatch)
	{
		if (FixturePatch->GetFixtureType() == FixtureType)
		{
			return;
		}

		const FScopedTransaction SetFixtureTypeTransaction(LOCTEXT("SetFixtureTypeTransaction", "Set Fixture Type of Patch"));
		FixturePatch->PreEditChange(UDMXEntityFixturePatch::StaticClass()->FindPropertyByName(UDMXEntityFixturePatch::GetParentFixtureTypeTemplatePropertyNameChecked()));

		FixturePatch->SetFixtureType(FixtureType);

		FixturePatch->PostEditChange();
	}
}

int32 FDMXFixturePatchListItem::GetModeIndex() const
{
	if (UDMXEntityFixturePatch* FixturePatch = WeakFixturePatch.Get())
	{
		return FixturePatch->GetActiveModeIndex();
	}

	return INDEX_NONE;
}

void FDMXFixturePatchListItem::SetModeIndex(int32 ModeIndex)
{
	// Don't handle other's transactions
	if (GIsTransacting)
	{
		return;
	}

	const TGuardValue<bool> ChangeFixturePatchGuard(bChangingFixturePatch, true);

	UDMXLibrary* DMXLibrary = GetDMXLibrary();
	UDMXEntityFixturePatch* FixturePatch = WeakFixturePatch.Get();

	if (DMXLibrary && FixturePatch)
	{
		if (ModeIndex == FixturePatch->GetActiveModeIndex())
		{
			return;
		}
		
		// If all should be changed, just change the patch
		const FScopedTransaction SetModeTransaction(LOCTEXT("SetModeTransaction", "Set Mode of Patch"));
		FixturePatch->PreEditChange(UDMXEntityFixturePatch::StaticClass()->FindPropertyByName(UDMXEntityFixturePatch::GetParentFixtureTypeTemplatePropertyNameChecked()));

		FixturePatch->SetActiveModeIndex(ModeIndex);
			
		FixturePatch->PostEditChange();
	}
}

bool FDMXFixturePatchListItem::GetActiveModeName(FString& OutModeName) const
{
	const UDMXEntityFixturePatch* FixturePatch = WeakFixturePatch.Get();
	const FDMXFixtureMode* ActiveModePtr = FixturePatch ? FixturePatch->GetActiveMode() : nullptr;

	if (ActiveModePtr)
	{
		OutModeName = ActiveModePtr->ModeName;
		return true;
	}

	return false;
}

int32 FDMXFixturePatchListItem::GetUniverse() const
{
	if (UDMXEntityFixturePatch* FixturePatch = WeakFixturePatch.Get())
	{
		return FixturePatch->GetUniverseID();
	}

	return -1;
}

int32 FDMXFixturePatchListItem::GetAddress() const
{
	if (UDMXEntityFixturePatch* FixturePatch = WeakFixturePatch.Get())
	{
		return FixturePatch->GetStartingChannel();
	}

	return -1;
}

void FDMXFixturePatchListItem::SetAddresses(int32 Universe, int32 Address)
{
	const TGuardValue<bool> ChangeFixturePatchGuard(bChangingFixturePatch, true);

	UDMXLibrary* DMXLibrary = GetDMXLibrary();
	UDMXEntityFixturePatch* FixturePatch = WeakFixturePatch.Get();
	const TSharedPtr<FDMXEditor> DMXEditor = WeakDMXEditor.Pin();
	const TSharedPtr<FDMXFixturePatchSharedData> SharedData = DMXEditor.IsValid() ? DMXEditor->GetFixturePatchSharedData() : nullptr;

	if (DMXLibrary && FixturePatch && SharedData.IsValid())
	{
		if (FixturePatch->GetUniverseID() == Universe &&
			FixturePatch->GetStartingChannel() == Address)
		{
			return;
		}

		// Only valid values
		const FDMXFixtureMode* ModePtr = FixturePatch->GetActiveMode();
		const int32 MaxAddress = ModePtr ? DMX_MAX_ADDRESS - ModePtr->ChannelSpan + 1 : DMX_MAX_ADDRESS;
		if (Universe < 0 || 
			Universe > DMX_MAX_UNIVERSE || 
			Address < 1 || 
			Address > MaxAddress)
		{
			return;
		}
		
		const FScopedTransaction SetAddressesTransaction(LOCTEXT("SetAddressesTransaction", "Set Addresses of Patch"));
		FixturePatch->PreEditChange(UDMXEntityFixturePatch::StaticClass()->FindPropertyByName(UDMXEntityFixturePatch::GetParentFixtureTypeTemplatePropertyNameChecked()));

		FixturePatch->SetUniverseID(Universe);
		FixturePatch->SetStartingChannel(Address);

		FixturePatch->PostEditChange();

		// Select the universe in Fixture Patch Shared Data
		SharedData->SelectUniverse(Universe);
	}
}

int32 FDMXFixturePatchListItem::GetNumChannels() const
{
	if (UDMXEntityFixturePatch* FixturePatch = WeakFixturePatch.Get())
	{
		return FixturePatch->GetChannelSpan();
	}

	return -1;
}

UDMXEntityFixturePatch* FDMXFixturePatchListItem::GetFixturePatch() const
{
	return WeakFixturePatch.Get();
}

UDMXLibrary* FDMXFixturePatchListItem::GetDMXLibrary() const
{
	if (const TSharedPtr<FDMXEditor> DMXEditor = WeakDMXEditor.Pin())
	{
		UDMXLibrary* DMXLibrary = DMXEditor->GetDMXLibrary();
		if (DMXLibrary)
		{
			return DMXLibrary;
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
