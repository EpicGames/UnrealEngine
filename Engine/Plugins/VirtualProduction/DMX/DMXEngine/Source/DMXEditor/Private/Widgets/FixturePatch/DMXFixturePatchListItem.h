// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVR/DMXMVRGeneralSceneDescription.h"
#include "UObject/WeakObjectPtr.h"

class FDMXEditor;
class FDMXFixturePatchSharedData;
class UDMXEntityFixturePatch;
class UDMXEntityFixtureType;
class UDMXLibrary;

/** An MVR Fixture as an Item in a List. A Primary Items is the first MVR Fixture in a Patch. Secondary Items are subsequent MVR Fixtures */
class FDMXFixturePatchListItem
	: public TSharedFromThis<FDMXFixturePatchListItem>
{
public:	
	/** Constructor */
	FDMXFixturePatchListItem(TWeakPtr<FDMXEditor> InDMXEditor, UDMXEntityFixturePatch* InFixturePatch);

	/** Returns the MVR UUID of the MVR Fixture */
	FGuid GetMVRUUID() const;

	/** Returns the background color of this Item */
	FLinearColor GetBackgroundColor() const;

	/** Returns the Name of the Fixture Patch */
	FString GetFixturePatchName() const;

	/** Sets the Name of the Fixture Patch */
	void SetFixturePatchName(const FString& InDesiredName, FString& OutNewName);

	/** Returns the Fixture ID */
	FString GetFixtureID() const;

	/** Sets the Fixture ID. Note, as other hard- and software, we allow to set integer values only. */
	void SetFixtureID(int32 InFixtureID);

	/** Returns the Fixture Type of the MVR Fixture */
	UDMXEntityFixtureType* GetFixtureType() const;
	
	/** Sets the fixture type of the MVR Fixture */
	void SetFixtureType(UDMXEntityFixtureType* FixtureType);

	/** Returns the Mode Index of the MVR Fixture */
	int32 GetModeIndex() const;

	/** Sets the Mode Index for the MVR */
	void SetModeIndex(int32 ModeIndex);

	/** Gets the name of the active mode. Returns false if there is no active mode for this patch. */
	bool GetActiveModeName(FString& OutModeName) const;

	/** Returns the Universe the MVR Fixture resides in */
	int32 GetUniverse() const;

	/** Returns the Address of the MVR Fixture */
	int32 GetAddress() const;

	/** Sets the Addresses of the MVR Fixture */
	void SetAddresses(int32 Universe, int32 Address);

	/** Returns Num Channels the MVR Fixture spans */
	int32 GetNumChannels() const;

	/** Returns the Fixture Patch. Const on purpose, should be edited via this class's methods */
	UDMXEntityFixturePatch* GetFixturePatch() const;

	/** Returns the DMX Library in which the MVR Fixture resides */
	UDMXLibrary* GetDMXLibrary() const;

	/** Returns true if this item is changing its Fixture Patch */
	bool IsChangingFixturePatch() const { return bChangingFixturePatch; }

	/** Warning Status Text of the Item */
	FText WarningStatusText;

	/** Error Status Text of the Item */
	FText ErrorStatusText;

private:
	/** True if this is in an even group */
	bool bIsEvenGroup = false;

	/** True while this item is changing the DMX Library */
	bool bChangingFixturePatch = false;

	/** The fixture patch the MVR Fixture UUID is assigned to */
	TWeakObjectPtr<UDMXEntityFixturePatch> WeakFixturePatch;

	/** The DMX Editor that owns this Item */
	TWeakPtr<FDMXEditor> WeakDMXEditor;
};
