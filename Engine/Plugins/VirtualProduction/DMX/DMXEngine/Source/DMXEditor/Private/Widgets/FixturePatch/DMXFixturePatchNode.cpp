// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXFixturePatchNode.h"

#include "DMXEditor.h"
#include "DMXFixturePatchEditorDefinitions.h"
#include "DMXFixturePatchSharedData.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXLibrary.h"
#include "ScopedTransaction.h"
#include "SDMXFixturePatchFragment.h"
#include "SDMXPatchedUniverse.h"
#include "Widgets/Layout/SGridPanel.h"

#define LOCTEXT_NAMESPACE "DMXFixturePatchNode"

/** A fixture patch fragment in a grid. CreateFragments creates DMXFixturePatchFragment from a DMXFixturePatchNode. */
class FDMXFixturePatchFragment
	: public TSharedFromThis<FDMXFixturePatchFragment>
{
public:
	/** Creates fragments of a patch spread across a grid */
	static void CreateFragments(TWeakObjectPtr<UDMXEntityFixturePatch> InFixturePatch, TArray<TSharedPtr<FDMXFixturePatchFragment>>& OutFragments)
	{
		OutFragments.Reset();

		UDMXEntityFixturePatch* FixturePatch = InFixturePatch.Get();
		if (!FixturePatch)
		{
			return;
		}

		UDMXLibrary* DMXLibrary = FixturePatch->GetParentLibrary();
		if (!DMXLibrary)
		{
			return;
		}

		TMap<TRange<int32>, int32> RangeToNumConflictsMap;
		BuildRangeToNumConflictsMap(*FixturePatch, RangeToNumConflictsMap);
		if (RangeToNumConflictsMap.IsEmpty())
		{
			return;
		}

		// Create a node for each range, at least one node per range
		TSharedPtr<FDMXFixturePatchFragment> PrevFragment;
		for (const TTuple<TRange<int32>, int32>& RangeToNumConflictsPair : RangeToNumConflictsMap)
		{
			const TRange<int32>& Range = RangeToNumConflictsPair.Key;
			const int32 NumConflicts = RangeToNumConflictsPair.Value;
			int32 ChannelIndex = Range.GetLowerBoundValue();
			int32 ChannelSpan = Range.GetUpperBoundValue() - Range.GetLowerBoundValue();

			do
			{
				const int32 Row = ChannelIndex / FDMXChannelGridSpecs::NumColumns;
				const int32 Column = ChannelIndex % FDMXChannelGridSpecs::NumColumns;
				int32 ColumnSpan = Column + ChannelSpan < FDMXChannelGridSpecs::NumColumns ? ChannelSpan : FDMXChannelGridSpecs::NumColumns - Column;
				if (ColumnSpan == 0)
				{
					// No empty fragments
					break;
				}

				const TSharedPtr<FDMXFixturePatchFragment> NewFragment = MakeShared<FDMXFixturePatchFragment>();
				NewFragment->FixturePatch = InFixturePatch;
				NewFragment->ChannelIndex = ChannelIndex;
				NewFragment->Column = Column;
				NewFragment->Row = Row;
				NewFragment->ColumnSpan = ColumnSpan;
				NewFragment->bIsConflict = NumConflicts > 0;

				OutFragments.Add(NewFragment);

				// Link lhs
				NewFragment->LhsFragment = PrevFragment;

				// Link rhs
				if (PrevFragment.IsValid())
				{
					PrevFragment->RhsFragment = NewFragment;
				}

				// Update for next node
				ChannelSpan -= ColumnSpan;
				ChannelIndex += ColumnSpan;
				PrevFragment = NewFragment;
			} while (ChannelSpan > 0);
		}
	}

	bool IsHead() const { return !LhsFragment.IsValid(); }
	bool IsTail() const { return !RhsFragment.IsValid(); }

	/** Channel Index of the fragment */
	int32 ChannelIndex = INDEX_NONE;

	/** Row of the fragment */
	int32 Row = -1;

	/** Column of the fragment */
	int32 Column = -1;

	/** Columnspan of the fragment */
	int32 ColumnSpan = -1;

	/** True if this fragment conflicts with others */
	bool bIsConflict = false;

	/** The fragment to the left of this fragment */
	TSharedPtr<FDMXFixturePatchFragment> LhsFragment;

	/** The fragment to the right of this fragment */
	TSharedPtr<FDMXFixturePatchFragment> RhsFragment;

	/** The fixture patch of which the fragment is part of */
	TWeakObjectPtr<UDMXEntityFixturePatch> FixturePatch;

private:
	/** Breaks up the patch in an array of ranges along with the number of conflicting patches */
	static void BuildRangeToNumConflictsMap(const UDMXEntityFixturePatch& InFixturePatch, TMap<TRange<int32>, int32>& OutRangeToNumConflictsMap)
	{
		TMap<int32, int32> IndexToNumConflictsMap;
		IndexToNumConflictsMap.Reserve(InFixturePatch.GetChannelSpan());

		const int32 StartingIndex = InFixturePatch.GetStartingChannel() - 1;
		const int32 EndingIndex = InFixturePatch.GetEndingChannel() - 1;
		const TArray<const UDMXEntityFixturePatch*> ConflictingFixturePatches = FindConflictingFixturePatches(InFixturePatch);
		for (const UDMXEntityFixturePatch* ConflictingFixturePatch : ConflictingFixturePatches)
		{
			const int32 ConflictStartingIndex = FMath::Max(StartingIndex, ConflictingFixturePatch->GetStartingChannel() - 1);
			const int32 ConflictEndingIndex = FMath::Min(EndingIndex, ConflictingFixturePatch->GetEndingChannel() - 1);
			for (int32 ConflictIndex = ConflictStartingIndex; ConflictIndex <= ConflictEndingIndex; ConflictIndex++)
			{
				IndexToNumConflictsMap.FindOrAdd(ConflictIndex, 0)++;
			}
		}
		for (int32 Index = StartingIndex; Index <= EndingIndex; Index++)
		{
			if (!IndexToNumConflictsMap.Contains(Index))
			{
				IndexToNumConflictsMap.Add(Index, 0);
			}
		}
		if (IndexToNumConflictsMap.IsEmpty())
		{
			return;
		}

		TRange<int32> Range(StartingIndex, StartingIndex);
		int32 NumConflicts = IndexToNumConflictsMap[StartingIndex];
		for (const TTuple<int32, int32>& IndexToNumConflictsPair : IndexToNumConflictsMap)
		{
			const int32 NextIndex = Range.GetUpperBoundValue() + 1;
			Range.SetUpperBoundValue(NextIndex);

			const int32 NextNumConflicts = IndexToNumConflictsMap.Contains(NextIndex) ? IndexToNumConflictsMap[NextIndex] : -1;
			if (NumConflicts != NextNumConflicts)
			{
				OutRangeToNumConflictsMap.Add(Range, NumConflicts);
				Range = TRange<int32>(Range.GetUpperBoundValue(), Range.GetUpperBoundValue());
				NumConflicts = NextNumConflicts;
			}
		}
	}

	/** Returns the fixture patches that conflict with the specified fixture patch */
	static TArray<const UDMXEntityFixturePatch*> FindConflictingFixturePatches(const UDMXEntityFixturePatch& InFixturePatch)
	{
		TArray<const UDMXEntityFixturePatch*> Result;

		if (InFixturePatch.GetChannelSpan() == 0)
		{
			return Result;
		}

		UDMXLibrary* DMXLibrary = InFixturePatch.GetParentLibrary();
		if (!DMXLibrary)
		{
			return Result;
		}

		// Get the other Fixture Patches in the DMX Library
		const UDMXEntityFixturePatch* FixturePatch = &InFixturePatch;
		const TArray<const UDMXEntityFixturePatch*> OtherFixturePatches = [FixturePatch, DMXLibrary]()
		{
			TArray<const UDMXEntityFixturePatch*> FixturePatches = DMXLibrary->GetEntitiesTypeCast<const UDMXEntityFixturePatch>();
			FixturePatches.Remove(FixturePatch);
			return FixturePatches;
		}();

		// Find the conflicts
		const int32 MyChannelIndex = FixturePatch->GetStartingChannel() - 1;
		const int32 MyChannelSpan = FixturePatch->GetChannelSpan();
		const TRange<int32> MyRange(MyChannelIndex, MyChannelIndex + MyChannelSpan);

		TArray<const UDMXEntityFixturePatch*> ConflictingFixturePatches = [FixturePatch, &OtherFixturePatches, &MyRange]()
		{
			TArray<const UDMXEntityFixturePatch*> Result;
			for (const UDMXEntityFixturePatch* Other : OtherFixturePatches)
			{
				if (!Other)
				{
					continue;
				}

				if (Other->GetUniverseID() != FixturePatch->GetUniverseID())
				{
					continue;
				}

				// Equal patches are not conflicting
				if (Other->GetFixtureType() == FixturePatch->GetFixtureType() &&
					Other->GetActiveModeIndex() == FixturePatch->GetActiveModeIndex() &&
					Other->GetStartingChannel() == FixturePatch->GetStartingChannel())
				{
					continue;
				}

				const int32 OtherChannelIndex = Other->GetStartingChannel() - 1;
				const int32 OtherChannelSpan = Other->GetChannelSpan();
				const TRange<int32> OtherRange(OtherChannelIndex, OtherChannelIndex + OtherChannelSpan);

				if (OtherRange.Overlaps(MyRange))
				{
					Result.Add(Other);
				}
			}
			return Result;
		}();

		return ConflictingFixturePatches;
	}
};

TSharedPtr<FDMXFixturePatchNode> FDMXFixturePatchNode::Create(TWeakPtr<FDMXEditor> InDMXEditor, const TWeakObjectPtr<UDMXEntityFixturePatch>& InFixturePatch)
{
	if (!ensureMsgf(InFixturePatch.IsValid(), TEXT("Trying to create Fixture Patch Node. But no Fixture Patch is not valid.")))
	{
		return nullptr;
	}

	const TSharedRef<FDMXFixturePatchNode> NewNode = MakeShared<FDMXFixturePatchNode>();
	NewNode->WeakDMXEditor = InDMXEditor;
	NewNode->FixturePatch= InFixturePatch;

	TSharedPtr<FDMXEditor> DMXEditor = NewNode->WeakDMXEditor.Pin();
	if (!DMXEditor.IsValid())
	{
		return nullptr;
	}

	TSharedPtr<FDMXFixturePatchSharedData> SharedData = DMXEditor->GetFixturePatchSharedData();
	if (SharedData.IsValid())
	{
		NewNode->UpdateIsSelected();
		SharedData->OnFixturePatchSelectionChanged.AddSP(NewNode, &FDMXFixturePatchNode::UpdateIsSelected);
	}

	return NewNode;
}

void FDMXFixturePatchNode::SetAddresses(int32 NewUniverseID, int32 NewStartingChannel, int32 NewChannelSpan)
{
	if (!FixturePatch.IsValid())
	{
		return;
	}

	FixturePatch->SetUniverseID(NewUniverseID);
	FixturePatch->SetStartingChannel(NewStartingChannel);
}

TArray<TSharedRef<SDMXFixturePatchFragment>> FDMXFixturePatchNode::GenerateWidgets(const TSharedRef<SDMXPatchedUniverse>& OwningUniverse, const TArray<TSharedPtr<FDMXFixturePatchNode>>& FixturePatchNodeGroup)
{
	// Fragment the patch to fit the grid
	TArray<TSharedPtr<FDMXFixturePatchFragment>> NewFragments;
	FDMXFixturePatchFragment::CreateFragments(FixturePatch, NewFragments);

	TArray<TSharedRef<SDMXFixturePatchFragment>> Result;
	for (const TSharedPtr<FDMXFixturePatchFragment>& Fragment : NewFragments)
	{
		Result.Add(
			SNew(SDMXFixturePatchFragment, SharedThis(this), FixturePatchNodeGroup)
			.bIsHead(Fragment->IsHead())
			.bIsTail(Fragment->IsTail())
			.bIsConflicting(Fragment->bIsConflict)
			.StartingChannel(Fragment->ChannelIndex + 1)
			.Column(Fragment->Column)
			.Row(Fragment->Row)
			.ColumnSpan(Fragment->ColumnSpan)
		);
	}

	return Result;
}

bool FDMXFixturePatchNode::OccupiesChannels(int32 Channel, int32 Span) const
{
	check(Span != 0);
	if ((Channel + Span <= GetStartingChannel()) || (Channel >= GetStartingChannel() + GetChannelSpan()))
	{
		return false;
	}
	return true;
}

bool FDMXFixturePatchNode::IsSelected() const
{
	return bSelected;
}

FString FDMXFixturePatchNode::GetFixtureID() const
{
	return FixturePatch.IsValid() ? FString::FromInt(FixturePatch->GetFixtureID()) : TEXT("Invalid");
}

int32 FDMXFixturePatchNode::GetUniverseID() const
{
	return FixturePatch.IsValid() ? FixturePatch->GetUniverseID() : -1;
}

int32 FDMXFixturePatchNode::GetStartingChannel() const
{
	return FixturePatch.IsValid() ? FixturePatch->GetStartingChannel() : -1;
}

int32 FDMXFixturePatchNode::GetChannelSpan() const
{
	return FixturePatch.IsValid() ? FixturePatch->GetChannelSpan() : -1;
}

void FDMXFixturePatchNode::UpdateIsSelected()
{
	TSharedPtr<FDMXEditor> DMXEditor = WeakDMXEditor.Pin();
	if (!DMXEditor.IsValid())
	{
		return;
	}

	const TSharedPtr<FDMXFixturePatchSharedData> SharedData = DMXEditor->GetFixturePatchSharedData();
	if (!SharedData.IsValid())
	{
		return;
	}

	bSelected = SharedData->GetSelectedFixturePatches().Contains(FixturePatch);
}

#undef LOCTEXT_NAMESPACE
