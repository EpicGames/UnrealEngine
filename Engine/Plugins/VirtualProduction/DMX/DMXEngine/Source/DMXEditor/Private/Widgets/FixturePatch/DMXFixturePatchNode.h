// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class FDMXEditor;
class SDMXFixturePatchFragment;
class SDMXPatchedUniverse;
class UDMXEntityFixturePatch;

/** 
 * A fixture patch in a grid, consists of several fragments, displayed as SDMXFixturePatchNodes (see GenerateWidgets). 
 * The patch node does not necessarily have to present the state of the object, e.g. when
 * dragged or when containing an occuppied universe/channel. 
 */
class FDMXFixturePatchNode
	: public TSharedFromThis<FDMXFixturePatchNode>
{
public:
	/** Creates a new node from Patch */
	static TSharedPtr<FDMXFixturePatchNode> Create(TWeakPtr<FDMXEditor> InDMXEditor, const TWeakObjectPtr<UDMXEntityFixturePatch>& InFixturePatch);

	/** Sets the Addresses of the Node */
	void SetAddresses(int32 UniverseID, int32 NewStartingChannel, int32 NewChannelSpan);

	/** Generates widgets. Requires Addresses to be set via SetAddresses */
	TArray<TSharedRef<SDMXFixturePatchFragment>> GenerateWidgets(const TSharedRef<SDMXPatchedUniverse>& OwningUniverse, const TArray<TSharedPtr<FDMXFixturePatchNode>>& FixturePatchNodeGroup);

	/** Returns wether the patch uses specified channels */
	bool OccupiesChannels(int32 Channel, int32 Span) const;

	/** Returns true if the node is selected */
	bool IsSelected() const;

	/** Sets if the patch is hovered */
	void SetIsHovered(bool bNewIsHovered) { bHovered = bNewIsHovered; }

	/** Returns true if this patch is hovered */
	bool IsHovered() const { return bHovered; }

	/** Returns the fixture patches this node holds */
	TWeakObjectPtr<UDMXEntityFixturePatch> GetFixturePatch() const { return FixturePatch; }

	/** Returns the Fixture ID of the patch */
	FString GetFixtureID() const;

	/** Returns the Universe ID the Node currently resides in. Returns a negative Value if not assigned to a Universe. */
	int32 GetUniverseID() const;

	/** Returns the Starting Channel of the Patch */
	int32 GetStartingChannel() const;

	/** Returns the Channel Span of the Patch */
	int32 GetChannelSpan() const;

	/** Sets the ZOrder of the Node */
	void SetZOrder(int32 NewZOrder) { ZOrder = NewZOrder; }

	/** Gets the ZOrder of the Node */
	int32 GetZOrder() const { return ZOrder; }

private:
	/** Updates if the node is selected */
	void UpdateIsSelected();

	/** If true the node is selected */
	bool bSelected = false;

	/** True if the node is hovered */
	bool bHovered = false;

	/** ZOrder of this node */
	int32 ZOrder = 0;

	/** The Fixture Patch this Node stands for */
	TWeakObjectPtr<UDMXEntityFixturePatch> FixturePatch;

	/** Weak DMXEditor refrence */
	TWeakPtr<FDMXEditor> WeakDMXEditor;
};
