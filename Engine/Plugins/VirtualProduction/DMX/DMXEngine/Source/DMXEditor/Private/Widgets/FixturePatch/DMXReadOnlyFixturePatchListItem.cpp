// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/DMXReadOnlyFixturePatchListItem.h"

#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXEntityReference.h"


#define LOCTEXT_NAMESPACE "DMXReadOnlyFixturePatchListItem"

FDMXReadOnlyFixturePatchListItem::FDMXReadOnlyFixturePatchListItem(const FDMXEntityFixturePatchRef& FixturePatchReference)
{
	FixturePatch = FixturePatchReference.GetFixturePatch();
}

UDMXLibrary* FDMXReadOnlyFixturePatchListItem::GetDMXLibrary() const
{
	return FixturePatch ? FixturePatch->GetParentLibrary() : nullptr;
}

FText FDMXReadOnlyFixturePatchListItem::GetNameText() const
{
	if (FixturePatch)
	{
		const FString FixturePatchName = FixturePatch->Name;
		return FText::FromString(FixturePatchName);
	}

	return FText::GetEmpty();
}

FText FDMXReadOnlyFixturePatchListItem::GetUniverseChannelText() const
{
	if (FixturePatch)
	{
		const int32 UniverseID = FixturePatch->GetUniverseID();
		const int32 StartingAddress = FixturePatch->GetStartingChannel();
		return FText::Format(LOCTEXT("AddressesText", "{0}.{1}"), UniverseID, StartingAddress);
	}

	return FText::GetEmpty();
}

FText FDMXReadOnlyFixturePatchListItem::GetFixtureIDText() const
{
	if (FixturePatch)
	{
		const FString FixtureIDAsString = FString::FromInt(FixturePatch->GetFixtureID());
		return FText::FromString(FixtureIDAsString);
	}

	return FText::GetEmpty();
}

FText FDMXReadOnlyFixturePatchListItem::GetFixtureTypeText() const
{
	if (FixturePatch && FixturePatch->GetFixtureType())
	{
		const FString FixtureTypeName = FixturePatch->GetFixtureType()->Name;
		return FText::FromString(FixtureTypeName);
	}

	return FText::GetEmpty();
}

FText FDMXReadOnlyFixturePatchListItem::GetModeText() const
{
	if (FixturePatch && FixturePatch->GetActiveMode())
	{
		const FString FixtureModeName = FixturePatch->GetActiveMode()->ModeName;
		return FText::FromString(FixtureModeName);
	}

	return FText::GetEmpty();
}

FSlateColor FDMXReadOnlyFixturePatchListItem::GetEditorColor() const
{
	return FixturePatch ? 
		FixturePatch->EditorColor : 
		FSlateColor::UseForeground();
}

void FDMXReadOnlyFixturePatchListItem::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(FixturePatch);
}

FString FDMXReadOnlyFixturePatchListItem::GetReferencerName() const
{
	return TEXT("FDMXReadOnlyFixturePatchListItem");
}

#undef LOCTEXT_NAMESPACE
