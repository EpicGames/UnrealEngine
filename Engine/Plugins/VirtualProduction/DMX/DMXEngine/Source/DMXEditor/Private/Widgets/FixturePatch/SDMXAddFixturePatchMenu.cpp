// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXAddFixturePatchMenu.h"

#include "Algo/Copy.h"
#include "Algo/Find.h"
#include "DMXAddFixturePatchMenuData.h"
#include "DMXEditor.h"
#include "DMXFixturePatchSharedData.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "FixturePatchAutoAssignUtility.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/DMXLibrary.h"
#include "Misc/ScopedSlowTask.h"
#include "ScopedTransaction.h"
#include "TimerManager.h"
#include "UObject/Package.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SDMXEntityDropdownMenu.h"


#define LOCTEXT_NAMESPACE "SDMXAddFixturePatchMenu"

namespace UE::DMXEditor::FixturePatchEditor
{
	SDMXAddFixturePatchMenu::~SDMXAddFixturePatchMenu()
	{
		UDMXAddFixturePatchMenuData* MenuData = GetMutableDefault<UDMXAddFixturePatchMenuData>();
		MenuData->SoftFixtureType = WeakFixtureType.Get();
	}

	void SDMXAddFixturePatchMenu::Construct(const FArguments& InArgs, TWeakPtr<FDMXEditor> InWeakDMXEditor)
	{
		WeakDMXEditor = InWeakDMXEditor;

		SharedData = WeakDMXEditor.IsValid() ? WeakDMXEditor.Pin()->GetFixturePatchSharedData() : nullptr;
		if (SharedData.IsValid())
		{
			UDMXLibrary::GetOnEntitiesAdded().AddSP(this, &SDMXAddFixturePatchMenu::OnEntityAddedOrRemoved);
			UDMXLibrary::GetOnEntitiesRemoved().AddSP(this, &SDMXAddFixturePatchMenu::OnEntityAddedOrRemoved);

			Refresh();
		}
	}

	void SDMXAddFixturePatchMenu::RequestRefresh()
	{
		RequestRefreshModeComboBoxTimerHandle.Invalidate();

		if (!RequestRefreshModeComboBoxTimerHandle.IsValid())
		{
			RequestRefreshModeComboBoxTimerHandle = GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateSP(this, &SDMXAddFixturePatchMenu::Refresh));
		}
	}

	void SDMXAddFixturePatchMenu::Refresh()
	{
		RequestRefreshModeComboBoxTimerHandle.Invalidate();

		UDMXLibrary* DMXLibrary = WeakDMXEditor.IsValid() ? WeakDMXEditor.Pin()->GetDMXLibrary() : nullptr;
		if (!DMXLibrary)
		{
			return;
		}

		// Mend the fixture type
		if (!WeakFixtureType.IsValid())
		{
			const TArray<UDMXEntityFixtureType*> FixtureTypes = DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixtureType>();
			WeakFixtureType = FixtureTypes.IsEmpty() ? nullptr : FixtureTypes[0];
		}


		// Mend the active mode index
		UDMXAddFixturePatchMenuData* MenuData = GetMutableDefault<UDMXAddFixturePatchMenuData>();
		if (WeakFixtureType.IsValid() && !WeakFixtureType->Modes.IsValidIndex(MenuData->ActiveModeIndex))
		{
			MenuData->ActiveModeIndex = 0;
			MenuData->SaveConfig();
		}


		// Create the combo box source and an intial selection
		ModeSources.Reset();
		if (UDMXEntityFixtureType* FixtureType = WeakFixtureType.Get())
		{
			for (int32 ModeIndex = 0; ModeIndex < FixtureType->Modes.Num(); ModeIndex++)
			{
				ModeSources.Add(MakeShared<uint32>(ModeIndex));
			}
		}

		const TArray<UDMXEntityFixtureType*> FixtureTypes = DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixtureType>();
		UDMXEntityFixtureType* const* SelectedFixtureTypePtr = Algo::FindByPredicate(FixtureTypes, [MenuData](const UDMXEntityFixtureType* FixtureType)
			{
				return FixtureType == MenuData->SoftFixtureType;
			});
		if (!SelectedFixtureTypePtr)
		{
			SelectedFixtureTypePtr = FixtureTypes.IsEmpty() ? nullptr : &FixtureTypes[0];
		}
		WeakFixtureType = SelectedFixtureTypePtr ? *SelectedFixtureTypePtr : nullptr;


		// Rebuild the widget
		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SGridPanel)
				.FillColumn(1, 1.f)

				// Select Fixture Type
				+ SGridPanel::Slot(0, 0)
				.Padding(4.f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SelectFixtureTypeLabel", "Fixture Type"))
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				]

				+ SGridPanel::Slot(1, 0)
				.Padding(4.f)
				.VAlign(VAlign_Center)
				[
					MakeFixtureTypeSelectWidget()
				]

				// Select Mode
				+ SGridPanel::Slot(0, 1)
				.Padding(4.f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.IsEnabled(this, &SDMXAddFixturePatchMenu::HasValidFixtureTypeAndMode)
					.Text(LOCTEXT("SelectModeLabel", "Mode"))
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				]
				+ SGridPanel::Slot(1, 1)
				.Padding(4.f)
				[
					MakeModeSelectWidget()
				]

				// Universe label 
				+ SGridPanel::Slot(0, 2)
				.Padding(4.f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.IsEnabled(this, &SDMXAddFixturePatchMenu::HasValidFixtureTypeAndMode)
					.Text(LOCTEXT("UniverseDotChannelLabel", "Universe.Channel"))
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				]

				// Select universe and channel 
				+ SGridPanel::Slot(1, 2)
				.Padding(4.f)
				.VAlign(VAlign_Center)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						MakeUniverseChannelSelectWidget()
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(4.f, 0.f, 0.f, 0.f)
					[
						MakeAutoIncrementChannelCheckBox()
					]
				]

				// Num Fixture Patches Label
				+ SGridPanel::Slot(0, 3)
				.Padding(4.f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.IsEnabled(this, &SDMXAddFixturePatchMenu::HasValidFixtureTypeAndMode)
					.Text(LOCTEXT("NumPatchesLabel", "Num Patches"))
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				]

				// Num Fixture Patches editable Text Box
				+ SGridPanel::Slot(1, 3)
				.Padding(4.f)
				[
					MakeNumFixturePatchesEditableTextBox()
				]

				// Add fixture patches button
				+ SGridPanel::Slot(1, 4)
				.Padding(4.f)
				[
					MakeAddFixturePatchesButton()
				]
			]
		];
	}

	TSharedRef<SWidget> SDMXAddFixturePatchMenu::MakeFixtureTypeSelectWidget()
	{
		return
			SNew(SBox)
			.MinDesiredWidth(100.f)
			[
				SAssignNew(FixtureTypeSelector, SDMXEntityPickerButton<UDMXEntityFixtureType>)
				.DMXEditor(WeakDMXEditor)
				.CurrentEntity_Lambda([this] { return WeakFixtureType.Get(); })
				.OnEntitySelected(this, &SDMXAddFixturePatchMenu::OnFixtureTypeSelected)
			];
	}

	TSharedRef<SWidget> SDMXAddFixturePatchMenu::MakeModeSelectWidget()
	{
		return
			SAssignNew(ModeComboBox, SComboBox<TSharedPtr<uint32>>)
			.IsEnabled(this, &SDMXAddFixturePatchMenu::HasValidFixtureTypeAndMode)
			.OptionsSource(&ModeSources)
			.OnGenerateWidget(this, &SDMXAddFixturePatchMenu::GenerateModeComboBoxEntry)
			.OnSelectionChanged(this, &SDMXAddFixturePatchMenu::OnModeSelected)
			.InitiallySelectedItem(0)
			[
				SNew(STextBlock)
				.MinDesiredWidth(50.0f)
				.Text(this, &SDMXAddFixturePatchMenu::GetActiveModeText)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			];
	}

	TSharedRef<SWidget> SDMXAddFixturePatchMenu::MakeUniverseChannelSelectWidget()
	{
		return
			SNew(SBox)
			.HAlign(HAlign_Left)
			[
				SAssignNew(UniverseChannelEditableTextBox, SEditableTextBox)
				.IsEnabled(this, &SDMXAddFixturePatchMenu::HasValidFixtureTypeAndMode)
				.MinDesiredWidth(60.f)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.SelectAllTextWhenFocused(true)
				.ClearKeyboardFocusOnCommit(true)
				.RevertTextOnEscape(true)
				.Text(this, &SDMXAddFixturePatchMenu::GetUniverseChannelText)
				.OnTextChanged(this, &SDMXAddFixturePatchMenu::OnUniverseChannelTextChanged)
				.OnTextCommitted(this, &SDMXAddFixturePatchMenu::OnUniverseChannelTextCommitted)
			];
	}

	TSharedRef<SWidget> SDMXAddFixturePatchMenu::MakeAutoIncrementChannelCheckBox()
	{
		return 
			SNew(SCheckBox)
			.IsEnabled(this, &SDMXAddFixturePatchMenu::HasValidFixtureTypeAndMode)
			.IsChecked_Lambda([]()
				{
					const UDMXAddFixturePatchMenuData* MenuData = GetDefault<UDMXAddFixturePatchMenuData>();
					return MenuData->bIncrementChannelAfterPatching ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
			.OnCheckStateChanged_Lambda([](ECheckBoxState NewState)
				{
					UDMXAddFixturePatchMenuData* MenuData = GetMutableDefault<UDMXAddFixturePatchMenuData>();
					MenuData->bIncrementChannelAfterPatching = NewState == ECheckBoxState::Checked;
					MenuData->SaveConfig();
				})
			[
				SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.Text(LOCTEXT("AutoIncrementChannelLabel", "Increment after patching"))
				.ToolTipText(LOCTEXT("AutoIncrementChannelTooltip", "Automatically increments the universe/channel to the first subsequent channel after patching."))
			];
	}

	TSharedRef<SWidget> SDMXAddFixturePatchMenu::MakeNumFixturePatchesEditableTextBox()
	{
		return
			SNew(SBox)
			.HAlign(HAlign_Left)
			[
				SNew(SEditableTextBox)
				.IsEnabled(this, &SDMXAddFixturePatchMenu::HasValidFixtureTypeAndMode)
				.MinDesiredWidth(60.f)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.SelectAllTextWhenFocused(true)
				.ClearKeyboardFocusOnCommit(true)
				.Text_Lambda([this]()
					{
						return FText::FromString(FString::FromInt(NumFixturePatchesToAdd));
					})
				.OnTextCommitted_Lambda([this](const FText& Text, ETextCommit::Type CommitType)
					{
						int32 Value;
						if (LexTryParseString<int32>(Value, *Text.ToString()) &&
							Value > 0)
						{
							constexpr int32 MaxNumFixturePatchesToAdd = 16384;
							NumFixturePatchesToAdd = FMath::Clamp(Value, 1, MaxNumFixturePatchesToAdd);
						}

						// Add fixture patches if enter is pressed
						if (CommitType == ETextCommit::OnEnter)
						{
							OnAddFixturePatchButtonClicked();
						}
					})
			];
	}

	TSharedRef<SWidget> SDMXAddFixturePatchMenu::MakeAddFixturePatchesButton()
	{
		return
			SNew(SBox)
			.HAlign(HAlign_Right)
			.MinDesiredWidth(120.f)
			[
				SNew(SButton)
				.IsEnabled(this, &SDMXAddFixturePatchMenu::HasValidFixtureTypeAndMode)
				.ContentPadding(FMargin(4.0f, 4.0f))
				.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
				.ForegroundColor(FLinearColor::White)
				.OnClicked(this, &SDMXAddFixturePatchMenu::OnAddFixturePatchButtonClicked)
				[
					SNew(STextBlock)
					.Text_Lambda([this]()
						{
							return FText::Format(LOCTEXT("AddFixturePatchButtonText", "Add Fixture {0}|plural(one=Patch, other=Patches)"), NumFixturePatchesToAdd);
						})
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.BoldFont")))
				]
			];
	}

	TSharedRef<SWidget> SDMXAddFixturePatchMenu::GenerateModeComboBoxEntry(const TSharedPtr<uint32> InModeIndex) const
	{
		UDMXEntityFixtureType* FixtureType = Cast<UDMXEntityFixtureType>(WeakFixtureType);
		if (!FixtureType)
		{
			return SNullWidget::NullWidget;
		}

		const TArray<FDMXFixtureMode>& Modes = FixtureType->Modes;

		return
			SNew(STextBlock)
			.MinDesiredWidth(50.0f)
			.Text_Lambda([&Modes, ModeIndex = *InModeIndex, this]()
				{
					const UDMXAddFixturePatchMenuData* MenuData = GetDefault<UDMXAddFixturePatchMenuData>();
					if (Modes.IsValidIndex(ModeIndex))
					{
						return FText::FromString(Modes[ModeIndex].ModeName);
					}
					else
					{
						return LOCTEXT("NoModeAvailableText", "No Mode available");
					}
				})
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));
	}

	void SDMXAddFixturePatchMenu::OnEntityAddedOrRemoved(UDMXLibrary* DMXLibrary, TArray<UDMXEntity*> Entities)
	{
		RequestRefresh();
	}

	void SDMXAddFixturePatchMenu::OnFixtureTypeSelected(UDMXEntity* InSelectedFixtureType)
	{
		UDMXEntityFixtureType* SelectedFixtureType = Cast<UDMXEntityFixtureType>(InSelectedFixtureType);
		WeakFixtureType = SelectedFixtureType;

		UDMXAddFixturePatchMenuData* MenuData = GetMutableDefault<UDMXAddFixturePatchMenuData>();
		MenuData->SoftFixtureType = SelectedFixtureType;
		MenuData->SaveConfig();

		RequestRefresh();
	}

	void SDMXAddFixturePatchMenu::OnModeSelected(TSharedPtr<uint32> InSelectedMode, ESelectInfo::Type SelectInfo)
	{
		UDMXAddFixturePatchMenuData* MenuData = GetMutableDefault<UDMXAddFixturePatchMenuData>();
		MenuData->ActiveModeIndex = InSelectedMode.IsValid() ? *InSelectedMode : 0;
		MenuData->SaveConfig();
	}

	FText SDMXAddFixturePatchMenu::GetUniverseChannelText() const
	{
		if (!Universe.IsSet() && !Channel.IsSet())
		{
			const int32 SelectedUniverse = SharedData->GetSelectedUniverse();
			return FText::FromString(FString::FromInt(SelectedUniverse) + TEXT(".") + FString::FromInt(1));
		}
		else
		{
			const FString UnivereString = Universe.IsSet() ? FString::FromInt(Universe.GetValue()) : TEXT("");
			const FString ChannelString = Channel.IsSet() ? FString::FromInt(Channel.GetValue()) : TEXT("");
			const FString Separator = Universe.IsSet() && Channel.IsSet() ? TEXT(".") : TEXT("");

			return FText::FromString(UnivereString + Separator + ChannelString);
		}
	}

	void SDMXAddFixturePatchMenu::OnUniverseChannelTextChanged(const FText& Text)
	{		
		if (!UniverseChannelEditableTextBox.IsValid())
		{
			return;
		}

		static const TCHAR* Delimiter[] = { TEXT("."), TEXT(","), TEXT(":"), TEXT(" ") };
		TArray<FString> Substrings;
		constexpr bool bCullEmpty = true;
		Text.ToString().ParseIntoArray(Substrings, Delimiter, 4, bCullEmpty);

		static const FText InvalidStringErrorMessage = LOCTEXT("InvalidUniverseString", "Must be in the form of 'Universe' or 'Universe.Channel'. E.g. '4', or '4.5'.");

		FText ErrorMessage;
		int32 ParsedUniverse;
		if (Substrings.IsValidIndex(0) &&
			LexTryParseString(ParsedUniverse, *Substrings[0]))
		{
			if (ParsedUniverse < 1 || ParsedUniverse > DMX_MAX_UNIVERSE)
			{
				ErrorMessage = FText::Format(LOCTEXT("InvalidUniverseValue", "Universe must be between 1 and {0}."), FText::FromString(FString::FromInt((int32)DMX_MAX_UNIVERSE)));
			}
		}
		else if (Substrings.IsValidIndex(0))
		{
			ErrorMessage = InvalidStringErrorMessage;
		}

		// Channel is optional, only test if Substrings[1] was parsed
		int32 ParsedChannel;
		if (Substrings.IsValidIndex(1) &&
			LexTryParseString(ParsedChannel, *Substrings[1]))
		{
			if (ParsedChannel < 1 || ParsedChannel > DMX_MAX_ADDRESS)
			{
				ErrorMessage = FText::Format(LOCTEXT("InvalidChannelValue", "Channel must be between 1 and {0}."), FText::FromString(FString::FromInt((int32)DMX_MAX_ADDRESS)));
			}
		}
		else if (Substrings.IsValidIndex(1))
		{
			ErrorMessage = InvalidStringErrorMessage;
		}

		UniverseChannelEditableTextBox->SetError(ErrorMessage);
	}

	void SDMXAddFixturePatchMenu::OnUniverseChannelTextCommitted(const FText& Text, ETextCommit::Type CommitType)
	{
		if (!UniverseChannelEditableTextBox.IsValid())
		{
			return;
		}

		static const TCHAR* Delimiter[] = { TEXT("."), TEXT(","), TEXT(":"), TEXT(" ") };
		TArray<FString> Substrings;
		constexpr bool bCullEmpty = true;
		Text.ToString().ParseIntoArray(Substrings, Delimiter, 4, bCullEmpty);

		int32 NewUniverse;
		if (Substrings.IsValidIndex(0) &&
			LexTryParseString(NewUniverse, *Substrings[0]))
		{
			Universe = FMath::Clamp(NewUniverse, 1, DMX_MAX_UNIVERSE);
			SharedData->SelectUniverse(NewUniverse);
		}
		else
		{
			Universe.Reset();
		}

		int32 NewChannel;
		if (Universe.IsSet() && // Only parse the channel if parsing universe was successful
			Substrings.IsValidIndex(1) &&
			LexTryParseString(NewChannel, *Substrings[1]))
		{
			Channel = FMath::Clamp(NewChannel, 1, DMX_MAX_ADDRESS);
		}
		else
		{
			Channel.Reset();
		}

		if (UniverseChannelEditableTextBox.IsValid())
		{
			UniverseChannelEditableTextBox->SetError(FText::GetEmpty());
		}

		// Add fixture patches if enter is pressed
		if (CommitType == ETextCommit::OnEnter)
		{
			OnAddFixturePatchButtonClicked();
		}

		// Never show an error, values are always mended.
		UniverseChannelEditableTextBox->SetError(FText::GetEmpty());
	}

	FReply SDMXAddFixturePatchMenu::OnAddFixturePatchButtonClicked()
	{
		FSlateApplication::Get().DismissAllMenus();

		UDMXEntityFixtureType* FixtureType = Cast<UDMXEntityFixtureType>(WeakFixtureType);
		if (!FixtureType || FixtureType->Modes.IsEmpty())
		{
			return FReply::Handled();
		}

		const TSharedPtr<FDMXEditor> DMXEditor = WeakDMXEditor.Pin();
		if (!DMXEditor.IsValid())
		{
			return FReply::Handled();
		}

		UDMXLibrary* DMXLibrary = DMXEditor->GetDMXLibrary();
		if (!DMXLibrary)
		{
			return FReply::Handled();
		}

		// Ensure valid mode
		const UDMXAddFixturePatchMenuData* MenuData = GetDefault<UDMXAddFixturePatchMenuData>();
		if (!ensureMsgf(FixtureType->Modes.IsValidIndex(MenuData->ActiveModeIndex), TEXT("Cannot apply mode. Mode index is invalid.")))
		{
			return FReply::Handled();
		}

		// Create new fixture patches
		const FScopedTransaction CreateFixturePatchTransaction(LOCTEXT("CreateFixturePatchTransaction", "Create Fixture Patch"));
		DMXLibrary->PreEditChange(UDMXLibrary::StaticClass()->FindPropertyByName(UDMXLibrary::GetEntitiesPropertyName()));

		const int32 PatchToUniverse = Universe.IsSet() ? Universe.GetValue() : SharedData->GetSelectedUniverse();

		const float NumSteps = NumFixturePatchesToAdd;
		FScopedSlowTask Task(NumSteps, LOCTEXT("AddFixturePatchesSlowTask", "Adding Fixture Patches..."));
		Task.MakeDialogDelayed(.5f);

		TArray<UDMXEntityFixturePatch*> NewFixturePatches;
		NewFixturePatches.Reserve(NumFixturePatchesToAdd);
		for (uint32 iNumFixturePatchesAdded = 0; iNumFixturePatchesAdded < NumFixturePatchesToAdd; iNumFixturePatchesAdded++)
		{
			Task.EnterProgressFrame();

			FDMXEntityFixturePatchConstructionParams FixturePatchConstructionParams;
			FixturePatchConstructionParams.FixtureTypeRef = FDMXEntityFixtureTypeRef(FixtureType);
			FixturePatchConstructionParams.ActiveMode = MenuData->ActiveModeIndex;
			FixturePatchConstructionParams.UniverseID = PatchToUniverse;
			FixturePatchConstructionParams.StartingAddress = Channel.IsSet() ? Channel.GetValue() : 1;

			constexpr bool bMarkLibraryDirty = false;
			UDMXEntityFixturePatch* NewFixturePatch = UDMXEntityFixturePatch::CreateFixturePatchInLibrary(FixturePatchConstructionParams, FixtureType->Name, bMarkLibraryDirty);
			
			if (NewFixturePatch)
			{
				NewFixturePatches.Add(NewFixturePatch);
			}
		}

		// Align
		using namespace UE::DMXEditor::AutoAssign;
		FAutoAssignUtility::Align(NewFixturePatches);

		DMXLibrary->PostEditChange();

		// Select universe and new fixture patches
		SharedData->SelectUniverse(PatchToUniverse);

		TArray<TWeakObjectPtr<UDMXEntityFixturePatch>> NewWeakFixturePatches;
		NewWeakFixturePatches.Reserve(NumFixturePatchesToAdd);
		Algo::Copy(NewFixturePatches, NewWeakFixturePatches);
		SharedData->SelectFixturePatches(NewWeakFixturePatches);

		// Increment universe/channel if desired
		if (!NewFixturePatches.IsEmpty() && GetDefault<UDMXAddFixturePatchMenuData>()->bIncrementChannelAfterPatching)
		{
			Channel = NewFixturePatches.Last()->GetStartingChannel() + NewFixturePatches.Last()->GetChannelSpan();
			Universe = NewFixturePatches.Last()->GetUniverseID();
			if (Channel.GetValue() > DMX_UNIVERSE_SIZE)
			{
				Channel = 1;
				Universe = Universe.GetValue() + 1;
			}
		}

		return FReply::Handled();
	}

	FText SDMXAddFixturePatchMenu::GetActiveModeText() const
	{
		UDMXEntityFixtureType* FixtureType = Cast<UDMXEntityFixtureType>(WeakFixtureType);
		if (!FixtureType)
		{
			return FText::GetEmpty();
		}

		const TArray<FDMXFixtureMode>& Modes = FixtureType->Modes;
		const UDMXAddFixturePatchMenuData* MenuData = GetMutableDefault<UDMXAddFixturePatchMenuData>();
		if (Modes.Num() > 0 && Modes.IsValidIndex(MenuData->ActiveModeIndex))
		{
			return FText::FromString(Modes[MenuData->ActiveModeIndex].ModeName);
		}
		else if (Modes.IsEmpty())
		{
			return LOCTEXT("NoModeAvailableComboButtonText", "No Modes in Fixture Type");
		}
		else
		{
			return LOCTEXT("NoFixtureTypeSelectedComboButtonText", "No Fixture Type selected");
		}
	}

	bool SDMXAddFixturePatchMenu::HasValidFixtureTypeAndMode() const
	{
		return WeakFixtureType.IsValid() && !WeakFixtureType->Modes.IsEmpty();
	}
}

#undef LOCTEXT_NAMESPACE
