// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "Widgets/SCompoundWidget.h"

class FDMXEditor;
class FDMXFixturePatchSharedData;
class FReply;
template <typename OptionType> class SComboBox;
template <typename EntityType> class SDMXEntityPickerButton;
class SEditableTextBox;
class UDMXAddFixturePatchMenuData;
class UDMXEntity;
class UDMXEntityFixtureType;
class UDMXLibrary;


namespace UE::DMXEditor::FixturePatchEditor
{
	/** Editor for Fixture Patches */
	class SDMXAddFixturePatchMenu final
		: public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SDMXAddFixturePatchMenu)
		{}
		SLATE_END_ARGS()

		virtual ~SDMXAddFixturePatchMenu();

		/** Constructs this widget */
		void Construct(const FArguments& InArgs, TWeakPtr<FDMXEditor> InWeakDMXEditor);

		/** Refreshes the widget on the next tick */
		void RequestRefresh();

	private:
		/** Refreshes the widget */
		void Refresh();

		/** Creates a widget to select a fixture type */
		TSharedRef<SWidget> MakeFixtureTypeSelectWidget();

		/** Creates a widget to select a mode */
		TSharedRef<SWidget> MakeModeSelectWidget();

		/** Creates an widget to select the Universe and Channel */
		TSharedRef<SWidget> MakeUniverseChannelSelectWidget();
		
		/** Creates a widget to toggle if the channels should be auto-incremented after patching */
		TSharedRef<SWidget> MakeAutoIncrementChannelCheckBox();

		/** Creates an editable text box to specify the number of patches */
		TSharedRef<SWidget> MakeNumFixturePatchesEditableTextBox();

		/** Creates a button that adds the fixture patches when clicked */
		TSharedRef<SWidget> MakeAddFixturePatchesButton();

		/** Generates an entry in the mode combo box */
		TSharedRef<SWidget> GenerateModeComboBoxEntry(const TSharedPtr<uint32> InMode) const;

		/** Called when an entity was added or removed from the DMX Library */
		void OnEntityAddedOrRemoved(UDMXLibrary* DMXLibrary, TArray<UDMXEntity*> Entities);

		/** Called when a fixture type was selected */
		void OnFixtureTypeSelected(UDMXEntity* InSelectedFixtureType);

		/** Called when a mode was selected */
		void OnModeSelected(TSharedPtr<uint32> InSelectedMode, ESelectInfo::Type SelectInfo);

		/** Returns the current universe channel text */
		FText GetUniverseChannelText() const;

		/** Called when the universe channel text changed */
		void OnUniverseChannelTextChanged(const FText& InText);

		/** Called when the universe channel text was committed */
		void OnUniverseChannelTextCommitted(const FText& Text, ETextCommit::Type CommitType);

		/** Called when the 'Add' button was clicked */
		FReply OnAddFixturePatchButtonClicked();

		/** Returns the text of the active mode */
		FText GetActiveModeText() const;

		/** Returns if a valid fixture type with a mode is selected */
		bool HasValidFixtureTypeAndMode() const;

		/** Current fixture type */
		TWeakObjectPtr<UDMXEntityFixtureType> WeakFixtureType;

		/** Universe where the patches will be added */
		TOptional<int32> Universe;

		/** Channel where the patches will be added */
		TOptional<int32> Channel; 

		/** The number of fixture patches to add */
		uint32 NumFixturePatchesToAdd = 1;

		/** Timer handle for the request refresh combo box timer */
		FTimerHandle RequestRefreshModeComboBoxTimerHandle;

		/** Widget to select the fixture type */
		TSharedPtr<SDMXEntityPickerButton<UDMXEntityFixtureType>> FixtureTypeSelector;

		/** Text boxt to set the channel where the patch will be spawned */
		TSharedPtr<SEditableTextBox> UniverseChannelEditableTextBox;

		/** Sources for the mode combo box */
		TArray<TSharedPtr<uint32>> ModeSources;

		/** Combo box to selected the mode */
		TSharedPtr<SComboBox<TSharedPtr<uint32>>> ModeComboBox;

		/** Shared data for fixture patches */
		TSharedPtr<FDMXFixturePatchSharedData> SharedData;

		/** The DMX editor this widget displays */
		TWeakPtr<FDMXEditor> WeakDMXEditor;
	};
}
