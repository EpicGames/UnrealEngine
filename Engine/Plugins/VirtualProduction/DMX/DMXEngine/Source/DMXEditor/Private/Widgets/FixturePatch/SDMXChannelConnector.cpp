// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXChannelConnector.h"

#include "DMXEditor.h"
#include "DMXFixturePatchSharedData.h"
#include "Library/DMXLibrary.h"
#include "Library/DMXEntityFixturePatch.h"

#include "Widgets/SDMXChannel.h"


void SDMXChannelConnector::Construct(const FArguments& InArgs, TWeakPtr<FDMXEditor> InWeakDMXEditor)
{
	WeakDMXEditor = InWeakDMXEditor;

	OnHovered = InArgs._OnHovered;
	OnUnhovered = InArgs._OnUnhovered;
	OnMouseButtonDownOnChannel = InArgs._OnMouseButtonDownOnChannel;
	OnMouseButtonUpOnChannel = InArgs._OnMouseButtonUpOnChannel;
	OnDragDetectedOnChannel = InArgs._OnDragDetectedOnChannel;
	OnDragEnterChannel = InArgs._OnDragEnterChannel;
	OnDragLeaveChannel = InArgs._OnDragLeaveChannel;
	OnDropOntoChannel = InArgs._OnDropOntoChannel;

	ChannelID = InArgs._ChannelID;
	UniverseID = InArgs._UniverseID;

	ChildSlot
		[
			SAssignNew(ChannelValueWidget, SDMXChannel)
			.ChannelID(ChannelID)
			.Value(InArgs._Value)	
			.bShowChannelIDBottom(true)
			.ChannelIDTextColor(FLinearColor::White.CopyWithNewOpacity(.6f))
			.ValueTextColor(FLinearColor::White.CopyWithNewOpacity(.6f))
			.Visibility(EVisibility::HitTestInvisible)
			.ChannelIDTextColor(InArgs._ChannelIDTextColor)
			.ValueTextColor(InArgs._ValueTextColor)
		];
}

void SDMXChannelConnector::SetValue(uint8 Value)
{
	ChannelValueWidget->SetValue(Value);
}

void SDMXChannelConnector::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	OnHovered.ExecuteIfBound();
}

void SDMXChannelConnector::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	OnUnhovered.ExecuteIfBound();
}

FReply SDMXChannelConnector::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (OnMouseButtonDownOnChannel.IsBound())
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			return OnMouseButtonDownOnChannel.Execute(ChannelValueWidget->GetChannelID(), MouseEvent).DetectDrag(AsShared(), EKeys::LeftMouseButton);
		}
		else
		{
			return OnMouseButtonDownOnChannel.Execute(ChannelValueWidget->GetChannelID(), MouseEvent);
		}
	}

	return FReply::Unhandled();
}

FReply SDMXChannelConnector::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (OnMouseButtonUpOnChannel.IsBound())
	{
		return OnMouseButtonUpOnChannel.Execute(ChannelValueWidget->GetChannelID(), MouseEvent);
	}

	return FReply::Unhandled();
}

FReply SDMXChannelConnector::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (OnDragDetectedOnChannel.IsBound())
	{
		return OnDragDetectedOnChannel.Execute(ChannelValueWidget->GetChannelID(), MouseEvent);
	}

	return FReply::Unhandled();
}

void SDMXChannelConnector::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	OnDragEnterChannel.ExecuteIfBound(ChannelValueWidget->GetChannelID(), DragDropEvent);
}

void SDMXChannelConnector::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	OnDragLeaveChannel.ExecuteIfBound(ChannelValueWidget->GetChannelID(), DragDropEvent);
}

FReply SDMXChannelConnector::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (OnDropOntoChannel.IsBound())
	{
		return OnDropOntoChannel.Execute(ChannelValueWidget->GetChannelID(), DragDropEvent);
	}

	return FReply::Unhandled();
}

UDMXLibrary* SDMXChannelConnector::GetDMXLibrary() const
{
	if (TSharedPtr<FDMXEditor> DMXEditor = WeakDMXEditor.Pin())
	{
		return DMXEditor->GetDMXLibrary();
	}
	return nullptr;
}
