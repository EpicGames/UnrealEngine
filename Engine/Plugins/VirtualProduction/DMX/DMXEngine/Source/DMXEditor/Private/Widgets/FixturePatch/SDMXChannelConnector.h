// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FDMXEditor;
class UDMXLibrary;
class UDMXEntityFixturePatch;


/** Visual representation of a connection to a channel in a dmx universe */
class SDMXChannelConnector
	: public SCompoundWidget
{
	DECLARE_DELEGATE_RetVal_TwoParams(FReply, FOnMouseEventWithReply, uint32 /** ChannelID */, const FPointerEvent&);

	DECLARE_DELEGATE_TwoParams(FOnDragEvent, uint32 /** ChannelID */, const FDragDropEvent&);

	DECLARE_DELEGATE_RetVal_TwoParams(FReply, FOnDragEventWithReply, uint32 /** ChannelID */, const FDragDropEvent&);

public:
	SLATE_BEGIN_ARGS(SDMXChannelConnector)
		: _ChannelID(0)
		, _UniverseID(-1)
		, _Value(0.0f)
		, _OnDragEnterChannel()
		, _OnDragLeaveChannel()
		, _OnDropOntoChannel()
	{}
		/** The channel ID this widget stands for */
		SLATE_ARGUMENT(int32, ChannelID)

		/** The Universe ID this widget resides in */
		SLATE_ARGUMENT(int32, UniverseID)

		/** The value of the channel */
		SLATE_ARGUMENT(uint8, Value)

		/** Called when the widget is hovered */
		SLATE_EVENT(FSimpleDelegate, OnHovered)

		/** Called when the widget is unhovered */
		SLATE_EVENT(FSimpleDelegate, OnUnhovered)

		/** Called when Mouse Button was pressed on the Channel */
		SLATE_EVENT(FOnMouseEventWithReply, OnMouseButtonDownOnChannel)

		/** Called when Mouse Button was released on the Channel */
		SLATE_EVENT(FOnMouseEventWithReply, OnMouseButtonUpOnChannel)

		/** Called when it was detected that the Channel was dragged */
		SLATE_EVENT(FOnMouseEventWithReply, OnDragDetectedOnChannel)

		/** Called when drag enters the Widget */
		SLATE_EVENT(FOnDragEvent, OnDragEnterChannel)

		/** Called when drag leaves the Widget */
		SLATE_EVENT(FOnDragEvent, OnDragLeaveChannel)

		/** Called when dropped onto the Channel */
		SLATE_EVENT(FOnDragEventWithReply, OnDropOntoChannel)

		/** Color of the Channel ID Text */
		SLATE_ATTRIBUTE(FSlateColor, ChannelIDTextColor)

		/** Color of the Value Text */
		SLATE_ATTRIBUTE(FSlateColor, ValueTextColor)

	SLATE_END_ARGS()

	/** Constructs this widget */
	void Construct(const FArguments& InArgs, TWeakPtr<FDMXEditor> InWeakDMXEditor);

	/** Returns the ChannelID of this connector */
	int32 GetChannelID() const { return ChannelID; }

	/** Returns the UniverseID of this connector */
	int32 GetUniverseID() const { return UniverseID; }

	/** Sets the UniverseID of this connector */
	void SetUniverseID(int32 InUniverseID) { UniverseID = InUniverseID; }

	/** Sets the displayed value */
	void SetValue(uint8 Value);

protected:
	// Begin SWidget interface	
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& PointerEvent) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	// End SWidget interface

private:
	/** Returns the DMXLibrary or nullptr if not available */
	UDMXLibrary* GetDMXLibrary() const;

	int32 Column = INDEX_NONE;
	int32 Row = INDEX_NONE;

	TSharedPtr<class SDMXChannel> ChannelValueWidget;

	int32 ChannelID;

	int32 UniverseID;

	TWeakPtr<FDMXEditor> WeakDMXEditor;

	// Slate Arguments
	FSimpleDelegate OnHovered;
	FSimpleDelegate OnUnhovered;
	FOnMouseEventWithReply OnMouseButtonDownOnChannel;
	FOnMouseEventWithReply OnMouseButtonUpOnChannel;
	FOnMouseEventWithReply OnDragDetectedOnChannel;
	FOnDragEvent OnDragEnterChannel;
	FOnDragEvent OnDragLeaveChannel;
	FOnDragEventWithReply OnDropOntoChannel;
};
