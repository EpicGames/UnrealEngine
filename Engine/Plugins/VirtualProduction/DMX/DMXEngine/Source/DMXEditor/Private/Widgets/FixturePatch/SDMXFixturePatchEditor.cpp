// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/FixturePatch/SDMXFixturePatchEditor.h"

#include "Customizations/DMXEntityFixturePatchDetails.h"
#include "DMXEditor.h"
#include "DMXEditorSettings.h"
#include "DMXFixturePatchSharedData.h"
#include "DMXSubsystem.h"
#include "Engine/TimerHandle.h"
#include "IO/DMXOutputPort.h"
#include "IO/DMXOutputPortReference.h"
#include "IO/DMXPortManager.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXLibrary.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SDMXFixturePatcher.h"
#include "Widgets/FixturePatch/SDMXFixturePatchList.h"
#include "Widgets/Layout/SSplitter.h"

#define LOCTEXT_NAMESPACE "SDMXFixturePatcher"

SDMXFixturePatchEditor::~SDMXFixturePatchEditor()
{
	const float LeftSideWidth = LhsRhsSplitter->SlotAt(0).GetSizeValue();

	UDMXEditorSettings* const DMXEditorSettings = GetMutableDefault<UDMXEditorSettings>();
	DMXEditorSettings->MVRFixtureListSettings.ListWidth = LeftSideWidth;
	DMXEditorSettings->SaveConfig();
}

void SDMXFixturePatchEditor::Construct(const FArguments& InArgs)
{
	SDMXEntityEditor::Construct(SDMXEntityEditor::FArguments());

	WeakDMXEditor = InArgs._DMXEditor;
	if (!WeakDMXEditor.IsValid())
	{
		return;
	}
	FixturePatchSharedData = WeakDMXEditor.Pin()->GetFixturePatchSharedData();

	SetCanTick(false);

	FixturePatchDetailsView = GenerateFixturePatchDetailsView();

	const UDMXEditorSettings* const DMXEditorSettings = GetDefault<UDMXEditorSettings>();
	const float LeftSideWidth = FMath::Clamp(DMXEditorSettings->MVRFixtureListSettings.ListWidth, 0.1f, 0.9f);
	const float RightSideWidth = FMath::Max(1.f - DMXEditorSettings->MVRFixtureListSettings.ListWidth, .1f);

	ChildSlot
	.VAlign(VAlign_Fill)
	.HAlign(HAlign_Fill)
	[
		SAssignNew(LhsRhsSplitter, SSplitter)
		.Orientation(EOrientation::Orient_Horizontal)
		.ResizeMode(ESplitterResizeMode::FixedPosition)
		
		// Left, MVR Fixture List
		+ SSplitter::Slot()	
		.Value(LeftSideWidth)
		[
			SAssignNew(FixturePatchList, SDMXFixturePatchList, WeakDMXEditor)
		]

		// Right, Fixture Patcher and Details
		+ SSplitter::Slot()	
		.Value(RightSideWidth)
		[
			SNew(SSplitter)
			.Orientation(EOrientation::Orient_Vertical)
			.ResizeMode(ESplitterResizeMode::FixedPosition)

			+SSplitter::Slot()			
			.Value(.618f)
			[
				SAssignNew(FixturePatcher, SDMXFixturePatcher)
				.DMXEditor(WeakDMXEditor)
			]
	
			+SSplitter::Slot()
			.Value(.382f)
			[
				FixturePatchDetailsView.ToSharedRef()
			]
		]
	];

	// Adopt the selection
	OnFixturePatchesSelected();

	// Listen to selection changes
	FixturePatchSharedData->OnFixturePatchSelectionChanged.AddSP(this, &SDMXFixturePatchEditor::OnFixturePatchesSelected);

	// Listen to Fixture Patch and Fixture Type changes
	UDMXEntityFixtureType::GetOnFixtureTypeChanged().AddSP(this, &SDMXFixturePatchEditor::OnFixtureTypeChanged);
	UDMXEntityFixturePatch::GetOnFixturePatchChanged().AddSP(this, &SDMXFixturePatchEditor::OnFixturePatchChanged);
}

FReply SDMXFixturePatchEditor::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	return FixturePatchList->ProcessCommandBindings(InKeyEvent);
}

void SDMXFixturePatchEditor::RequestRenameOnNewEntity(const UDMXEntity* InEntity, ESelectInfo::Type SelectionType)
{
	if (FixturePatchList.IsValid())
	{
		FixturePatchList->EnterFixturePatchNameEditingMode();
	}
}

void SDMXFixturePatchEditor::SelectEntity(UDMXEntity* InEntity, ESelectInfo::Type InSelectionType /*= ESelectInfo::Type::Direct*/)
{
	if (UDMXEntityFixturePatch* FixturePatch = Cast<UDMXEntityFixturePatch>(InEntity))
	{
		FixturePatchSharedData->SelectFixturePatch(FixturePatch);
	}
}

void SDMXFixturePatchEditor::SelectEntities(const TArray<UDMXEntity*>& InEntities, ESelectInfo::Type InSelectionType /*= ESelectInfo::Type::Direct*/)
{
	TArray<TWeakObjectPtr<UDMXEntityFixturePatch>> FixturePatches;
	for (UDMXEntity* Entity : InEntities)
	{
		if (UDMXEntityFixturePatch* FixturePatch = Cast<UDMXEntityFixturePatch>(Entity))
		{
			FixturePatches.Add(FixturePatch);
		}
	}
	FixturePatchSharedData->SelectFixturePatches(FixturePatches);
}

TArray<UDMXEntity*> SDMXFixturePatchEditor::GetSelectedEntities() const
{
	TArray<TWeakObjectPtr<UDMXEntityFixturePatch>> SelectedFixturePatches = FixturePatchSharedData->GetSelectedFixturePatches();
	TArray<UDMXEntity*> SelectedEntities;
	for (TWeakObjectPtr<UDMXEntityFixturePatch> WeakSelectedFixturePatch : SelectedFixturePatches)
	{
		if (UDMXEntity* Entity = WeakSelectedFixturePatch.Get())
		{
			SelectedEntities.Add(Entity);
		}
	}

	return SelectedEntities;
}

TSharedRef<IDetailsView> SDMXFixturePatchEditor::GenerateFixturePatchDetailsView() const
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = true;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;

	TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView->RegisterInstancedCustomPropertyLayout(UDMXEntityFixturePatch::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FDMXEntityFixturePatchDetails::MakeInstance, WeakDMXEditor));

	return DetailsView;
}

void SDMXFixturePatchEditor::SelectUniverse(int32 UniverseID)
{
	if (!ensureMsgf(UniverseID >= 0 && UniverseID <= DMX_MAX_UNIVERSE, TEXT("Invalid Universe when trying to select Universe %i."), UniverseID))
	{
		return;
	}

	FixturePatchSharedData->SelectUniverse(UniverseID);
}

void SDMXFixturePatchEditor::OnFixturePatchesSelected()
{
	RequestRefreshFixturePatchDetailsView();
}

void SDMXFixturePatchEditor::OnFixtureTypeChanged(const UDMXEntityFixtureType* ChangedFixtureType)
{
	RequestRefreshFixturePatchDetailsView();
}

void SDMXFixturePatchEditor::OnFixturePatchChanged(const UDMXEntityFixturePatch* ChangedFixturePatch)
{
	RequestRefreshFixturePatchDetailsView();
}

void SDMXFixturePatchEditor::RequestRefreshFixturePatchDetailsView()
{
	if (!RefreshFixturePatchDetailsViewTimerHandle.IsValid())
	{
		RefreshFixturePatchDetailsViewTimerHandle = GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateSP(this, &SDMXFixturePatchEditor::RefreshFixturePatchDetailsView));
	}
}

void SDMXFixturePatchEditor::RefreshFixturePatchDetailsView()
{
	const TSharedPtr<FDMXEditor> DMXEditor = WeakDMXEditor.Pin();
	UDMXLibrary* DMXLibrary = DMXEditor.IsValid() ? DMXEditor->GetDMXLibrary() : nullptr;
	
	if (DMXLibrary && FixturePatchDetailsView.IsValid())
	{
		RefreshFixturePatchDetailsViewTimerHandle.Invalidate();

		const TArray<TWeakObjectPtr<UDMXEntityFixturePatch>> SelectedFixturePatches = FixturePatchSharedData->GetSelectedFixturePatches();
		
		// Try to make a valid selection if nothing is selected
		if (SelectedFixturePatches.IsEmpty())
		{		
			const TArray<UDMXEntityFixturePatch*> FixturePatches = DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>();
			if (!FixturePatches.IsEmpty())
			{
				FixturePatchSharedData->SelectFixturePatch(FixturePatches[0]);
			}
		}

		TArray<UObject*> SelectedObjects;
		Algo::TransformIf(SelectedFixturePatches, SelectedObjects,
			[](const TWeakObjectPtr<UDMXEntityFixturePatch>& WeakFixturePatch)
			{
				return WeakFixturePatch.IsValid();
			},
			[](const TWeakObjectPtr<UDMXEntityFixturePatch>& WeakFixturePatch)
			{
				return WeakFixturePatch.Get();
			});

		FixturePatchDetailsView->SetObjects(SelectedObjects);
	}
}

#undef LOCTEXT_NAMESPACE
