// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TimerHandle.h"
#include "Widgets/SDMXEntityEditor.h"

class FDMXEditor;
class FDMXFixturePatchSharedData;
class IDetailsView;
class SDMXFixturePatcher;
class SDMXFixturePatchTree;
class SDMXFixturePatchList;
class SSplitter;
class UDMXEntityFixturePatch;
class UDMXEntityFixtureType;
struct FPropertyChangedEvent;

/** Editor for Fixture Patches */
class SDMXFixturePatchEditor final
	: public SDMXEntityEditor
{
public:
	SLATE_BEGIN_ARGS(SDMXFixturePatchEditor)
		: _DMXEditor(nullptr)
	{}
	
		SLATE_ARGUMENT(TWeakPtr<FDMXEditor>, DMXEditor)

	SLATE_END_ARGS()

public:	
	/** Destructor */
	~SDMXFixturePatchEditor();

	/** Constructs this widget */
	void Construct(const FArguments& InArgs);

public:
	//~ Begin SWidget interface
	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget interface

	// Begin SDMXEntityEditorTab interface
	void RequestRenameOnNewEntity(const UDMXEntity* InEntity, ESelectInfo::Type SelectionType);
	void SelectEntity(UDMXEntity* InEntity, ESelectInfo::Type InSelectionType = ESelectInfo::Type::Direct);
	void SelectEntities(const TArray<UDMXEntity*>& InEntities, ESelectInfo::Type SelectionType = ESelectInfo::Type::Direct);
	TArray<UDMXEntity*> GetSelectedEntities() const;
	// ~End SDMXEntityEditorTab interface 

private:
	/** Generates a Detail View for the edited Fixture Patch */
	TSharedRef<IDetailsView> GenerateFixturePatchDetailsView() const;
	
	/** Selects the patch */
	void SelectUniverse(int32 UniverseID);

	/** Called whewn Fixture Patches were selected in Fixture Patch Shared Data */
	void OnFixturePatchesSelected();

	/** Called when a Fixture Type changed */
	void OnFixtureTypeChanged(const UDMXEntityFixtureType* ChangedFixtureType);

	/** Called when a Fixture Patch changed */
	void OnFixturePatchChanged(const UDMXEntityFixturePatch* ChangedFixturePatch);

	/** Refreshes the Fixture Patch Details View on the next tick */
	void RequestRefreshFixturePatchDetailsView();

	/** Refreshes the Fixture Patch Details View */
	void RefreshFixturePatchDetailsView();

	/** List of Fixture Patches as MVR Fixtures */
	TSharedPtr<SDMXFixturePatchList> FixturePatchList;

	/** Details View for the selected Fixture Patches */
	TSharedPtr<IDetailsView> FixturePatchDetailsView;

	/** The main splitter that divides the view in an left and right side */
	TSharedPtr<SSplitter> LhsRhsSplitter;

	/** Widget where the user can drag drop fixture patches */
	TSharedPtr<SDMXFixturePatcher> FixturePatcher;

	/** Shared data for Fixture Patches */
	TSharedPtr<FDMXFixturePatchSharedData> FixturePatchSharedData;

	/** Pointer back to the DMXEditor tool that owns us */
	TWeakPtr<FDMXEditor> WeakDMXEditor;

	/** Timer handle to refresh the fixture patch details view */
	FTimerHandle RefreshFixturePatchDetailsViewTimerHandle;
}; 
