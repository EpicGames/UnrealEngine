// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXFixturePatchFragment.h"

#include "DMXEditor.h"
#include "DMXEditorSettings.h"
#include "DMXEditorStyle.h"
#include "DMXFixturePatchNode.h"
#include "DMXFixturePatchSharedData.h"
#include "SDMXChannelConnector.h"
#include "Library/DMXEntityFixturePatch.h"

#include "Rendering/DrawElementPayloads.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"


#define LOCTEXT_NAMESPACE "SDMXFixturePatchFragment"

/** A widget that draws the color of a fixture patch depending on its state */
class SDMXFixturePatchFragmentColor
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMXFixturePatchFragmentColor)
		: _bIsTopmost(false)
		, _bIsConflicting(false)
	{}
		SLATE_DEFAULT_SLOT(FArguments, Content)

		SLATE_ARGUMENT(bool, bIsTopmost)

		SLATE_ARGUMENT(bool, bIsConflicting)

		SLATE_ATTRIBUTE(bool, IsNodeHovered)

	SLATE_END_ARGS()
			
	/** Constructs this widget */
	void Construct(const FArguments& InArgs, const TSharedPtr<FDMXFixturePatchNode>& InFixturePatchNode)
	{
		FixturePatchNode = InFixturePatchNode;

		bIsTopmost = InArgs._bIsTopmost;
		bIsConflicting = InArgs._bIsConflicting;
		IsNodeHovered = InArgs._IsNodeHovered;

		ConflictBrush = FDMXEditorStyle::Get().GetBrush("FixturePatchFragment.ConflictBackground");
		SelectedBrush = FDMXEditorStyle::Get().GetBrush("DMXEditor.WhiteBrush");

		ChildSlot
			[
				InArgs._Content.Widget
			];
	}

	/** Sets the content of the widget */
	void SetContent(const TSharedRef<SWidget>& InContent)
	{
		ChildSlot
		[
			InContent
		];
	}

protected:
	//~ Begin SWidget interface
	int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
	{
		if (!bIsTopmost)
		{
			return LayerId + 1;
		}

		const UDMXEntityFixturePatch* FixturePatch = FixturePatchNode->GetFixturePatch().Get();
		if (!FixturePatch)
		{
			return LayerId + 1;
		}

		const bool bEnabled = ShouldBeEnabled(bParentEnabled);
		if (bIsConflicting)
		{
			const FLinearColor BackgroundColor = ConflictBrush->GetTint(InWidgetStyle);

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(),
				ConflictBrush,
				ESlateDrawEffect::None,
				BackgroundColor);
		}
		else if (FixturePatchNode->IsSelected())
		{
			const FLinearColor BackgroundColor = FixturePatch->EditorColor.CopyWithNewOpacity(.5f);

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(),
				SelectedBrush,
				ESlateDrawEffect::None,
				BackgroundColor);
		}
		else 
		{
			const float InvertedLuminanceClamped = FMath::Clamp(FMath::Abs(1.f - FixturePatch->EditorColor.GetLuminance()), .5f, .8f);
			const float Opacity = IsNodeHovered.Get() ? InvertedLuminanceClamped : InvertedLuminanceClamped / 2.f;

			TArray<FSlateGradientStop> GradientStops;
			GradientStops.Add(FSlateGradientStop(FVector2D::ZeroVector, FixturePatch->EditorColor.CopyWithNewOpacity(Opacity)));
			GradientStops.Add(FSlateGradientStop(AllottedGeometry.GetLocalSize(), FixturePatch->EditorColor.CopyWithNewOpacity(Opacity / 4.f)));
			GradientStops.Add(FSlateGradientStop(AllottedGeometry.GetLocalSize(), FLinearColor::Transparent));

			FSlateDrawElement::MakeGradient(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(),
				GradientStops,
				EOrientation::Orient_Horizontal,
				ESlateDrawEffect::None | ESlateDrawEffect::NoGamma
			);
		}

		return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	}
	//~ End SWidget interface

private:
	/** Topmost attribute */
	bool bIsTopmost = false;

	/** Is conflicting attribute */
	bool bIsConflicting = false;

	/** True if parent is hovered */
	TAttribute<bool> IsNodeHovered = false;

	/** Brush used to draw the selected color */
	const FSlateBrush* SelectedBrush = nullptr;

	/** Brush used to draw the warning color */
	const FSlateBrush* ConflictBrush = nullptr;

	/** Fixture Patch Node being displayed by this widget */
	TSharedPtr<FDMXFixturePatchNode> FixturePatchNode;
};

/** Fixture patch text widget */
class SDMXFixturePatchFragmentText
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMXFixturePatchFragmentText)
	{}

	SLATE_END_ARGS()

	/** Constructs this widget */
	void Construct(const FArguments& InArgs, const TSharedPtr<FDMXFixturePatchNode>& InFixturePatchNode, const TArray<TSharedPtr<FDMXFixturePatchNode>>& InFixturePatchNodeGroup)
	{
		FixturePatchNode = InFixturePatchNode;
		FixturePatchNodeGroup = InFixturePatchNodeGroup;

		InitializeNodeGroupText();

		ChildSlot
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Top)
		.Padding(2.f)
		[
			SNew(SBorder)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Top)
			.Padding(FMargin(2.f, 2.f, 0.f, 2.f))
			.BorderImage(FDMXEditorStyle::Get().GetBrush("DMXEditor.RoundedPropertyBorder"))
			[
				SNew(SHorizontalBox)
				.Clipping(EWidgetClipping::ClipToBoundsAlways)
				.Visibility(EVisibility::Visible)

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Top)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(this, &SDMXFixturePatchFragmentText::GetNameText)
					.TextStyle(FAppStyle::Get(), "SmallText")
					.ColorAndOpacity(FLinearColor::White)
				]
				
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Top)
				.AutoWidth()
				.Padding(FMargin(10.f, 0.f, 0.f, 0.f))
				[
					SNew(STextBlock)
					.Text_Lambda([this]()
						{
							return NodeGroupText;
						})
					.TextStyle(FAppStyle::Get(), "SmallText")
					.ColorAndOpacity(FLinearColor(.5f, .5f, .5f))
				]
			]
		];
	}

private:
	/** Updates the Fixture Patch Name Text */
	FText GetNameText() const
	{
		if (UDMXEntityFixturePatch* FixturePatch = FixturePatchNode->GetFixturePatch().Get())
		{
			const UDMXEditorSettings* DMXEditorSettings = GetDefault<UDMXEditorSettings>();

			const EDMXFixturePatcherNameDisplayMode NameDisplayMode = DMXEditorSettings->FixturePatcherSettings.FixturePatchNameDisplayMode;
			if (NameDisplayMode == EDMXFixturePatcherNameDisplayMode::FixtureIDAndFixturePatchName)
			{
				return FText::FromString(FString::Printf(TEXT("%s - %s"), *FixturePatchNode->GetFixtureID(), *FixturePatch->GetDisplayName()));
			}
			else if (NameDisplayMode == EDMXFixturePatcherNameDisplayMode::FixtureID)
			{
				return FText::FromString(FixturePatchNode->GetFixtureID());
			}
			else if (NameDisplayMode == EDMXFixturePatcherNameDisplayMode::FixturePatchName)
			{
				return FText::FromString(FixturePatch->GetDisplayName());
			}
			else
			{
				checkf(0, TEXT("Unhandled Fixture Patch Name Display Mode"));
			}
		}

		return FText::GetEmpty();
	}

	/** Updates the Node Group Text */
	void InitializeNodeGroupText()
	{
		NodeGroupText =
			FixturePatchNodeGroup.Num() > 1 ?
			FText::Format(LOCTEXT("NodeGroupText", "(and {0} more)"), FixturePatchNodeGroup.Num() - 1) :
			FText::GetEmpty();
	}

	/** The Node Group Text the Node displays */
	FText NodeGroupText;

	/** Fixture Patch Node being displayed by this widget */
	TSharedPtr<FDMXFixturePatchNode> FixturePatchNode;

	/** The group of nodes (same Universe, same Channel) this widget belongs to */
	TArray<TSharedPtr<FDMXFixturePatchNode>> FixturePatchNodeGroup;
};


void SDMXFixturePatchFragment::Construct(const FArguments& InArgs, const TSharedPtr<FDMXFixturePatchNode>& InFixturePatchNode, const TArray<TSharedPtr<FDMXFixturePatchNode>>& InFixturePatchNodeGroup)
{
	check(InFixturePatchNode.IsValid());
	check(InFixturePatchNodeGroup.Contains(InFixturePatchNode));
	check(InArgs._StartingChannel != -1);
	check(InArgs._Column != -1);
	check(InArgs._Row != -1);
	check(InArgs._ColumnSpan != -1);

	FixturePatchNode = InFixturePatchNode;
	FixturePatchNodeGroup = InFixturePatchNodeGroup;
	StartingChannel = InArgs._StartingChannel;
	Column = InArgs._Column;
	Row = InArgs._Row;
	ColumnSpan = InArgs._ColumnSpan;

	bIsHead = InArgs._bIsHead;
	bIsTail = InArgs._bIsTail;
	bIsConflicting = InArgs._bIsConflicting;

	SetVisibility(EVisibility::HitTestInvisible);

	InitializeIsTopmost();
	InitializeBorderPadding();

	ChildSlot
	[
		SAssignNew(ContentOverlay, SOverlay)
	
		+ SOverlay::Slot()
		[
			SNew(SDMXFixturePatchFragmentColor, FixturePatchNode)
			.bIsTopmost(bIsTopmost)
			.bIsConflicting(bIsConflicting)
			.IsNodeHovered_Lambda([this]()
				{
					return FixturePatchNode->IsHovered();
				})
		]
		
		+ SOverlay::Slot()
		[
			SNew(SBox)
			.Padding(BorderPadding)
			[
				SNew(SBorder)
				.Padding(0.f)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.BorderImage(GetBorderBrush())
				.BorderBackgroundColor(this, &SDMXFixturePatchFragment::GetBorderColor)
			]
		]

		+ SOverlay::Slot()
		[
			SAssignNew(ChannelsHorizontalBox, SHorizontalBox)
		]
	];

	if (bIsTopmost)
	{	
		// Add fixture patch text, only for the topmost head node
		if (bIsHead)
		{
			ContentOverlay->AddSlot(3)
			[
				SNew(SBox)
				.MaxDesiredHeight(1.0f) // Prevents from resizing the widget to text size
				.MaxDesiredWidth(1.0f) // Prevents from resizing the widget to text size
				.Padding(BorderPadding)
				[
					SNew(SDMXFixturePatchFragmentText, FixturePatchNode, FixturePatchNodeGroup)
				]
			];
		}

		// Add channel connectors
		for (int32 Channel = StartingChannel; Channel < StartingChannel + ColumnSpan; Channel++)
		{
			ChannelsHorizontalBox->AddSlot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SDMXChannelConnector, DMXEditorPtr)
					.ChannelID(Channel)
					.UniverseID(FixturePatchNode->GetUniverseID())
					.ChannelIDTextColor(this, &SDMXFixturePatchFragment::GetChannelIDTextColor)
					.ValueTextColor(this, &SDMXFixturePatchFragment::GetValueTextColor)
				];
		}
	}
}

void SDMXFixturePatchFragment::SetChannelVisibility(EVisibility NewVisibility)
{
	ChannelsHorizontalBox->SetVisibility(NewVisibility);
}

void SDMXFixturePatchFragment::InitializeBorderPadding()
{
	if (bIsHead && FixturePatchNodeGroup.Num() > 1)
	{
		// Sort by ZOrder ascending
		FixturePatchNodeGroup.StableSort([](const TSharedPtr<FDMXFixturePatchNode>& NodeA, const TSharedPtr<FDMXFixturePatchNode>& NodeB)
			{
				return
					NodeA->GetZOrder() < NodeB->GetZOrder();
			});

		const int32 IndexOfThis = FixturePatchNodeGroup.IndexOfByKey(FixturePatchNode);

		constexpr int32 OffsetPerIndex = 3;
		const int32 ClampedIndexOfThis = FMath::Min(IndexOfThis, 5);
		const int32 OffsetLeft = ClampedIndexOfThis * OffsetPerIndex;

		BorderPadding = FMargin(OffsetLeft, 0.f, 0.f, 0.f);
	}
	else
	{
		BorderPadding = FMargin(0.f);
	}
}

void SDMXFixturePatchFragment::InitializeIsTopmost()
{
	if (!ensureMsgf(FixturePatchNodeGroup.Num() > 0, TEXT("Unexpected empty node group for Fixture Patch Fragment Widget.")))
	{
		return;
	}

	bIsTopmost = !FixturePatchNodeGroup.ContainsByPredicate([this](const TSharedPtr<FDMXFixturePatchNode>& OtherNode)
		{
			const bool bOtherHasHigherZOrder = OtherNode->GetZOrder() > FixturePatchNode->GetZOrder();
			const bool bOtherNodeIntersects = OtherNode->GetStartingChannel() + OtherNode->GetChannelSpan() > StartingChannel;

			return bOtherHasHigherZOrder && bOtherNodeIntersects;
		});
}

const FSlateBrush* SDMXFixturePatchFragment::GetBorderBrush() const
{
	if (bIsHead && bIsTail)
	{
		return FDMXEditorStyle::Get().GetBrush("FixturePatcher.FragmentBorder.Normal");
	}
	else if (bIsHead)
	{
		return FDMXEditorStyle::Get().GetBrush("FixturePatcher.FragmentBorder.L");
	}
	else if (bIsTail)
	{
		return FDMXEditorStyle::Get().GetBrush("FixturePatcher.FragmentBorder.R");
	}
	else
	{
		return FDMXEditorStyle::Get().GetBrush("FixturePatcher.FragmentBorder.TB");
	}
}

FSlateColor SDMXFixturePatchFragment::GetBorderColor() const
{
	// Note all colors need be without oppacity so they draw over the conflict background brush
	if (FixturePatchNode->IsSelected())
	{
		return FLinearColor::White;
	}
	else if (UDMXEntityFixturePatch* Patch = FixturePatchNode->GetFixturePatch().Get())
	{
		const FLinearColor WhiteAdjustedColor = (Patch->EditorColor * 2.f + FLinearColor::White) / 3.f;
		return WhiteAdjustedColor.CopyWithNewOpacity(0.8f);
	}
	else
	{
		return FLinearColor(1.f, 0.f, 1.f);
	}
}

FSlateColor SDMXFixturePatchFragment::GetChannelIDTextColor() const
{
	if (bIsConflicting)
	{
		return FLinearColor::Black;
	}
	else if (FixturePatchNode->IsHovered() || FixturePatchNode->IsSelected())
	{
		return FLinearColor::White;
	}
	else
	{
		return FLinearColor(.8f, .8f, .8f, 1.f);
	}
}

FSlateColor SDMXFixturePatchFragment::GetValueTextColor() const
{
	if (bIsConflicting)
	{
		return FLinearColor::Black;
	}
	else if (FixturePatchNode->IsHovered() || FixturePatchNode->IsSelected())
	{
		return FLinearColor::White;
	}
	else
	{
		return FLinearColor(.6f, .6f, .6f, 1.f);
	}
}

#undef LOCTEXT_NAMESPACE
