// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FDMXEditor;
class FDMXFixturePatchNode;

class SHorizontalBox;
class SOverlay;


class SDMXFixturePatchFragment
	: public SCompoundWidget
{
	DECLARE_DELEGATE_RetVal_TwoParams(FReply, FOnMouseEventWithReply, uint32 /** ChannelID */, const FPointerEvent&);

	DECLARE_DELEGATE_TwoParams(FOnDragEvent, uint32 /** ChannelID */, const FDragDropEvent&);

	DECLARE_DELEGATE_RetVal_TwoParams(FReply, FOnDragEventWithReply, uint32 /** ChannelID */, const FDragDropEvent&);

public:
	SLATE_BEGIN_ARGS(SDMXFixturePatchFragment)
		: _bIsHead(true)
		, _bIsTail(true)
		, _bIsConflicting(false)
		, _Column(-1)
		, _Row(-1)
		, _ColumnSpan(1)
	{}
		/** If true displays as head of a patch */
		SLATE_ARGUMENT(bool, bIsHead)

		/** If true displays as tail of a patch */
		SLATE_ARGUMENT(bool, bIsTail)

		/** If true displays a conflict */
		SLATE_ARGUMENT(bool, bIsConflicting)

		/** The starting channel of the fragment */
		SLATE_ARGUMENT(int32, StartingChannel)

		/** The column in which the fragment should be displayed */
		SLATE_ARGUMENT(int32, Column)

		/** The row in which the fragment should be displayed */
		SLATE_ARGUMENT(int32, Row)

		/** The column span of the fragment */
		SLATE_ARGUMENT(int32, ColumnSpan)

	SLATE_END_ARGS()

	/** Constructs this widget */
	void Construct(const FArguments& InArgs, const TSharedPtr<FDMXFixturePatchNode>& InFixturePatchNode, const TArray<TSharedPtr<FDMXFixturePatchNode>>& InFixturePatchNodeGroup);

	/** Sets if the fragment is conflicting with others */
	void SetConflicting(bool bConflicting) { bIsConflicting = bConflicting; }
	
	/** Sets the visibility of the channels of the fragment */
	void SetChannelVisibility(EVisibility NewVisibility);

	/** Gets the fixture patch node this widget refers to */
	const TSharedPtr<FDMXFixturePatchNode>& GetFixturePatchNode() const { return FixturePatchNode; }

	/** Starting Channel of the Fragment */
	int32 StartingChannel = -1;

	/** Column of the Fragment */
	int32 Column = -1;

	/** Row of the Fragment */
	int32 Row = -1;

	/** Column span of the Fragment */
	int32 ColumnSpan = 1;

private:
	/** Updates the Padding of the outermost Border */
	void InitializeBorderPadding();

	/** Updates the bTopmost member from the ZOrder of the node */
	void InitializeIsTopmost();

	/** Returns the brush for the border around the widget */
	const FSlateBrush* GetBorderBrush() const;

	/** Returns the border color of the border around the widget */
	FSlateColor GetBorderColor() const;

	/** Returns the color of the channel ID texts */
	FSlateColor GetChannelIDTextColor() const;

	/** Returns the color of the Value texts */
	FSlateColor GetValueTextColor() const;

	/** If true, the patch has the highest ZOrder in its node group */
	bool bIsTopmost = false;

	/** If true this is the Head of the Fixture Patch */
	bool bIsHead = false;

	/** If true this is the Tail of the Fixture Patch */
	bool bIsTail = false;

	/** If true, the Widget displays a conflict background */
	bool bIsConflicting = false;

	/** The padding of the outermost border */
	FMargin BorderPadding;

	/** Horizontal box containing the channels of this fragment */
	TSharedPtr<SHorizontalBox> ChannelsHorizontalBox;

	/** Overlay for content */
	TSharedPtr<SOverlay> ContentOverlay;

	/** Fixture Patch Node being displayed by this widget */
	TSharedPtr<FDMXFixturePatchNode> FixturePatchNode;

	/** The group of nodes (same Universe, same Channel) this widget belongs to */
	TArray<TSharedPtr<FDMXFixturePatchNode>> FixturePatchNodeGroup;

	/** Weak DMXEditor refrence */
	TWeakPtr<FDMXEditor> DMXEditorPtr;
};
