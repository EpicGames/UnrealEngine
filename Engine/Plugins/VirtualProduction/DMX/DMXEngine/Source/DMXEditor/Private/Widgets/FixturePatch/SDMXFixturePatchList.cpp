// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXFixturePatchList.h"

#include "Algo/AnyOf.h"
#include "Algo/Copy.h"
#include "Algo/MaxElement.h"
#include "Algo/MinElement.h"
#include "Algo/Sort.h"
#include "Algo/Transform.h"
#include "Commands/DMXEditorCommands.h"
#include "DMXEditor.h"
#include "DMXEditorSettings.h"
#include "DMXEditorUtils.h"
#include "DMXFixturePatchSharedData.h"
#include "DMXRuntimeUtils.h"
#include "Exporters/Exporter.h"
#include "Factories.h"
#include "FixturePatchAutoAssignUtility.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/DMXLibrary.h"
#include "Misc/StringOutputDevice.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "TimerManager.h"
#include "UnrealExporter.h"
#include "Widgets/FixturePatch/DMXFixturePatchListItem.h"
#include "Widgets/FixturePatch/SDMXFixturePatchListRow.h"
#include "Widgets/FixturePatch/SDMXFixturePatchListToolbar.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SDMXFixturePatchList"

namespace UE::DMX::SDMXFixturePatchList::Private
{
	/** Copies a fixture patch as text to the clipboard */
	static void ClipboardCopyFixturePatches(const TArray<UDMXEntityFixturePatch*>& FixturePatches)
	{
		// Clear the mark state for saving.
		UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));

		const FExportObjectInnerContext Context;
		FStringOutputDevice Archive;

		// Export the component object(s) to text for copying
		for (UDMXEntityFixturePatch* FixturePatch : FixturePatches)
		{
			if (!FixturePatch)
			{
				continue;
			}

			// Export the entity object to the given string
			UExporter::ExportToOutputDevice(&Context, FixturePatch, nullptr, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false, GetTransientPackage());
		}

		// Copy text to clipboard
		FString ExportedText = Archive;

		// Avoid exporting the OnFixturePatchReceived Binding
		TArray<FString> ExportedTextLines;
		constexpr bool bCullEmpty = false;
		ExportedText.ParseIntoArrayLines(ExportedTextLines, bCullEmpty);
		FString ExportedTextWithoutOnFixturePatchReceivedBinding;
		for (const FString& String : ExportedTextLines)
		{
			if (String.Contains(TEXT("OnFixturePatchReceivedDMX")))
			{
				continue;
			}
			ExportedTextWithoutOnFixturePatchReceivedBinding.Append(String + LINE_TERMINATOR);
		}

		FPlatformApplicationMisc::ClipboardCopy(*ExportedTextWithoutOnFixturePatchReceivedBinding);
	}

	/** Duplicates an existing patch */
	UDMXEntityFixturePatch* DuplicatePatch(UDMXLibrary* DMXLibrary, UDMXEntityFixturePatch* FixturePatchToDuplicate)
	{
		if (!DMXLibrary || !FixturePatchToDuplicate)
		{
			return nullptr;
		}


		// Duplicate
		DMXLibrary->PreEditChange(UDMXLibrary::StaticClass()->FindPropertyByName(UDMXLibrary::GetEntitiesPropertyName()));
		UDMXEntityFixtureType* FixtureTypeOfPatchToDuplicate = FixturePatchToDuplicate->GetFixtureType();
		if (FixtureTypeOfPatchToDuplicate && FixtureTypeOfPatchToDuplicate->GetParentLibrary() != DMXLibrary)
		{
			FDMXEntityFixtureTypeConstructionParams FixtureTypeConstructionParams;
			FixtureTypeConstructionParams.DMXCategory = FixtureTypeOfPatchToDuplicate->DMXCategory;
			FixtureTypeConstructionParams.Modes = FixtureTypeOfPatchToDuplicate->Modes;
			FixtureTypeConstructionParams.ParentDMXLibrary = DMXLibrary;

			constexpr bool bMarkLibraryDirty = false;
			UDMXEntityFixtureType::CreateFixtureTypeInLibrary(FixtureTypeConstructionParams, FixtureTypeOfPatchToDuplicate->Name, bMarkLibraryDirty);
		}

		// Duplicate the Fixture Patch
		FDMXEntityFixturePatchConstructionParams ConstructionParams;
		ConstructionParams.FixtureTypeRef = FixturePatchToDuplicate->GetFixtureType();
		ConstructionParams.ActiveMode = FixturePatchToDuplicate->GetActiveModeIndex();
		ConstructionParams.UniverseID = FixturePatchToDuplicate->GetUniverseID();
		ConstructionParams.StartingAddress = FixturePatchToDuplicate->GetStartingChannel();
		ConstructionParams.DefaultTransform = FixturePatchToDuplicate->GetDefaultTransform();

		constexpr bool bMarkLibraryDirty = false;
		UDMXEntityFixturePatch* NewFixturePatch = UDMXEntityFixturePatch::CreateFixturePatchInLibrary(ConstructionParams, FixturePatchToDuplicate->Name, bMarkLibraryDirty);

		// Use the same color as the duplicated patch
		NewFixturePatch->EditorColor = FixturePatchToDuplicate->EditorColor;

		DMXLibrary->PostEditChange();

		return NewFixturePatch;
	}

	/** Text object factory for pasting DMX Fixture Patches */
	struct FDMXFixturePatchObjectTextFactory final
		: public FCustomizableTextObjectFactory
	{
		/** Constructor */
		FDMXFixturePatchObjectTextFactory(UDMXLibrary* InDMXLibrary)
			: FCustomizableTextObjectFactory(GWarn)
			, WeakDMXLibrary(InDMXLibrary)
		{}

		/** Returns true if Fixture Patches can be constructed from the Text Buffer */
		static bool CanCreate(const FString& InTextBuffer, UDMXLibrary* InDMXLibrary)
		{
			TSharedRef<FDMXFixturePatchObjectTextFactory> Factory = MakeShared<FDMXFixturePatchObjectTextFactory>(InDMXLibrary);

			// Create new objects if we're allowed to
			return Factory->CanCreateObjectsFromText(InTextBuffer);
		}

		/**
		 * Constructs a new object factory from the given text buffer. Returns the factor or nullptr if no factory can be created.
		 * An updated General Scene Description of the library needs be passed explicitly to avoid recurring update calls.
		 */
		static bool Create(const FString& InTextBuffer, UDMXLibrary* InDMXLibrary, TArray<UDMXEntityFixturePatch*>& OutNewFixturePatches)
		{
			if (!InDMXLibrary)
			{
				return false;
			}

			OutNewFixturePatches.Reset();

			const TSharedRef<FDMXFixturePatchObjectTextFactory> Factory = MakeShared < FDMXFixturePatchObjectTextFactory>(InDMXLibrary);

			// Create new objects if we're allowed to
			if (Factory->CanCreateObjectsFromText(InTextBuffer))
			{
				Factory->WeakDMXLibrary = InDMXLibrary;

				EObjectFlags ObjectFlags = RF_Transactional;
				Factory->ProcessBuffer(InDMXLibrary, ObjectFlags, InTextBuffer);

				OutNewFixturePatches = Factory->NewFixturePatches;
			}

			return true;
		}

	protected:
		//~ Begin FCustomizableTextObjectFactory implementation
		virtual bool CanCreateClass(UClass* ObjectClass, bool& bOmitSubObjs) const override
		{
			return ObjectClass->IsChildOf(UDMXEntityFixturePatch::StaticClass());
		}

		virtual void ProcessConstructedObject(UObject* NewObject) override
		{
			UDMXLibrary* DMXLibrary = WeakDMXLibrary.Get();
			UDMXEntityFixturePatch* NewFixturePatch = Cast<UDMXEntityFixturePatch>(NewObject);
			if (DMXLibrary && NewFixturePatch)
			{
				const FScopedTransaction Transaction(TransactionText);

				const bool bIsDuplicating = Algo::FindBy(DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>(), NewFixturePatch->GetMVRFixtureUUID(), &UDMXEntityFixturePatch::GetMVRFixtureUUID) != nullptr;
				if (bIsDuplicating)
				{					
					// Create a duplicate of the new patch that is properly initialized
					using namespace UE::DMX::SDMXFixturePatchList::Private;
					UDMXEntityFixturePatch* DuplicatedFixturePatch = DuplicatePatch(DMXLibrary, NewFixturePatch);
					NewFixturePatches.Add(DuplicatedFixturePatch);

					// Remove the patch created by the factory
					UDMXEntityFixturePatch::RemoveFixturePatchFromLibrary(NewFixturePatch);
				}
				else
				{
					// Simply assign the new patch to the library
					NewFixturePatch->Rename(*MakeUniqueObjectName(DMXLibrary, NewFixturePatch->GetClass()).ToString(), DMXLibrary, REN_DoNotDirty | REN_DontCreateRedirectors);
					NewFixturePatch->SetName(FDMXRuntimeUtils::FindUniqueEntityName(DMXLibrary, NewFixturePatch->GetClass(), NewFixturePatch->GetDisplayName()));
					NewFixturePatch->SetParentLibrary(DMXLibrary);
					NewFixturePatch->RefreshID();

					NewFixturePatches.Add(NewFixturePatch);
				}
			}
		}
		//~ End FCustomizableTextObjectFactory implementation

	private:
		/** Instantiated Fixture Patches */
		TArray<UDMXEntityFixturePatch*> NewFixturePatches;

		/** Transaction text displayed when pasting */
		FText TransactionText;

		/** Weak DMX Editor in which the operation should occur */
		TWeakObjectPtr<UDMXLibrary> WeakDMXLibrary;
	};
}


/** Helper to generate Status Text for MVR Fixture List Items */
class FDMXFixturePatchListStatusTextGenerator
{
public:
	FDMXFixturePatchListStatusTextGenerator(const TArray<TSharedPtr<FDMXFixturePatchListItem>>& InItems)
		: Items(InItems)
	{}

	/** Generates warning texts. Returns a map of those Items that need a warning set along with the warning Text */
	TMap<TSharedPtr<FDMXFixturePatchListItem>, FText> GenerateWarningTexts() const
	{
		TMap<TSharedPtr<FDMXFixturePatchListItem>, FText> AccumulatedConflicts;

		TMap<TSharedPtr<FDMXFixturePatchListItem>, FText> FixtureTypeIssues = GetFixtureTypeIssues();
		AppendConflictTexts(FixtureTypeIssues, AccumulatedConflicts);

		TMap<TSharedPtr<FDMXFixturePatchListItem>, FText> FixtureIDIssues = GetFixtureIDIssues();
		AppendConflictTexts(FixtureIDIssues, AccumulatedConflicts);

		TMap<TSharedPtr<FDMXFixturePatchListItem>, FText> FixtureIDConflicts = GetFixtureIDConflicts();
		AppendConflictTexts(FixtureIDConflicts, AccumulatedConflicts);

		TMap<TSharedPtr<FDMXFixturePatchListItem>, FText> ChannelExcessConflicts = GetChannelExcessConflicts();
		AppendConflictTexts(ChannelExcessConflicts, AccumulatedConflicts);

		TMap<TSharedPtr<FDMXFixturePatchListItem>, FText> ChannelOverlapConflicts = GetChannelOverlapConflicts();
		AppendConflictTexts(ChannelOverlapConflicts, AccumulatedConflicts);

		return AccumulatedConflicts;
	}

private:
	void AppendConflictTexts(const TMap<TSharedPtr<FDMXFixturePatchListItem>, FText>& InItemToConflictTextMap, TMap<TSharedPtr<FDMXFixturePatchListItem>, FText>& InOutConflictTexts) const
	{
		for (const TTuple<TSharedPtr<FDMXFixturePatchListItem>, FText>& ItemToConflictTextPair : InItemToConflictTextMap)
		{
			if (InOutConflictTexts.Contains(ItemToConflictTextPair.Key))
			{
				const FText LineTerminator = FText::FromString(LINE_TERMINATOR);
				const FText AccumulatedErrorText = FText::Format(FText::FromString(TEXT("{0}{1}{2}{3}")), InOutConflictTexts[ItemToConflictTextPair.Key], LineTerminator, LineTerminator, ItemToConflictTextPair.Value);
				InOutConflictTexts[ItemToConflictTextPair.Key] = AccumulatedErrorText;
			}
			else
			{
				InOutConflictTexts.Add(ItemToConflictTextPair);
			}
		}
	}

	/** The patch of an item. Useful to Get Conflicts with Other */
	struct FItemPatch
	{
		FItemPatch(const TSharedPtr<FDMXFixturePatchListItem>& InItem)
			: Item(InItem)
		{
			Universe = Item->GetUniverse();
			AddressRange = TRange<int32>(Item->GetAddress(), Item->GetAddress() + Item->GetNumChannels());
		}

		/** Returns a conflict text if this item conflicts with Other */
		FText GetConfictsWithOther(const FItemPatch& Other) const
		{
			// No conflict with self
			if (Other.Item == Item)
			{
				return FText::GetEmpty();
			}

			// No conflict with the same patch
			if (Item->GetFixturePatch() == Other.Item->GetFixturePatch())
			{
				return FText::GetEmpty();
			}

			// No conflict if not in the same universe
			if (Other.Universe != Universe)
			{
				return FText::GetEmpty();
			}

			// No conflict if channels don't overlap
			if (!AddressRange.Overlaps(Other.AddressRange))
			{
				return FText::GetEmpty();
			}

			// No conflict if patches are functionally equal
			if (AddressRange.GetLowerBound() == Other.AddressRange.GetLowerBound() &&
				Item->GetFixtureType() == Other.Item->GetFixtureType() &&
				Item->GetModeIndex() == Other.Item->GetModeIndex())
			{
				return FText::GetEmpty();
			}

			const FText FixtureIDText = MakeBeautifulItemText(Item);
			const FText OtherFixtureIDText = MakeBeautifulItemText(Other.Item);
			if (AddressRange.GetLowerBound() == Other.AddressRange.GetLowerBound() &&
				Item->GetFixtureType() == Other.Item->GetFixtureType())
			{
				// Modes confict
				check(Item->GetModeIndex() != Other.Item->GetModeIndex());
				return FText::Format(LOCTEXT("ModeConflict", "Uses same Address and Fixture Type as Fixture {1}, but Modes differ."), FixtureIDText, OtherFixtureIDText);
			}
			else if (AddressRange.GetLowerBound() == Other.AddressRange.GetLowerBound())
			{
				// Fixture Types conflict
				check(Item->GetFixtureType() != Other.Item->GetFixtureType());
				return FText::Format(LOCTEXT("FixtureTypeConflict", "Uses same Address as Fixture {1}, but Fixture Types differ."), FixtureIDText, OtherFixtureIDText);
			}
			else
			{
				// Addresses conflict
				return FText::Format(LOCTEXT("AddressConflict", "Overlaps Addresses with Fixture {1}"), FixtureIDText, OtherFixtureIDText);
			}
		}

		const TSharedPtr<FDMXFixturePatchListItem>& GetItem() const { return Item; };

	private:
		int32 Universe = -1;
		TRange<int32> AddressRange;

		TSharedPtr<FDMXFixturePatchListItem> Item;
	};

	/** Returns a Map of Items to Channels that have Fixture Types with issues set */
	TMap<TSharedPtr<FDMXFixturePatchListItem>, FText> GetFixtureTypeIssues() const
	{
		TMap<TSharedPtr<FDMXFixturePatchListItem>, FText> ItemToIssueMap;
		for (const TSharedPtr<FDMXFixturePatchListItem>& Item : Items)
		{
			if (!Item->GetFixtureType())
			{
				const FText IssueText = LOCTEXT("NoFixtureTypeIssue", "No Fixture Type selected.");
				ItemToIssueMap.Add(Item, IssueText);
			}
			else if (Item->GetFixtureType()->Modes.IsEmpty())
			{
				const FText IssueText = LOCTEXT("NoModesIssue", "Fixture Type has no Modes defined.");
				ItemToIssueMap.Add(Item, IssueText);
			}
			else if (Item->GetFixturePatch()->GetActiveMode() && 
				!Item->GetFixturePatch()->GetActiveMode()->bFixtureMatrixEnabled &&
				Item->GetFixturePatch()->GetActiveMode()->Functions.IsEmpty())
			{
				const FText IssueText = LOCTEXT("ActiveModeHasNoFunctionsIssue", "Mode does not define any Functions.");
				ItemToIssueMap.Add(Item, IssueText);
			}
		}

		return ItemToIssueMap;
	}

	/** Returns a Map of Items to Channels exceeding the DMX address range Texts */
	TMap<TSharedPtr<FDMXFixturePatchListItem>, FText> GetChannelExcessConflicts() const
	{
		TMap<TSharedPtr<FDMXFixturePatchListItem>, FText> ItemToConflictMap;
		for (const TSharedPtr<FDMXFixturePatchListItem>& Item : Items)
		{
			const int32 EndingAddress = Item->GetAddress() + Item->GetNumChannels() - 1;
			if (Item->GetAddress() < 1 &&
				EndingAddress > DMX_MAX_ADDRESS)
			{
				const FText ConflictText = FText::Format(LOCTEXT("ChannelExceedsMinAndMaxChannelConflict", "Exceeds available DMX Address range. Staring Address is {0} but min Address is 1. Ending Address is {1} but max Address is 512."), Item->GetAddress(), EndingAddress);
				ItemToConflictMap.Add(Item, ConflictText);
			}
			else if (Item->GetAddress() < 1)
			{
				const FText ConflictText = FText::Format(LOCTEXT("ChannelExceedsMinChannelNumberConflict", "Exceeds available DMX Address range. Staring Address is {0} but min Address is 1."), Item->GetAddress());
				ItemToConflictMap.Add(Item, ConflictText);
			}
			else if (EndingAddress > DMX_MAX_ADDRESS)
			{
				const FText ConflictText = FText::Format(LOCTEXT("ChannelExeedsMaxChannelNumberConflict", "Exceeds available DMX Address range. Ending Address is {0} but max Address is 512."), EndingAddress);
				ItemToConflictMap.Add(Item, ConflictText);
			}			
		}

		return ItemToConflictMap;
	}

	/** Returns a Map of Items to overlapping Channel conflict Texts */
	TMap<TSharedPtr<FDMXFixturePatchListItem>, FText> GetChannelOverlapConflicts() const
	{
		TArray<FItemPatch> ItemPatches;
		ItemPatches.Reserve(Items.Num());
		for (const TSharedPtr<FDMXFixturePatchListItem>& Item : Items)
		{
			FItemPatch ItemPatch(Item);
			ItemPatches.Add(ItemPatch);
		}

		TMap<TSharedPtr<FDMXFixturePatchListItem>, FText> ItemToConflictMap;
		for (const FItemPatch& ItemPatch : ItemPatches)
		{
			for (const FItemPatch& Other : ItemPatches)
			{
				const FText ConflictWithOtherText = ItemPatch.GetConfictsWithOther(Other);
				if (!ConflictWithOtherText.IsEmpty())
				{
					if (ItemToConflictMap.Contains(ItemPatch.GetItem()))
					{
						FText AppendConflictText = FText::Format(FText::FromString(TEXT("{0}{1}{2}")), ItemToConflictMap[ItemPatch.GetItem()], FText::FromString(FString(LINE_TERMINATOR)), ConflictWithOtherText);
						ItemToConflictMap[ItemPatch.GetItem()] = AppendConflictText;
					}
					else
					{
						ItemToConflictMap.Add(ItemPatch.GetItem(), ConflictWithOtherText);
					}
				}
			}
		}

		return ItemToConflictMap;
	}

	/** Returns an Map of Items to Fixture IDs issues Texts */
	TMap<TSharedPtr<FDMXFixturePatchListItem>, FText> GetFixtureIDIssues() const
	{
		TMap<TSharedPtr<FDMXFixturePatchListItem>, FText> Result;
		for (const TSharedPtr<FDMXFixturePatchListItem>& Item : Items)
		{
			int32 FixtureIDNumerical;
			if (!LexTryParseString(FixtureIDNumerical, *Item->GetFixtureID()))
			{
				Result.Add(Item, LOCTEXT("FixtureIDNotNumericalIssueText", "FID has to be a number."));
			}
		}
		return Result;
	}

	/** Returns an Map of Items to Fixture IDs conflict Texts */
	TMap<TSharedPtr<FDMXFixturePatchListItem>, FText> GetFixtureIDConflicts() const
	{
		TMap<FString, TArray<TSharedPtr<FDMXFixturePatchListItem>>> FixtureIDMap;
		FixtureIDMap.Reserve(Items.Num());
		for (const TSharedPtr<FDMXFixturePatchListItem>& Item : Items)
		{
			FixtureIDMap.FindOrAdd(Item->GetFixtureID()).Add(Item);
		}
		TArray<TArray<TSharedPtr<FDMXFixturePatchListItem>>> FixtureIDConflicts;
		FixtureIDMap.GenerateValueArray(FixtureIDConflicts);
		FixtureIDConflicts.RemoveAll([](const TArray<TSharedPtr<FDMXFixturePatchListItem>>& ConflictingItems)
			{
				return ConflictingItems.Num() < 2;
			});
		
		TMap<TSharedPtr<FDMXFixturePatchListItem>, FText> ItemToConflictMap;
		for (TArray<TSharedPtr<FDMXFixturePatchListItem>>& ConflictingItems : FixtureIDConflicts)
		{
			ConflictingItems.Sort([](const TSharedPtr<FDMXFixturePatchListItem>& ItemA, const TSharedPtr<FDMXFixturePatchListItem>& ItemB)
				{
					return ItemA->GetFixtureID() < ItemB->GetFixtureID();
				});

			check(ConflictingItems.Num() > 0);
			FText ConflictText = FText::Format(LOCTEXT("BaseFixtureIDConflictText", "Ambiguous FIDs in {0}"), MakeBeautifulItemText(ConflictingItems[0]));
			for (int32 ConflictingItemIndex = 1; ConflictingItemIndex < ConflictingItems.Num(); ConflictingItemIndex++)
			{
				ConflictText = FText::Format(LOCTEXT("AppendFixtureIDConflictText", "{0}, {1}"), ConflictText, MakeBeautifulItemText(ConflictingItems[ConflictingItemIndex]));
			}
			
			for (const TSharedPtr<FDMXFixturePatchListItem>& Item : ConflictingItems)
			{
				ItemToConflictMap.Add(Item, ConflictText);
			}
		}

		return ItemToConflictMap;
	}

	static FText MakeBeautifulItemText(const TSharedPtr<FDMXFixturePatchListItem>& Item)
	{
		const FString AddressesString = FString::FromInt(Item->GetUniverse()) + TEXT(".") + FString::FromInt(Item->GetAddress());
		const FString ItemNameString = TEXT("'") + Item->GetFixturePatchName() + TEXT("'");
		const FString BeautifulItemString = ItemNameString + TEXT(" (") + AddressesString + TEXT(")");;
		return FText::FromString(BeautifulItemString);
	}

	/** The items the class handles */
	TArray<TSharedPtr<FDMXFixturePatchListItem>> Items;
};

const FName FDMXFixturePatchListCollumnID::EditorColor = "EditorColor";
const FName FDMXFixturePatchListCollumnID::FixturePatchName = "FixturePatchName";
const FName FDMXFixturePatchListCollumnID::Status = "Status";
const FName FDMXFixturePatchListCollumnID::FixtureID = "FixtureID";
const FName FDMXFixturePatchListCollumnID::FixtureType = "FixtureType";
const FName FDMXFixturePatchListCollumnID::Mode = "Mode";
const FName FDMXFixturePatchListCollumnID::Patch = "Patch";

SDMXFixturePatchList::SDMXFixturePatchList()
	: SortMode(EColumnSortMode::Ascending)
	, SortedByColumnID(FDMXFixturePatchListCollumnID::FixtureID)
{}

SDMXFixturePatchList::~SDMXFixturePatchList()
{
	if (HeaderRow.IsValid())
	{
		UDMXEditorSettings* EditorSettings = GetMutableDefault<UDMXEditorSettings>();
		for (const SHeaderRow::FColumn& Column : HeaderRow->GetColumns())
		{
			if (Column.ColumnId == FDMXFixturePatchListCollumnID::FixtureID)
			{
				EditorSettings->MVRFixtureListSettings.FixtureIDColumnWidth = Column.Width.Get();
			}
			if (Column.ColumnId == FDMXFixturePatchListCollumnID::FixtureType)
			{
				EditorSettings->MVRFixtureListSettings.FixtureTypeColumnWidth = Column.Width.Get();
			}
			else if (Column.ColumnId == FDMXFixturePatchListCollumnID::Mode)
			{
				EditorSettings->MVRFixtureListSettings.ModeColumnWidth = Column.Width.Get();
			}
			else if (Column.ColumnId == FDMXFixturePatchListCollumnID::Patch)
			{
				EditorSettings->MVRFixtureListSettings.PatchColumnWidth = Column.Width.Get();
			}
		}

		EditorSettings->SaveConfig();
	}
}

void SDMXFixturePatchList::PostUndo(bool bSuccess)
{
	RequestListRefresh();
}

void SDMXFixturePatchList::PostRedo(bool bSuccess)
{
	RequestListRefresh();
}

void SDMXFixturePatchList::Construct(const FArguments& InArgs, TWeakPtr<FDMXEditor> InDMXEditor)
{
	if (!InDMXEditor.IsValid())
	{
		return;
	}
	
	WeakDMXEditor = InDMXEditor;
	FixturePatchSharedData = InDMXEditor.Pin()->GetFixturePatchSharedData();

	const UDMXEditorSettings* EditorSettings = GetDefault<UDMXEditorSettings>();
	SortedByColumnID = EditorSettings->MVRFixtureListSettings.SortByCollumnID;
	SortMode = static_cast<EColumnSortMode::Type>(EditorSettings->MVRFixtureListSettings.SortPriorityEnumIndex);

	// Handle Entity changes
	UDMXLibrary::GetOnEntitiesAdded().AddSP(this, &SDMXFixturePatchList::OnEntityAddedOrRemoved);
	UDMXLibrary::GetOnEntitiesRemoved().AddSP(this, &SDMXFixturePatchList::OnEntityAddedOrRemoved);
	UDMXEntityFixturePatch::GetOnFixturePatchChanged().AddSP(this, &SDMXFixturePatchList::OnFixturePatchChanged);
	UDMXEntityFixtureType::GetOnFixtureTypeChanged().AddSP(this, &SDMXFixturePatchList::OnFixtureTypeChanged);

	// Handle Shared Data selection changes
	FixturePatchSharedData->OnFixturePatchSelectionChanged.AddSP(this, &SDMXFixturePatchList::OnFixturePatchSharedDataSelectedFixturePatches);

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.AutoHeight()
		[
			SAssignNew(Toolbar, SDMXFixturePatchListToolbar, WeakDMXEditor)
			.OnSearchChanged(this, &SDMXFixturePatchList::OnSearchChanged)
		]

		+ SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.FillHeight(1.f)
		[
			SAssignNew(ListContentBorder, SBorder)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
		]
	];

	RefreshList();

	RegisterCommands();
}

void SDMXFixturePatchList::RequestListRefresh()
{
	if (!RequestListRefreshTimerHandle.IsValid())
	{	
		// If a fixture patch item is changing a fixture patch, don't refresh. Instead let the rows update themselves.
		const bool bAnyItemIsChangingFixturePatch = Algo::AnyOf(ListSource, [](const TSharedPtr<FDMXFixturePatchListItem>& Item)
			{
				return Item->IsChangingFixturePatch();
			});
		if (bAnyItemIsChangingFixturePatch)
		{
			return;
		}

		RequestListRefreshTimerHandle = GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateSP(this, &SDMXFixturePatchList::RefreshList));
	}
}

void SDMXFixturePatchList::EnterFixturePatchNameEditingMode()
{
	const TArray<TSharedPtr<FDMXFixturePatchListItem>> SelectedItems = ListView->GetSelectedItems();
	if (SelectedItems.Num() == 0)
	{
		const TSharedPtr<SDMXFixturePatchListRow>* SelectedRowPtr = Rows.FindByPredicate([&SelectedItems](const TSharedPtr<SDMXFixturePatchListRow>& Row)
			{
				return Row->GetItem() == SelectedItems[0];
			});
		if (SelectedRowPtr)
		{
			(*SelectedRowPtr)->EnterFixturePatchNameEditingMode();
		}
	}
}	

FReply SDMXFixturePatchList::ProcessCommandBindings(const FKeyEvent& InKeyEvent)
{
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SDMXFixturePatchList::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	return ProcessCommandBindings(InKeyEvent);
}

void SDMXFixturePatchList::OnSearchChanged()
{
	RequestListRefresh();
}

void SDMXFixturePatchList::RefreshList()
{
	RequestListRefreshTimerHandle.Invalidate();

	const TSharedPtr<FDMXEditor> DMXEditor = WeakDMXEditor.Pin();
	UDMXLibrary* DMXLibrary = DMXEditor.IsValid() ? DMXEditor->GetDMXLibrary() : nullptr;
	if (!DMXLibrary)
	{
		ChildSlot
		[
			SNullWidget::NullWidget
		];

		return;
	}

	// Clear cached data
	Rows.Reset();
	ListSource.Reset();

	// Make a new list source
	const TArray<UDMXEntityFixturePatch*> FixturePatches = DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>();
	Algo::TransformIf(FixturePatches, ListSource,
		[](const UDMXEntityFixturePatch* FixturePatch)
		{
			return FixturePatch != nullptr;
		},
		[&DMXEditor](UDMXEntityFixturePatch* FixturePatch)
		{
			return MakeShared<FDMXFixturePatchListItem>(DMXEditor.ToSharedRef(), FixturePatch);
		});
	SortListSource(EColumnSortPriority::Max, SortedByColumnID, SortMode);

	// Apply search filters. Relies on up-to-date status to find conflicts.
	ListSource = Toolbar->FilterItems(ListSource);

	// Generate status texts
	GenereateStatusText();

	ListContentBorder->SetContent(
		SAssignNew(ListView, SDMXFixturePatchListType)
		.ListViewStyle(&FAppStyle::Get().GetWidgetStyle<FTableViewStyle>("TreeView"))
		.HeaderRow(GenerateHeaderRow())
		.ListItemsSource(&ListSource)
		.OnGenerateRow(this, &SDMXFixturePatchList::OnGenerateRow)
		.OnSelectionChanged(this, &SDMXFixturePatchList::OnSelectionChanged)
		.OnContextMenuOpening(this, &SDMXFixturePatchList::OnContextMenuOpening));

	AdoptSelectionFromFixturePatchSharedData();
}

void SDMXFixturePatchList::GenereateStatusText()
{
	for (const TSharedPtr<FDMXFixturePatchListItem>& Item : ListSource)
	{
		Item->WarningStatusText = FText::GetEmpty();
		Item->ErrorStatusText = FText::GetEmpty();
	}

	FDMXFixturePatchListStatusTextGenerator StatusTextGenerator(ListSource);

	const TMap<TSharedPtr<FDMXFixturePatchListItem>, FText> WarningTextMap = StatusTextGenerator.GenerateWarningTexts();
	for (const TTuple<TSharedPtr<FDMXFixturePatchListItem>, FText>& ItemToWarningTextPair : WarningTextMap)
	{
		ItemToWarningTextPair.Key->WarningStatusText = ItemToWarningTextPair.Value;
	}
}

TSharedRef<ITableRow> SDMXFixturePatchList::OnGenerateRow(TSharedPtr<FDMXFixturePatchListItem> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	const TSharedRef<SDMXFixturePatchListRow> NewRow =
		SNew(SDMXFixturePatchListRow, OwnerTable, InItem.ToSharedRef())
		.OnRowRequestsListRefresh(this, &SDMXFixturePatchList::RequestListRefresh)
		.OnRowRequestsStatusRefresh(this, &SDMXFixturePatchList::GenereateStatusText)
		.IsSelected_Lambda([this, InItem]()
			{
				return ListView->IsItemSelected(InItem);
			});

	Rows.Add(NewRow);

	return NewRow;
}

void SDMXFixturePatchList::OnSelectionChanged(TSharedPtr<FDMXFixturePatchListItem> InItem, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo == ESelectInfo::Direct)
	{
		return;
	}

	LastSelectedItem = InItem;
 
	const TArray<TSharedPtr<FDMXFixturePatchListItem>> SelectedItems = ListView->GetSelectedItems();
	TArray<TWeakObjectPtr<UDMXEntityFixturePatch>> FixturePatchesToSelect;
	for (const TSharedPtr<FDMXFixturePatchListItem>& Item : SelectedItems)
	{
		if (UDMXEntityFixturePatch* FixturePatch = Item->GetFixturePatch())
		{
			FixturePatchesToSelect.AddUnique(FixturePatch);
		}
	}

	FixturePatchSharedData->SelectFixturePatches(FixturePatchesToSelect);

	if (FixturePatchesToSelect.Num() > 0)
	{
		const int32 SelectedUniverse = FixturePatchSharedData->GetSelectedUniverse();
		const int32 UniverseOfFirstItem = FixturePatchesToSelect[0]->GetUniverseID();
		if (SelectedUniverse != UniverseOfFirstItem)
		{
			FixturePatchSharedData->SelectUniverse(UniverseOfFirstItem);
		}
	}
}

void SDMXFixturePatchList::OnEntityAddedOrRemoved(UDMXLibrary* DMXLibrary, TArray<UDMXEntity*> Entities)
{
	RequestListRefresh();
}

void SDMXFixturePatchList::OnFixturePatchChanged(const UDMXEntityFixturePatch* FixturePatch)
{
	// Refresh only if the fixture patch is in the library this editor handles
	const UDMXLibrary* DMXLibrary = WeakDMXEditor.IsValid() ? WeakDMXEditor.Pin()->GetDMXLibrary() : nullptr;
	if (FixturePatch && FixturePatch->GetParentLibrary() == DMXLibrary)
	{
		RequestListRefresh();
	}
}

void SDMXFixturePatchList::OnFixtureTypeChanged(const UDMXEntityFixtureType* FixtureType)
{
	// Refresh only if the fixture type is in the library this editor handles
	const UDMXLibrary* DMXLibrary = WeakDMXEditor.IsValid() ? WeakDMXEditor.Pin()->GetDMXLibrary() : nullptr;
	if (FixtureType && FixtureType->GetParentLibrary() == DMXLibrary)
	{
		RequestListRefresh();
	}
}

void SDMXFixturePatchList::OnFixturePatchSharedDataSelectedFixturePatches()
{
	TArray<TWeakObjectPtr<UDMXEntityFixturePatch>> SelectedFixturePatches = FixturePatchSharedData->GetSelectedFixturePatches();
	SelectedFixturePatches.RemoveAll([](TWeakObjectPtr<UDMXEntityFixturePatch> FixturePatch)
		{
			return !FixturePatch.IsValid();
		});

	TArray<TSharedPtr<FDMXFixturePatchListItem>> NewSelection;
	for (const TSharedPtr<FDMXFixturePatchListItem>& Item : ListSource)
	{
		if (SelectedFixturePatches.Contains(Item->GetFixturePatch()))
		{
			NewSelection.Add(Item);
		}
	}

	if (NewSelection.Num() > 0)
	{
		ListView->ClearSelection();
		ListView->SetItemSelection(NewSelection, true, ESelectInfo::OnMouseClick);
	}
	else
	{
		ListView->ClearSelection();
	}
}

void SDMXFixturePatchList::AdoptSelectionFromFixturePatchSharedData()
{
	const TArray<TWeakObjectPtr<UDMXEntityFixturePatch>> SelectedFixturePatches = FixturePatchSharedData->GetSelectedFixturePatches();

	TArray<TSharedPtr<FDMXFixturePatchListItem>> NewSelection;
	for (const TWeakObjectPtr<UDMXEntityFixturePatch> SelectedFixturePatch : SelectedFixturePatches)
	{
		const TSharedPtr<FDMXFixturePatchListItem>* SelectedItemPtr = ListSource.FindByPredicate([SelectedFixturePatch](const TSharedPtr<FDMXFixturePatchListItem>& Item)
			{
				return SelectedFixturePatch.IsValid() && Item->GetFixturePatch() == SelectedFixturePatch;
			});

		if (SelectedItemPtr)
		{
			NewSelection.Add(*SelectedItemPtr);
		}
	}

	if (NewSelection.Num() > 0)
	{
		ListView->ClearSelection();

		constexpr bool bSelected = true;
		ListView->SetItemSelection(NewSelection, bSelected, ESelectInfo::OnMouseClick);
		ListView->RequestScrollIntoView(NewSelection[0]);
	}
	else if (ListView->GetSelectedItems().IsEmpty() && !ListSource.IsEmpty())
	{	
		// Make an initial selection if nothing was selected from Fixture Patch Shared Data, as if the user clicked it
		ListView->SetSelection(ListSource[0], ESelectInfo::OnMouseClick);
	}
}

void SDMXFixturePatchList::AutoAssignFixturePatches(UE::DMXEditor::AutoAssign::EAutoAssignMode Mode)
{
	if (!WeakDMXEditor.IsValid())
	{
		return;
	}

	TArray<UDMXEntityFixturePatch*> FixturePatchesToAutoAssign;
	const TArray<TWeakObjectPtr<UDMXEntityFixturePatch>> SelectedFixturePatches = FixturePatchSharedData->GetSelectedFixturePatches();
	for (TWeakObjectPtr<UDMXEntityFixturePatch> WeakFixturePatch : SelectedFixturePatches)
	{
		if (UDMXEntityFixturePatch* FixturePatch = WeakFixturePatch.Get())
		{
			FixturePatch->PreEditChange(UDMXEntityFixturePatch::StaticClass()->FindPropertyByName(UDMXEntityFixturePatch::GetUniverseIDPropertyNameChecked()));
			FixturePatch->PreEditChange(UDMXEntityFixturePatch::StaticClass()->FindPropertyByName(UDMXEntityFixturePatch::GetStartingChannelPropertyNameChecked()));
			FixturePatchesToAutoAssign.Add(FixturePatch);
		}
	}

	if (FixturePatchesToAutoAssign.IsEmpty())
	{
		return;
	}

	using namespace UE::DMXEditor::AutoAssign;
	
	const int32 DesiredUniverse = 1;
	const int32 DesiredChannel = 1;
	FAutoAssignUtility::AutoAssign(Mode, WeakDMXEditor.Pin().ToSharedRef(), FixturePatchesToAutoAssign, DesiredUniverse, DesiredChannel);
	FixturePatchSharedData->SelectUniverse(FixturePatchesToAutoAssign[0]->GetUniverseID());

	for (UDMXEntityFixturePatch* FixturePatch : FixturePatchesToAutoAssign)
	{
		// Post edit change for both properties
		FixturePatch->PostEditChange();
		FixturePatch->PostEditChange();
	}

	RequestListRefresh();
}

bool SDMXFixturePatchList::DoesDMXLibraryHaveReachableUniverses() const
{
	UDMXLibrary* DMXLibrary = WeakDMXEditor.IsValid() ? WeakDMXEditor.Pin()->GetDMXLibrary() : nullptr;
	if (DMXLibrary)
	{
		return !DMXLibrary->GetInputPorts().IsEmpty() && !DMXLibrary->GetOutputPorts().IsEmpty();
	}
	return false;
}

void SDMXFixturePatchList::SetKeyboardFocus()
{
	FSlateApplication::Get().SetKeyboardFocus(AsShared());
}

TSharedRef<SHeaderRow> SDMXFixturePatchList::GenerateHeaderRow()
{
	const UDMXEditorSettings* EditorSettings = GetDefault<UDMXEditorSettings>();

	const float StatusColumnWidth = FMath::Max(FAppStyle::GetBrush("Icons.Warning")->GetImageSize().X + 6.f, FAppStyle::GetBrush("Icons.Error")->GetImageSize().X + 6.f);
	const float PatchColumnWidth = EditorSettings->MVRFixtureListSettings.PatchColumnWidth > .02f ? EditorSettings->MVRFixtureListSettings.PatchColumnWidth : .1f;
	const float EditorColorColumnWidth = StatusColumnWidth;
	const float FixtureIDColumnWidth = EditorSettings->MVRFixtureListSettings.FixtureIDColumnWidth > .01f ? EditorSettings->MVRFixtureListSettings.FixtureIDColumnWidth : .1f;
	const float FixtureTypeColumnWidth = EditorSettings->MVRFixtureListSettings.FixtureTypeColumnWidth > .02f ? EditorSettings->MVRFixtureListSettings.FixtureTypeColumnWidth : .1f;
	const float ModeColumnWidth = EditorSettings->MVRFixtureListSettings.ModeColumnWidth > .02f ? EditorSettings->MVRFixtureListSettings.ModeColumnWidth : .1f;

	HeaderRow = SNew(SHeaderRow);

	HeaderRow->AddColumn(
		SHeaderRow::FColumn::FArguments()
		.ColumnId(FDMXFixturePatchListCollumnID::EditorColor)
		.DefaultLabel(FText())
		.FixedWidth(EditorColorColumnWidth)
	);

	HeaderRow->AddColumn(
		SHeaderRow::FColumn::FArguments()
		.ColumnId(FDMXFixturePatchListCollumnID::FixturePatchName)
		.SortMode(this, &SDMXFixturePatchList::GetColumnSortMode, FDMXFixturePatchListCollumnID::FixturePatchName)
		.OnSort(this, &SDMXFixturePatchList::SortList)
		.DefaultLabel(LOCTEXT("FixturePatchNameColumnLabel", "Fixture Patch"))
		.FillWidth(PatchColumnWidth)
	);

	HeaderRow->AddColumn(
		SHeaderRow::FColumn::FArguments()
		.ColumnId(FDMXFixturePatchListCollumnID::Status)
		.DefaultLabel(FText())
		.FixedWidth(StatusColumnWidth)
	);

	HeaderRow->AddColumn(
		SHeaderRow::FColumn::FArguments()
		.ColumnId(FDMXFixturePatchListCollumnID::FixtureID)
		.SortMode(this, &SDMXFixturePatchList::GetColumnSortMode, FDMXFixturePatchListCollumnID::FixtureID)
		.OnSort(this, &SDMXFixturePatchList::SortList)
		.DefaultLabel(LOCTEXT("FixtureIDColumnLabel", "FID"))
		.FillWidth(FixtureIDColumnWidth)
	);

	HeaderRow->AddColumn(
		SHeaderRow::FColumn::FArguments()
		.ColumnId(FDMXFixturePatchListCollumnID::FixtureType)
		.SortMode(this, &SDMXFixturePatchList::GetColumnSortMode, FDMXFixturePatchListCollumnID::FixtureType)
		.OnSort(this, &SDMXFixturePatchList::SortList)
		.DefaultLabel(LOCTEXT("FixtureTypeColumnLabel", "FixtureType"))
		.FillWidth(FixtureTypeColumnWidth)
	);

	HeaderRow->AddColumn(
		SHeaderRow::FColumn::FArguments()
		.ColumnId(FDMXFixturePatchListCollumnID::Mode)
		.SortMode(this, &SDMXFixturePatchList::GetColumnSortMode, FDMXFixturePatchListCollumnID::Mode)
		.OnSort(this, &SDMXFixturePatchList::SortList)
		.DefaultLabel(LOCTEXT("ModeColumnLabel", "Mode"))
		.FillWidth(ModeColumnWidth)
	);

	HeaderRow->AddColumn(
		SHeaderRow::FColumn::FArguments()
		.ColumnId(FDMXFixturePatchListCollumnID::Patch)
		.SortMode(this, &SDMXFixturePatchList::GetColumnSortMode, FDMXFixturePatchListCollumnID::Patch)
		.OnSort(this, &SDMXFixturePatchList::SortList)
		.DefaultLabel(LOCTEXT("PatchColumnLabel", "Patch"))
		.FillWidth(0.1f)
	);

	return HeaderRow.ToSharedRef();
}

EColumnSortMode::Type SDMXFixturePatchList::GetColumnSortMode(const FName ColumnId) const
{
	return SortMode;
}

void SDMXFixturePatchList::SortListSource(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	SortedByColumnID = ColumnId;
	SortMode = InSortMode;

	const bool bAscending = InSortMode == EColumnSortMode::Ascending ? true : false;
	if (ColumnId == FDMXFixturePatchListCollumnID::FixturePatchName)
	{
		Algo::StableSort(ListSource, [bAscending](const TSharedPtr<FDMXFixturePatchListItem>& ItemA, const TSharedPtr<FDMXFixturePatchListItem>& ItemB)
			{
				const FString FixturePatchNameA = ItemA->GetFixturePatchName();
				const FString FixturePatchNameB = ItemB->GetFixturePatchName();

				const bool bIsGreater = FixturePatchNameA >= FixturePatchNameB;
				return bAscending ? !bIsGreater : bIsGreater;
			});
	}
	else if (ColumnId == FDMXFixturePatchListCollumnID::FixtureID)
	{
		Algo::StableSort(ListSource, [bAscending](const TSharedPtr<FDMXFixturePatchListItem>& ItemA, const TSharedPtr<FDMXFixturePatchListItem>& ItemB)
			{
				bool bIsGreater = [ItemA, ItemB]()
				{
					const FString FixtureIDStringA = ItemA->GetFixtureID();
					const FString FixtureIDStringB = ItemB->GetFixtureID();

					int32 FixtureIDA = 0;
					int32 FixtureIDB = 0;

					const bool bCanParseA = LexTryParseString(FixtureIDA, *FixtureIDStringA);
					const bool bCanParseB = LexTryParseString(FixtureIDB, *FixtureIDStringB);
					
					const bool bIsNumeric = bCanParseA && bCanParseB;
					if (bIsNumeric)
					{
						return FixtureIDA >= FixtureIDB;
					}
					else
					{
						return FixtureIDStringA >= FixtureIDStringB;
					}
				}();

				return bAscending ? !bIsGreater : bIsGreater;
			});
	}
	else if (ColumnId == FDMXFixturePatchListCollumnID::FixtureType)
	{
		Algo::StableSort(ListSource, [bAscending](const TSharedPtr<FDMXFixturePatchListItem>& ItemA, const TSharedPtr<FDMXFixturePatchListItem>& ItemB)
			{
				const FString FixtureTypeA = ItemA->GetFixtureType()->Name;
				const FString FixtureTypeB = ItemB->GetFixtureType()->Name;

				const bool bIsGreater = FixtureTypeA >= FixtureTypeB;
				return bAscending ? !bIsGreater : bIsGreater;
			});
	}
	else if (ColumnId == FDMXFixturePatchListCollumnID::Mode)
	{
		Algo::StableSort(ListSource, [bAscending](const TSharedPtr<FDMXFixturePatchListItem>& ItemA, const TSharedPtr<FDMXFixturePatchListItem>& ItemB)
			{
				const bool bIsGreater = ItemA->GetModeIndex() >= ItemB->GetModeIndex();
				return bAscending ? !bIsGreater : bIsGreater;
			});
	}
	else if (ColumnId == FDMXFixturePatchListCollumnID::Patch)
	{
		Algo::StableSort(ListSource, [bAscending](const TSharedPtr<FDMXFixturePatchListItem>& ItemA, const TSharedPtr<FDMXFixturePatchListItem>& ItemB)
			{
				const UDMXEntityFixturePatch* FixturePatchA = ItemA->GetFixturePatch();
				const UDMXEntityFixturePatch* FixturePatchB = ItemB->GetFixturePatch();

				const bool bIsUniverseIDGreater = ItemA->GetUniverse() > ItemB->GetUniverse();
				const bool bIsSameUniverse = ItemA->GetUniverse() == ItemB->GetUniverse();
				const bool bAreAddressesGreater = ItemA->GetAddress() > ItemB->GetAddress();

				const bool bIsGreater = bIsUniverseIDGreater || (bIsSameUniverse && bAreAddressesGreater);
				return bAscending ? !bIsGreater : bIsGreater;
			});
	}
}

void SDMXFixturePatchList::SortList(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	SortListSource(SortPriority, ColumnId, InSortMode);

	ListView->RequestListRefresh();

	UDMXEditorSettings* EditorSettings = GetMutableDefault<UDMXEditorSettings>();
	EditorSettings->MVRFixtureListSettings.SortByCollumnID = SortedByColumnID;
	EditorSettings->MVRFixtureListSettings.SortPriorityEnumIndex = static_cast<int32>(SortMode);

	EditorSettings->SaveConfig();
}

TSharedPtr<SWidget> SDMXFixturePatchList::OnContextMenuOpening()
{
	const bool bCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bCloseWindowAfterMenuSelection, CommandList);
	MenuBuilder.BeginSection("BasicOperationsSection", LOCTEXT("BasicOperationsSection", "Basic Operations"));
	{
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
	}
	MenuBuilder.EndSection();

	using namespace UE::DMXEditor::AutoAssign;
	MenuBuilder.BeginSection("AutoAssignSection", LOCTEXT("AutoAssignActionsSection", "Auto-Assign"));
	{
		MenuBuilder.AddMenuEntry(FDMXEditorCommands::Get().AutoAssignSelectedUniverse);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SDMXFixturePatchList::RegisterCommands()
{
	if (CommandList.IsValid())
	{
		return;
	}

	CommandList = MakeShared<FUICommandList>();
	CommandList->MapAction(FGenericCommands::Get().Cut, 
		FUIAction(
			FExecuteAction::CreateSP(this, &SDMXFixturePatchList::OnCutSelectedItems),
			FCanExecuteAction::CreateSP(this, &SDMXFixturePatchList::CanCutItems)
		)
	);
	CommandList->MapAction(FGenericCommands::Get().Copy,
		FUIAction(
			FExecuteAction::CreateSP(this, &SDMXFixturePatchList::OnCopySelectedItems),
			FCanExecuteAction::CreateSP(this, &SDMXFixturePatchList::CanCopyItems)
		)
	);
	CommandList->MapAction(FGenericCommands::Get().Paste,
		FUIAction(
			FExecuteAction::CreateSP(this, &SDMXFixturePatchList::OnPasteItems),
			FCanExecuteAction::CreateSP(this, &SDMXFixturePatchList::CanPasteItems)
		)
	);
	CommandList->MapAction(FGenericCommands::Get().Duplicate,
		FUIAction(
			FExecuteAction::CreateSP(this, &SDMXFixturePatchList::OnDuplicateItems),
			FCanExecuteAction::CreateSP(this, &SDMXFixturePatchList::CanDuplicateItems)
		)
	);
	CommandList->MapAction(FGenericCommands::Get().Delete,
		FUIAction(
			FExecuteAction::CreateSP(this, &SDMXFixturePatchList::OnDeleteItems),
			FCanExecuteAction::CreateSP(this, &SDMXFixturePatchList::CanDeleteItems),
			EUIActionRepeatMode::RepeatEnabled
		)
	);

	using namespace UE::DMXEditor::AutoAssign;
	CommandList->MapAction
	(
		FDMXEditorCommands::Get().AutoAssignSelectedUniverse,
		FExecuteAction::CreateSP(this, &SDMXFixturePatchList::AutoAssignFixturePatches, EAutoAssignMode::SelectedUniverse)
	);
}

bool SDMXFixturePatchList::CanCutItems() const
{
	return CanCopyItems() && CanDeleteItems() && !GIsTransacting;
}

void SDMXFixturePatchList::OnCutSelectedItems()
{
	const TArray<TSharedPtr<FDMXFixturePatchListItem>> SelectedItems = ListView->GetSelectedItems();

	const FScopedTransaction Transaction(SelectedItems.Num() > 1 ? LOCTEXT("CutFixturePatches", "Cut Fixtures") : LOCTEXT("CutFixturePatche", "Cut Fixture"));

	OnCopySelectedItems();
	OnDeleteItems();
}

bool SDMXFixturePatchList::CanCopyItems() const
{
	return FixturePatchSharedData->GetSelectedFixturePatches().Num() > 0 && !GIsTransacting;
}

void SDMXFixturePatchList::OnCopySelectedItems()
{
	const TArray<TSharedPtr<FDMXFixturePatchListItem>> SelectedItems = ListView->GetSelectedItems();
	TArray<UDMXEntityFixturePatch*> FixturePatchesToCopy;
	for (const TSharedPtr<FDMXFixturePatchListItem>& Item : SelectedItems)
	{
		FixturePatchesToCopy.Add(Item->GetFixturePatch());
	}

	using namespace UE::DMX::SDMXFixturePatchList::Private;
	ClipboardCopyFixturePatches(FixturePatchesToCopy);
}

bool SDMXFixturePatchList::CanPasteItems() const
{
	using namespace UE::DMX::SDMXFixturePatchList::Private;

	UDMXLibrary* DMXLibrary = WeakDMXEditor.IsValid() ? WeakDMXEditor.Pin()->GetDMXLibrary() : nullptr;
	if (!DMXLibrary)
	{
		return false;
	}

	// Get the text from the clipboard
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);

	return FDMXFixturePatchObjectTextFactory::CanCreate(TextToImport, DMXLibrary) && !GIsTransacting;
}

void SDMXFixturePatchList::OnPasteItems()
{
	using namespace UE::DMX::SDMXFixturePatchList::Private;

	const TSharedPtr<FDMXEditor> DMXEditor = WeakDMXEditor.Pin();
	UDMXLibrary* DMXLibrary = DMXEditor.IsValid() ? DMXEditor->GetDMXLibrary() : nullptr;
	if (!DMXEditor.IsValid() || !DMXLibrary)
	{
		return;
	}

	const FText TransactionText = LOCTEXT("PasteFixturePatchesTransaction", "Paste Fixture Patches");
	const FScopedTransaction PasteTransaction(TransactionText);

	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);

	TArray<UDMXEntityFixturePatch*> PastedFixturePatches;
	if(FDMXFixturePatchObjectTextFactory::Create(TextToImport, DMXLibrary, PastedFixturePatches))
	{
		TArray<TWeakObjectPtr<UDMXEntityFixturePatch>> WeakPastedFixturePatches;
		for (UDMXEntityFixturePatch* FixturePatch : PastedFixturePatches)
		{
			WeakPastedFixturePatches.Add(FixturePatch);
		}

		// Assign
		using namespace UE::DMXEditor::AutoAssign;
		int32 AssignedToUniverse = FAutoAssignUtility::AutoAssign(EAutoAssignMode::SelectedUniverse, DMXEditor.ToSharedRef(), PastedFixturePatches);

		FixturePatchSharedData->SelectUniverse(AssignedToUniverse);
		FixturePatchSharedData->SelectFixturePatches(WeakPastedFixturePatches);
	}
}

bool SDMXFixturePatchList::CanDuplicateItems() const
{
	return FixturePatchSharedData->GetSelectedFixturePatches().Num() > 0 && !GIsTransacting;
}

void SDMXFixturePatchList::OnDuplicateItems()
{
	const TSharedPtr<FDMXEditor> DMXEditor = WeakDMXEditor.Pin();
	UDMXLibrary* DMXLibrary = DMXEditor.IsValid() ? DMXEditor->GetDMXLibrary() : nullptr;
	if (!DMXEditor.IsValid() || !DMXLibrary)
	{
		return;
	}

	const FText TransactionText = LOCTEXT("DuplicateFixturePatchesTransaction", "Duplicate Fixture Patches");
	const FScopedTransaction PasteTransaction(TransactionText);
	DMXLibrary->PreEditChange(nullptr);

	TArray<TWeakObjectPtr<UDMXEntityFixturePatch>> SelectedWeakFixturePatches = FixturePatchSharedData->GetSelectedFixturePatches();
	SelectedWeakFixturePatches.Remove(nullptr);
	if (SelectedWeakFixturePatches.IsEmpty())
	{
		return;
	}

	TArray<UDMXEntityFixturePatch*> SelectedFixturePatches;
	Algo::TransformIf(SelectedWeakFixturePatches, SelectedFixturePatches, 
		[](const TWeakObjectPtr<UDMXEntityFixturePatch>& WeakFixturePatch) { return WeakFixturePatch.IsValid(); },
		[](const TWeakObjectPtr<UDMXEntityFixturePatch>& WeakFixturePatch) { return WeakFixturePatch.Get(); });

	// Duplicate in order of patch
	Algo::StableSortBy(SelectedFixturePatches, [this](UDMXEntityFixturePatch* FixturePatch)
		{
			return FixturePatch->GetUniverseID() * DMX_UNIVERSE_SIZE + FixturePatch->GetStartingChannel();
		});

	TArray<UDMXEntityFixturePatch*> NewFixturePatches;
	for (UDMXEntityFixturePatch* FixturePatch : SelectedFixturePatches)
	{
		if(!FixturePatch->GetParentLibrary())
		{
			continue;
		}

		using namespace UE::DMX::SDMXFixturePatchList::Private;
		UDMXEntityFixturePatch* NewFixturePatch = DuplicatePatch(DMXLibrary, FixturePatch);
		NewFixturePatches.Add(NewFixturePatch);
	}

	using namespace UE::DMXEditor::AutoAssign;
	const int32 AssignedToUniverse = FAutoAssignUtility::AutoAssign(EAutoAssignMode::AfterLastPatchedUniverse, DMXEditor.ToSharedRef(), NewFixturePatches);

	DMXLibrary->PostEditChange();

	TArray<TWeakObjectPtr<UDMXEntityFixturePatch>> NewWeakFixturePatches;
	Algo::Copy(NewFixturePatches, NewWeakFixturePatches);
	FixturePatchSharedData->SelectFixturePatches(NewWeakFixturePatches);
	FixturePatchSharedData->SelectUniverse(AssignedToUniverse);
}

bool SDMXFixturePatchList::CanDeleteItems() const
{
	return FixturePatchSharedData->GetSelectedFixturePatches().Num() > 0 && !GIsTransacting;
}

void SDMXFixturePatchList::OnDeleteItems()
{
	const TArray<TSharedPtr<FDMXFixturePatchListItem>> SelectedItems = ListView->GetSelectedItems();

	if (SelectedItems.Num() == 0)
	{
		return;
	}
	
	UDMXLibrary* DMXLibrary = SelectedItems[0]->GetDMXLibrary();
	if (!DMXLibrary)
	{
		return;
	}
	
	const FText DeleteFixturePatchesTransactionText = FText::Format(LOCTEXT("DeleteFixturePatchesTransaction", "Delete Fixture {0}|plural(one=Patch, other=Patches)"), SelectedItems.Num() > 1);
	const FScopedTransaction DeleteFixturePatchTransaction(DeleteFixturePatchesTransactionText);

	DMXLibrary->PreEditChange(nullptr);
	for (const TSharedPtr<FDMXFixturePatchListItem>& Item : SelectedItems)
	{
		if (UDMXEntityFixturePatch* FixturePatch = Item->GetFixturePatch())
		{
			UDMXEntityFixturePatch::RemoveFixturePatchFromLibrary(FixturePatch);
		}
	}
	DMXLibrary->PostEditChange();

	// Make a meaningful selection invariant to ordering of the List
	TSharedPtr<FDMXFixturePatchListItem> NewSelection;
	for (int32 ItemIndex = 0; ItemIndex < ListSource.Num(); ItemIndex++)
	{
		if (SelectedItems.Contains(ListSource[ItemIndex]))
		{
			if (ListSource.IsValidIndex(ItemIndex + 1) && !SelectedItems.Contains(ListSource[ItemIndex + 1]))
			{
				NewSelection = ListSource[ItemIndex + 1];
				break;
			}
			else if (ListSource.IsValidIndex(ItemIndex - 1) && !SelectedItems.Contains(ListSource[ItemIndex - 1]))
			{
				NewSelection = ListSource[ItemIndex - 1];
				break;
			}
		}
	}
	if (NewSelection.IsValid())
	{
		ListView->SetSelection(NewSelection, ESelectInfo::OnMouseClick);
	}
}

#undef LOCTEXT_NAMESPACE
