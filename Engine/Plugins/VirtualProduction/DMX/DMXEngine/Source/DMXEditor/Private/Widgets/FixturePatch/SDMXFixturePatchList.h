// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "Engine/EngineTypes.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SHeaderRow.h"

class FDMXEditor;
class FDMXFixturePatchListItem;
class FDMXFixturePatchSharedData;
class FUICommandList;
class ITableRow;
class SBorder;
class SDMXFixturePatchListRow;
class SDMXFixturePatchListToolbar;
class SHeaderRow;
template <typename ItemType> class SListView;
class STableViewBase;
class UDMXEntity;
class UDMXEntityFixturePatch;
class UDMXEntityFixtureType;
class UDMXLibrary;
class UDMXMVRGeneralSceneDescription;
namespace UE::DMXEditor::AutoAssign { enum class EAutoAssignMode : uint8; }

/** Collumn IDs in the Fixture Patch List */
struct FDMXFixturePatchListCollumnID
{
	static const FName EditorColor;
	static const FName FixturePatchName;
	static const FName Status;
	static const FName FixtureID;
	static const FName FixtureType;
	static const FName Mode;
	static const FName Patch;
};

using SDMXFixturePatchListType = SListView<TSharedPtr<FDMXFixturePatchListItem>>;

/** Sortable, editable List of Fixture Patches in the library */
class SDMXFixturePatchList
	: public SCompoundWidget
	, public FSelfRegisteringEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(SDMXFixturePatchList)
	{}

	SLATE_END_ARGS()

	SDMXFixturePatchList();
	virtual ~SDMXFixturePatchList();

	//~Begin EditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	//~End EditorUndoClient interface

	/** Constructs this widget */
	void Construct(const FArguments& InArgs, TWeakPtr<FDMXEditor> InDMXEditor);

	/** Updates the List on the next tick */
	void RequestListRefresh();

	/** If a single Row is selected, enters Editing Mode for the Fixture Patch Name of the selected Row */
	void EnterFixturePatchNameEditingMode();

	/** Processes command bindings for the Key Event */
	FReply ProcessCommandBindings(const FKeyEvent& InKeyEvent);

protected:
	//~ Begin SWidget Interface
	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget Interface

private:
	/** Called when the search changed */
	void OnSearchChanged();

	/** Refreshes the List */
	void RefreshList();

	/** Generates Status Text for each Item and sets it */
	void GenereateStatusText();

	/** Called when a row in the List gets generated */
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FDMXFixturePatchListItem> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Called when the Selection in the list changed */
	void OnSelectionChanged(TSharedPtr<FDMXFixturePatchListItem> InItem, ESelectInfo::Type SelectInfo);

	/** Called when entities were added or removed from the DMX Library */
	void OnEntityAddedOrRemoved(UDMXLibrary* DMXLibrary, TArray<UDMXEntity*> Entities);

	/** Called when a Fixture Patch changed */
	void OnFixturePatchChanged(const UDMXEntityFixturePatch* FixturePatch);

	/** Called when a Fixture Type changed */
	void OnFixtureTypeChanged(const UDMXEntityFixtureType* FixtureType);

	/** Called when Fixture Patch Shared Data selected Fixture Patches */
	void OnFixturePatchSharedDataSelectedFixturePatches();

	/** Selects what is selected in Fixture Patch Shared Data */
	void AdoptSelectionFromFixturePatchSharedData();

	/** Auto assigns selected Fixture Patches */
	void AutoAssignFixturePatches(UE::DMXEditor::AutoAssign::EAutoAssignMode Mode);

	/** Returns true if the DMX Library has reachable Universes */
	bool DoesDMXLibraryHaveReachableUniverses() const;

	/** Sets keyboard focus on this widget */
	void SetKeyboardFocus();

	/** Generates the Header Row of the List */
	TSharedRef<SHeaderRow> GenerateHeaderRow();

	/** Returns the column sort mode for the list*/
	EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const;

	/** Sorts the list source */
	void SortListSource(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);

	/** Sort the list */
	void SortList(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);

	/** The current Sort Mode */
	EColumnSortMode::Type SortMode;

	/** By which column ID the List is sorted */
	FName SortedByColumnID;

	/** Set to true while changing the DMX Library and only redraw the list when this is false, e.g. for Duplicate, Paste etc. */
	bool bChangingDMXLibrary = false;

	/** Source array for the Fixture Patch List */
	TArray<TSharedPtr<FDMXFixturePatchListItem>> ListSource;

	/** List source when filtered by search */
	TArray<TSharedPtr<FDMXFixturePatchListItem>> FilteredListSource;

	/** The item that was last selected in the list */
	TSharedPtr<FDMXFixturePatchListItem> LastSelectedItem;

	/** The Search Bar for the List */
	TSharedPtr<SDMXFixturePatchListToolbar> Toolbar;

	/** The the list of Fixture Patches */
	TSharedPtr<SDMXFixturePatchListType> ListView;

	/** The Header Row of the List */
	TSharedPtr<SHeaderRow> HeaderRow;

	/** Rows of Mode widgets in the List */
	TArray<TSharedPtr<SDMXFixturePatchListRow>> Rows;

	/** The border that holds the list */
	TSharedPtr<SBorder> ListContentBorder;

	/** Timer handle for the Request List Refresh method */
	FTimerHandle RequestListRefreshTimerHandle;

	/** Shared Data for Fixture Patch Editors */
	TSharedPtr<FDMXFixturePatchSharedData> FixturePatchSharedData;

	/** The Fixture Type Editor that owns this widget */
	TWeakPtr<FDMXEditor> WeakDMXEditor;

private:
	///////////////////////////////////////////////////
	// Context menu Commands related

	TSharedPtr<SWidget> OnContextMenuOpening();

	/** Registers the commands */
	void RegisterCommands();

	bool CanCutItems() const;
	void OnCutSelectedItems();
	bool CanCopyItems() const;
	void OnCopySelectedItems();
	bool CanPasteItems() const;
	void OnPasteItems();
	bool CanDuplicateItems() const;
	void OnDuplicateItems();
	bool CanDeleteItems() const;
	void OnDeleteItems();

	TSharedPtr<FUICommandList> CommandList;
};
