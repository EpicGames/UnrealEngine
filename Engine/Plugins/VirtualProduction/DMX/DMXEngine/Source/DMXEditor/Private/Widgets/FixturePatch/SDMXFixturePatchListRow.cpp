// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXFixturePatchListRow.h"

#include "DMXEditor.h"
#include "DMXEditorStyle.h"
#include "Engine/EngineTypes.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/DMXLibrary.h"
#include "MVR/DMXMVRGeneralSceneDescription.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "Styling/AppStyle.h"
#include "Widgets/FixturePatch/DMXFixturePatchListItem.h"
#include "Widgets/FixturePatch/SDMXFixturePatchList.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "SDMXFixturePatchListRow"

/////////////////////////////////////////////////////
// SDMXFixturePatchFixtureTypePicker

/** Widget to pick a fixture type for a Fixture Patch */
class SDMXFixturePatchFixtureTypePicker
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMXFixturePatchFixtureTypePicker)
	{}

	SLATE_END_ARGS()

	/** Constructs this widget */
	void Construct(const FArguments& InArgs, const TSharedPtr<FDMXFixturePatchListItem>& InItem)
	{
		if (!ensureMsgf(InItem.IsValid(), TEXT("Invalid Fixture Patch List Item, cannot draw Fixture Type Picker for patch")))
		{
			return;
		}
		Item = InItem;

		UDMXLibrary* DMXLibrary = Item->GetDMXLibrary();
		const TArray<UDMXEntityFixtureType*> FixtureTypes = DMXLibrary ? DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixtureType>() : TArray<UDMXEntityFixtureType*>{};

		for (UDMXEntityFixtureType* FixtureType : FixtureTypes)
		{
			const TSharedPtr<FString> FixtureTypeName = MakeShared<FString>(FixtureType->Name);

			ComboBoxSource.Add(FixtureTypeName);
		}

		// Find an initial selection
		const TSharedPtr<FString>* InitialSelectionPtr = Algo::FindByPredicate(ComboBoxSource, 
			[this](const TSharedPtr<FString>& FixtureTypeName)
			{
				if (const UDMXEntityFixtureType* FixtureType = Item->GetFixtureType())
				{
					return *FixtureTypeName == FixtureType->Name;
				}

				return false;
			});

		const TSharedPtr<FString> InitialSelection = InitialSelectionPtr ? *InitialSelectionPtr : nullptr;

		ChildSlot
		[
			SNew(SSearchableComboBox)
			.OptionsSource(&ComboBoxSource)
			.InitiallySelectedItem(InitialSelection)
			.OnGenerateWidget(this, &SDMXFixturePatchFixtureTypePicker::OnGenerateWidget)
			.OnSelectionChanged(this, &SDMXFixturePatchFixtureTypePicker::OnSelectionChanged)
			[
				SNew(STextBlock)
				.Text(this, &SDMXFixturePatchFixtureTypePicker::GetSelectedItemText)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			]
		];
	}

private:
	/** Called when the combo box generates a widget */
	TSharedRef<SWidget> OnGenerateWidget(TSharedPtr<FString> InItem)
	{
		return
			SNew(STextBlock)
			.Text(FText::FromString(*InItem));
	}

	/** Called when the combo box selection changed */
	void OnSelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
	{
		UDMXLibrary* DMXLibrary = Item->GetDMXLibrary();
		const TArray<UDMXEntityFixtureType*> FixtureTypes = DMXLibrary ? DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixtureType>() : TArray<UDMXEntityFixtureType*>{};

		UDMXEntityFixtureType* const* FixtureTypePtr = Algo::FindBy(FixtureTypes, *NewSelection, &UDMXEntityFixtureType::Name);
		UDMXEntityFixtureType* FixtureType = FixtureTypePtr ? *FixtureTypePtr : nullptr;
		
		Item->SetFixtureType(FixtureType);
	}

	/** Returns the text of the selected item */
	FText GetSelectedItemText() const
	{
		if (UDMXEntityFixtureType* FixtureType = Item->GetFixtureType())
		{
			return FText::FromString(FixtureType->Name);
		}

		return LOCTEXT("NoFixtureType", "None");
	}

	/** Names of Fixture Types in the Combo Box */
	TArray<TSharedPtr<FString>> ComboBoxSource;

	/** The Fixture Patch Item for which this Fixture Type Picker is displayed */
	TSharedPtr<FDMXFixturePatchListItem> Item;
};


/////////////////////////////////////////////////////
// SDMXFixturePatchModePicker

/** Widget to pick a fixture type for an MVR Fixture */
class SDMXFixturePatchModePicker
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMXFixturePatchModePicker)
	{}

	SLATE_END_ARGS()

	/** Constructs this widget */
	void Construct(const FArguments& InArgs, const TSharedPtr<FDMXFixturePatchListItem>& InItem)
	{
		if (!ensureMsgf(InItem.IsValid(), TEXT("Invalid Fixture Patch List Item, cannot draw Mode Picker for patch")))
		{
			return;
		}
		Item = InItem;

		UpdateComboBoxSource();

		ChildSlot
		[
			SNew(SVerticalBox)
			.IsEnabled_Lambda([this]
				{
					return !ComboBoxSource.IsEmpty();
				})
				
			+ SVerticalBox::Slot()
			[
				SAssignNew(ComboBox, STextComboBox)
				.Visibility_Lambda([this]()
					{
						return Item->GetFixtureType() ? EVisibility::Visible : EVisibility::Collapsed;
					})
				.OptionsSource(&ComboBoxSource)
				.OnSelectionChanged(this, &SDMXFixturePatchModePicker::OnSelectionChanged)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			]

			+ SVerticalBox::Slot()
			[
				SNew(STextBlock)
				.Visibility_Lambda([this]()
					{
						return Item->GetFixtureType() ? EVisibility::Collapsed : EVisibility::Visible;
					})
				.Text(LOCTEXT("NoModeBecauseNoFixtureTypeSelectedInfo", "No Fixture Type Selected"))
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			]
		];

		AdoptSelectionFromFixturePatch();

		UDMXEntityFixturePatch::GetOnFixturePatchChanged().AddSP(this, &SDMXFixturePatchModePicker::OnFixturePatchChanged);
	}

private:
	/** Called when the combo box selection changed */
	void OnSelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
	{
		if (!NewSelection.IsValid())
		{
			Item->SetModeIndex(INDEX_NONE);
		}
		else if (const UDMXEntityFixtureType* FixtureType = Item->GetFixtureType())
		{
			const int32 ModeIndex = FixtureType->Modes.IndexOfByPredicate([&NewSelection](const FDMXFixtureMode& Mode)
				{
					return Mode.ModeName == *NewSelection;
				});

			Item->SetModeIndex(ModeIndex);
		}
	}
	
	/** Updates the combo box source */
	void UpdateComboBoxSource()
	{
		if (UDMXEntityFixtureType* FixtureType = Item->GetFixtureType())
		{
			for (int32 ModeIndex = 0; ModeIndex < FixtureType->Modes.Num(); ModeIndex++)
			{
				const TSharedPtr<FString> ModeName = MakeShared<FString>(FixtureType->Modes[ModeIndex].ModeName);

				ComboBoxSource.Add(ModeName);
			}
		}
	}

	/** Adopts the currently selected Item from the Mode of the Fixture Patch */
	void AdoptSelectionFromFixturePatch()
	{
		FString ActiveModeName;
		if (Item->GetActiveModeName(ActiveModeName))
		{
			const TSharedPtr<FString>* SelectionPtr = Algo::FindByPredicate(ComboBoxSource,
				[this](const TSharedPtr<FString>& ModeName)
				{
					FString ActiveModeName;
					if (Item->GetActiveModeName(ActiveModeName))
					{
						return *ModeName == ActiveModeName;
					}

					return false;
				});

			if (SelectionPtr)
			{
				ComboBox->SetSelectedItem(*SelectionPtr);
				return;
			}
		}

		ComboBox->ClearSelection();
	}

	/** Called when the fixture patch for which modes are displayed changed */
	void OnFixturePatchChanged(const UDMXEntityFixturePatch* FixturePatch)
	{
		if (Item->GetFixturePatch() == FixturePatch)
		{
			UpdateComboBoxSource();
			ComboBox->RefreshOptions();

			AdoptSelectionFromFixturePatch();
		}
	}

	/** The combo box to select a mode */
	TSharedPtr<STextComboBox> ComboBox;

	/** Names of Fixture Types in the Combo Box */
	TArray<TSharedPtr<FString>> ComboBoxSource;

	/** The Fixture Patch Item for which this Fixture Type Picker is displayed */
	TSharedPtr<FDMXFixturePatchListItem> Item;
};


/////////////////////////////////////////////////////
// SDMXFixturePatchListRow

void SDMXFixturePatchListRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, const TSharedRef<FDMXFixturePatchListItem>& InItem)
{
	Item = InItem;
	OnRowRequestsStatusRefresh = InArgs._OnRowRequestsStatusRefresh;
	OnRowRequestsListRefresh = InArgs._OnRowRequestsListRefresh;
	IsSelected = InArgs._IsSelected;

	SMultiColumnTableRow<TSharedPtr<FDMXFixturePatchListItem>>::Construct(
		FSuperRowType::FArguments()
		.Style(&FDMXEditorStyle::Get().GetWidgetStyle<FTableRowStyle>("FixturePatchList.Row")),
		InOwnerTable);
}

void SDMXFixturePatchListRow::EnterFixturePatchNameEditingMode()
{
	if (FixturePatchNameTextBlock.IsValid())
	{
		FixturePatchNameTextBlock->EnterEditingMode();
	}
}

TSharedRef<SWidget> SDMXFixturePatchListRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == FDMXFixturePatchListCollumnID::EditorColor)
	{
		return GenerateEditorColorWidget();
	}
	if (ColumnName == FDMXFixturePatchListCollumnID::FixturePatchName)
	{
		return GenerateFixturePatchNameWidget();
	}
	else if (ColumnName == FDMXFixturePatchListCollumnID::Status)
	{
		return GenerateStatusWidget();
	}
	else if (ColumnName == FDMXFixturePatchListCollumnID::FixtureID)
	{
		return GenerateFixtureIDWidget();
	}
	else if (ColumnName == FDMXFixturePatchListCollumnID::FixtureType)
	{
		return GenerateFixtureTypeWidget();
	}
	else if (ColumnName == FDMXFixturePatchListCollumnID::Mode)
	{
		return GenerateModeWidget();
	}
	else if (ColumnName == FDMXFixturePatchListCollumnID::Patch)
	{
		return GeneratePatchWidget();
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SDMXFixturePatchListRow::GenerateEditorColorWidget()
{
	return
		SNew(SBorder)
		.HAlign(HAlign_Fill)
		.Padding(5.f, 2.f)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		[
			SNew(SImage)
			.Image(FDMXEditorStyle::Get().GetBrush("DMXEditor.WhiteRoundedPropertyBorder"))
			.ColorAndOpacity_Lambda([this]()
			{
				return Item->GetBackgroundColor();
			})
		];
}

TSharedRef<SWidget> SDMXFixturePatchListRow::GenerateFixturePatchNameWidget()
{
	return
		SAssignNew(FixturePatchNameBorder, SBorder)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.Padding(4.f)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.OnMouseDoubleClick(this, &SDMXFixturePatchListRow::OnFixturePatchNameBorderDoubleClicked)
		[
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.BorderImage(FDMXEditorStyle::Get().GetBrush("DMXEditor.RoundedPropertyBorder"))
			[
				SAssignNew(FixturePatchNameTextBlock, SInlineEditableTextBlock)
				.Text_Lambda([this]()
				{
					const FString FixturePatchName = Item->GetFixturePatchName();
					return FText::FromString(FixturePatchName);
				})
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.OnTextCommitted(this, &SDMXFixturePatchListRow::OnFixturePatchNameCommitted)
				.IsSelected(IsSelected)
			]
		];
}

FReply SDMXFixturePatchListRow::OnFixturePatchNameBorderDoubleClicked(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (FixturePatchNameTextBlock.IsValid())
		{
			FixturePatchNameTextBlock->EnterEditingMode();
		}
	}

	return FReply::Handled();
}

void SDMXFixturePatchListRow::OnFixturePatchNameCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	if (InNewText.IsEmpty())
	{
		return;
	}

	FString ResultingName;
	Item->SetFixturePatchName(InNewText.ToString(), ResultingName);
	FixturePatchNameTextBlock->SetText(FText::FromString(ResultingName));
}

TSharedRef<SWidget> SDMXFixturePatchListRow::GenerateStatusWidget()
{
	return
		SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image_Lambda([this]()
				{
					if (!Item->ErrorStatusText.IsEmpty() || !Item->WarningStatusText.IsEmpty())
					{
						return FDMXEditorStyle::Get().GetBrush("Icons.WarningExclamationMark");
					}

					static const FSlateBrush EmptyBrush = FSlateNoResource();
					return &EmptyBrush;
				})
			.ToolTipText_Lambda([this]()
				{
					if (!Item->ErrorStatusText.IsEmpty())
					{
						return Item->ErrorStatusText;
					}
					else if (!Item->WarningStatusText.IsEmpty())
					{
						return Item->WarningStatusText;
					}

					return FText::GetEmpty();
				})
		];
}

TSharedRef<SWidget> SDMXFixturePatchListRow::GenerateFixtureIDWidget()
{
	return
		SNew(SBorder)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.Padding(4.f)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.OnMouseDoubleClick(this, &SDMXFixturePatchListRow::OnFixtureIDBorderDoubleClicked)
		[
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.BorderImage(FDMXEditorStyle::Get().GetBrush("DMXEditor.RoundedPropertyBorder"))
			[
				SAssignNew(FixtureIDTextBlocK, SInlineEditableTextBlock)
				.Text_Lambda([this]()
				{
					return FText::FromString(Item->GetFixtureID());
				})
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.OnTextCommitted(this, &SDMXFixturePatchListRow::OnFixtureIDCommitted)
				.IsSelected(IsSelected)
			]
		];
}

FReply SDMXFixturePatchListRow::OnFixtureIDBorderDoubleClicked(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (FixtureIDTextBlocK.IsValid())
		{
			FixtureIDTextBlocK->EnterEditingMode();
		}
	}

	return FReply::Handled();
}

void SDMXFixturePatchListRow::OnFixtureIDCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	const FString StringValue = InNewText.ToString();
	int32 NewFixtureID;
	if (LexTryParseString<int32>(NewFixtureID, *StringValue))
	{
		Item->SetFixtureID(NewFixtureID);

		const FString ParsedFixtureIDString = FString::FromInt(NewFixtureID);
		FixtureIDTextBlocK->SetText(FText::FromString(ParsedFixtureIDString));

		OnRowRequestsStatusRefresh.ExecuteIfBound();
	}
}

TSharedRef<SWidget> SDMXFixturePatchListRow::GenerateFixtureTypeWidget()
{
	return 
		SNew(SBorder)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.Padding(4.f)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		[
			SNew(SDMXFixturePatchFixtureTypePicker, Item)
		];
}

TSharedRef<SWidget> SDMXFixturePatchListRow::GenerateModeWidget()
{
	return
		SNew(SBorder)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.Padding(4.f)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		[
			SNew(SDMXFixturePatchModePicker, Item)
		];
}

TSharedRef<SWidget> SDMXFixturePatchListRow::GeneratePatchWidget()
{
	return
		SNew(SBorder)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.Padding(4.f)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.OnMouseDoubleClick(this, &SDMXFixturePatchListRow::OnPatchBorderDoubleClicked)
		[
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.BorderImage(FDMXEditorStyle::Get().GetBrush("DMXEditor.RoundedPropertyBorder"))
			[
				SAssignNew(PatchTextBlock, SInlineEditableTextBlock)
				.Text_Lambda([this]()
					{						
						const int32 UniverseID = Item->GetUniverse();
						const int32 StartingAddress = Item->GetAddress();
						if (UniverseID > 0 && StartingAddress > 0)
						{
							return FText::Format(LOCTEXT("AddressesText", "{0}.{1}"), UniverseID, StartingAddress);
						}
						else
						{
							return LOCTEXT("NotPatchedText", "Not patched");
						}
					})
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.OnTextCommitted(this, &SDMXFixturePatchListRow::OnPatchNameCommitted)
				.IsSelected(IsSelected)
			]
		];
}

FReply SDMXFixturePatchListRow::OnPatchBorderDoubleClicked(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (PatchTextBlock.IsValid())
		{
			PatchTextBlock->EnterEditingMode();
		}
	}

	return FReply::Handled();
}

void SDMXFixturePatchListRow::OnPatchNameCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	const FString PatchString = InNewText.ToString();
	static const TCHAR* ParamDelimiters[] =
	{
		TEXT("."),
		TEXT(","),
		TEXT(":"),
		TEXT(";")
	};

	TArray<FString> ValueStringArray;
	PatchString.ParseIntoArray(ValueStringArray, ParamDelimiters, 4);
	if (ValueStringArray.Num() == 2)
	{
		int32 Universe;
		if (!LexTryParseString(Universe, *ValueStringArray[0]))
		{
			return;
		}
		
		int32 Address;
		if (!LexTryParseString(Address, *ValueStringArray[1]))
		{
			return;
		}

		Item->SetAddresses(Universe, Address);

		const FString UniverseString = FString::FromInt(Universe);
		const FString AddressString = FString::FromInt(Address);
		PatchTextBlock->SetText(FText::Format(LOCTEXT("UniverseDotAddressText", "{0}.{1}"), FText::FromString(UniverseString), FText::FromString(AddressString)));

		OnRowRequestsListRefresh.ExecuteIfBound();
	}
}

#undef LOCTEXT_NAMESPACE
