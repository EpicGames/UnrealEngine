// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

enum class EDMXFixturePatchListEditMode : uint8;
class FDMXFixturePatchListItem;
class SInlineEditableTextBlock;
class UDMXEntityFixturePatch;
class UDMXEntityFixtureType;

/** MVR Fixture view as a row in a list */
class SDMXFixturePatchListRow
	: public SMultiColumnTableRow<TSharedPtr<FDMXFixturePatchListItem>>
{
public:
	SLATE_BEGIN_ARGS(SDMXFixturePatchListRow)
	{}
		/** Delegate executed when the row requests to refresh the statuses */
		SLATE_EVENT(FSimpleDelegate, OnRowRequestsStatusRefresh)

		/** Delegate executed when the row requests to refresh the whole list */
		SLATE_EVENT(FSimpleDelegate, OnRowRequestsListRefresh)

		/** Callback to check if the row is selected (should be hooked up if a parent widget is handling selection or focus) */
		SLATE_EVENT(FIsSelected, IsSelected)

	SLATE_END_ARGS()
			
	/** Constructs this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, const TSharedRef<FDMXFixturePatchListItem>& InItem);

	/** Enters editing mode for the Fixture Patch Name */
	void EnterFixturePatchNameEditingMode();

	/** Returns the Item of this row */
	TSharedPtr<FDMXFixturePatchListItem> GetItem() const { return Item; };

protected:
	//~ Begin SMultiColumnTableRow interface
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
	//~ End SMultiColumnTableRow interface

private:
	/** Generates the Widget that displays the Editor Color */
	TSharedRef<SWidget> GenerateEditorColorWidget();

	/** Generates the Widget that displays the Fixture Patch Name */
	TSharedRef<SWidget> GenerateFixturePatchNameWidget();

	/** Called when the Fixture Patch Name Border was double-clicked */
	FReply OnFixturePatchNameBorderDoubleClicked(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);

	/** Called when a Fixture Patch Name was committed */
	void OnFixturePatchNameCommitted(const FText& InNewText, ETextCommit::Type InTextCommit);

	/** Generates the widget that displays the Status */
	TSharedRef<SWidget> GenerateStatusWidget();

	/** Generates the Widget that displays the Fixture ID */
	TSharedRef<SWidget> GenerateFixtureIDWidget();

	/** Called when the Fixture ID Border was double-clicked */
	FReply OnFixtureIDBorderDoubleClicked(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);

	/** Called when a Fixture ID was committed */
	void OnFixtureIDCommitted(const FText& InNewText, ETextCommit::Type InTextCommit);

	/** Generates the Widget that displays the Fixture Type */
	TSharedRef<SWidget> GenerateFixtureTypeWidget();

	/** Generates the Widget that displays the Mode */
	TSharedRef<SWidget> GenerateModeWidget();

	/** Generates the Widget that displays the Patch */
	TSharedRef<SWidget> GeneratePatchWidget();

	/** Called when the Patch Border was double-clicked */
	FReply OnPatchBorderDoubleClicked(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);

	/** Called when a Patch was committed */
	void OnPatchNameCommitted(const FText& InNewText, ETextCommit::Type InTextCommit);

	/** The Edit Mode the Widget should present */
	EDMXFixturePatchListEditMode EditMode;

	/** The outermost border around the the Fixture Patch Name Column */
	TSharedPtr<SBorder> FixturePatchNameBorder;

	/** The text block to edit the Fixture Patch Name */
	TSharedPtr<SInlineEditableTextBlock> FixturePatchNameTextBlock;

	/** The text block to edit the Fixture ID */
	TSharedPtr<SInlineEditableTextBlock> FixtureIDTextBlocK;

	/** The text block to edit the Name */
	TSharedPtr<SInlineEditableTextBlock> NameTextBlock;

	/** The text block to edit the Name */
	TSharedPtr<SInlineEditableTextBlock> PatchTextBlock;

	/** The MVR Fixture List Item this row displays */
	TSharedPtr<FDMXFixturePatchListItem> Item;

	// Slate arguments
	FSimpleDelegate OnRowRequestsStatusRefresh;
	FSimpleDelegate OnRowRequestsListRefresh;
	FIsSelected IsSelected;
};
