// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXFixturePatchListToolbar.h"

#include "Algo/MaxElement.h"
#include "Commands/DMXEditorCommands.h"
#include "DMXEditor.h"
#include "DMXEditorUtils.h"
#include "DMXFixturePatchListItem.h"
#include "DMXFixturePatchSharedData.h"
#include "EditorStyleSet.h"
#include "FixturePatchAutoAssignUtility.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXEntityFixtureType.h"
#include "ScopedTransaction.h"
#include "SDMXAddFixturePatchMenu.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SDMXEntityDropdownMenu.h"

#define LOCTEXT_NAMESPACE "SDMXFixturePatchListToolbar"

void SDMXFixturePatchListToolbar::Construct(const FArguments& InArgs, TWeakPtr<FDMXEditor> InDMXEditor)
{
	WeakDMXEditor = InDMXEditor;
	OnSearchChanged = InArgs._OnSearchChanged;

	ChildSlot
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNew(SBorder)
			.Padding(FMargin(8.f, 8.f))
			.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f))
			.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
			[
				SNew(SWrapBox)
				.InnerSlotPadding(FVector2D(14.f, 8.f))
				.UseAllottedSize(true)

				// Add Fixture Button
				+ SWrapBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					GenerateFixtureTypeDropdownMenu()
				]
										
				+ SWrapBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SSeparator)
					.Orientation(EOrientation::Orient_Vertical)
				]

				// Search
				+ SWrapBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SSearchBox)
					.MinDesiredWidth(400.f)
					.OnTextChanged(this, &SDMXFixturePatchListToolbar::OnSearchTextChanged)
					.ToolTipText(LOCTEXT("SearchBarTooltip", "Examples:\n\n* PatchName\n* FixtureTypeName\n* SomeMode\n* 1.\n* 1.1\n* Universe 1\n* Uni 1-3\n* Uni 1, 3\n* Uni 1, 4-5'."))
				]
									
				+ SWrapBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SSeparator)
					.Orientation(EOrientation::Orient_Vertical)
				]

				// Show Conflicts Only option
				+ SWrapBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ShowConflictsOnlyCheckBoxLabel", "Show Conflicts only"))
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				]

				+ SWrapBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
					.IsChecked(false)
					.OnCheckStateChanged(this, &SDMXFixturePatchListToolbar::OnShowConflictsOnlyCheckStateChanged)
				]
			]
		];
}

TArray<TSharedPtr<FDMXFixturePatchListItem>> SDMXFixturePatchListToolbar::FilterItems(const TArray<TSharedPtr<FDMXFixturePatchListItem>>& Items)
{
	// Apply 'conflicts only' if enabled
	TArray<TSharedPtr<FDMXFixturePatchListItem>> Result = Items;
	if (bShowConfictsOnly)
	{
		Result.RemoveAll([](const TSharedPtr<FDMXFixturePatchListItem>& Item)
			{
				return
					Item->ErrorStatusText.IsEmpty() &&
					Item->WarningStatusText.IsEmpty();
			});
	}

	// Filter and return in order of precendence
	if (SearchString.IsEmpty())
	{
		return Result;
	}

	const TArray<int32> Universes = FDMXEditorUtils::ParseUniverses(SearchString);
	if(!Universes.IsEmpty())
	{
		Result.RemoveAll([Universes](const TSharedPtr<FDMXFixturePatchListItem>& Item)
			{
				return !Universes.Contains(Item->GetUniverse());
			});

		return Result;
	}

	int32 Address;
	if (FDMXEditorUtils::ParseAddress(SearchString, Address))
	{
		Result.RemoveAll([Address](const TSharedPtr<FDMXFixturePatchListItem>& Item)
			{
				return Item->GetAddress() != Address;
			});

		return Result;
	}
	
	const TArray<int32> FixtureIDs = FDMXEditorUtils::ParseFixtureIDs(SearchString);
	for (int32 FixtureID : FixtureIDs)
	{
		TArray<TSharedPtr<FDMXFixturePatchListItem>> FixtureIDsOnlyResult = Result;
		FixtureIDsOnlyResult.RemoveAll([FixtureID](const TSharedPtr<FDMXFixturePatchListItem>& Item)
			{
				int32 OtherFixtureIDNumerical;
				if (FDMXEditorUtils::ParseFixtureID(Item->GetFixtureID(), OtherFixtureIDNumerical))
				{
					return OtherFixtureIDNumerical != FixtureID;
				}
				return true;
			});

		if (FixtureIDsOnlyResult.Num() > 0)
		{
			return FixtureIDsOnlyResult;
		}
	}

	Result.RemoveAll([this](const TSharedPtr<FDMXFixturePatchListItem>& Item)
		{
			return !Item->GetFixturePatchName().Contains(SearchString);
		});

	return Result;
}

TSharedRef<SWidget> SDMXFixturePatchListToolbar::GenerateFixtureTypeDropdownMenu()
{
	FText AddButtonLabel = FDMXEditorCommands::Get().AddNewEntityFixturePatch->GetLabel();
	FText AddButtonToolTip = FDMXEditorCommands::Get().AddNewEntityFixturePatch->GetDescription();

	TSharedRef<SComboButton> AddComboButton = 
		SNew(SComboButton)
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(0, 1))
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Plus"))
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(FMargin(2, 0, 2, 0))
				[
					SNew(STextBlock)
					.Text(AddButtonLabel)
				]
			]
			.MenuContent()
			[
				SNew(SBox)
				.Padding(4.f)
				.MinDesiredWidth(460.f)
				[
					SAssignNew(AddFixturePatchMenu, UE::DMXEditor::FixturePatchEditor::SDMXAddFixturePatchMenu, WeakDMXEditor)
				]
			]
			.IsFocusable(true)
			.ContentPadding(FMargin(4.0f, 4.0f))
			.ComboButtonStyle(FAppStyle::Get(), "ToolbarComboButton")
			.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
			.ForegroundColor(FLinearColor::White)
			.ToolTipText(AddButtonToolTip)
			.OnComboBoxOpened_Lambda([this]()
				{
					if (AddFixturePatchMenu.IsValid())
					{
						AddFixturePatchMenu->RequestRefresh();
					}
				});

	return AddComboButton;
}

void SDMXFixturePatchListToolbar::OnSearchTextChanged(const FText& SearchText)
{
	SearchString = SearchText.ToString();
	OnSearchChanged.ExecuteIfBound();
}

void SDMXFixturePatchListToolbar::OnShowConflictsOnlyCheckStateChanged(const ECheckBoxState NewCheckState)
{
	bShowConfictsOnly = NewCheckState == ECheckBoxState::Checked;
	OnSearchChanged.ExecuteIfBound();
}

#undef LOCTEXT_NAMESPACE
