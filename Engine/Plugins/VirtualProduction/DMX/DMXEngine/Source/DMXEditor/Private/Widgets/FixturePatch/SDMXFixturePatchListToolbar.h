// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"

enum class ECheckBoxState : uint8;
class FDMXEditor;
class FDMXFixturePatchListItem;
template<typename TEntityType> class SDMXEntityDropdownMenu;
class UDMXEntity;
class UDMXEntityFixtureType;
namespace UE::DMXEditor::FixturePatchEditor { class SDMXAddFixturePatchMenu; }


/** Search bar for the MVR Fixture List */
class SDMXFixturePatchListToolbar
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMXFixturePatchListToolbar)
	{}

		/** Executed when the search changed */
		SLATE_EVENT(FSimpleDelegate, OnSearchChanged)

	SLATE_END_ARGS()

	/** Constructs this widget */
	void Construct(const FArguments& InArgs, TWeakPtr<FDMXEditor> InDMXEditor);

	/** Filters items according to the state of the search bar */
	[[nodiscard]] TArray<TSharedPtr<FDMXFixturePatchListItem>> FilterItems(const TArray<TSharedPtr<FDMXFixturePatchListItem>>& Items);

private:
	/** Generates the 'Add MVR Fixture' Dropdown Menu */
	TSharedRef<SWidget> GenerateFixtureTypeDropdownMenu();

	/** Called when the Search Text changed */
	void OnSearchTextChanged(const FText& SearchText);

	/** Called when the Check State of the 'Show Conflicts Only' Check Box changed */
	void OnShowConflictsOnlyCheckStateChanged(const ECheckBoxState NewCheckState);

	/** Current Search String */
	FString SearchString;

	/** True if only conflicts are searched for */
	bool bShowConfictsOnly = false;

	/** Menu displayed to add fixture patches */
	TSharedPtr<UE::DMXEditor::FixturePatchEditor::SDMXAddFixturePatchMenu> AddFixturePatchMenu;

	/** The DMX Editor that owns this widget */
	TWeakPtr<FDMXEditor> WeakDMXEditor;

	// Slate Args
	FSimpleDelegate OnSearchChanged;
};
