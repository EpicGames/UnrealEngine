// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXFixturePatcher.h"

#include "Algo/RemoveIf.h"
#include "DMXEditor.h"
#include "DMXEditorSettings.h"
#include "DMXEditorTabNames.h"
#include "DMXFixturePatchSharedData.h"
#include "DMXFixturePatchNode.h"
#include "SDMXPatchedUniverse.h"
#include "DragDrop/DMXEntityDragDropOp.h"
#include "DragDrop/DMXEntityFixturePatchDragDropOp.h"
#include "IO/DMXPortManager.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXLibrary.h"
#include "Styling/AppStyle.h"
#include "ScopedTransaction.h"
#include "SlateOptMacros.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"


#define LOCTEXT_NAMESPACE "SDMXFixturePatcher"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SDMXFixturePatcher::Construct(const FArguments& InArgs)
{
	DMXEditorPtr = InArgs._DMXEditor;
	if (!DMXEditorPtr.IsValid())
	{
		return;
	}

	SharedData = DMXEditorPtr.Pin()->GetFixturePatchSharedData();

	const TSharedRef<SScrollBar> HorizontalScrollBar = SNew(SScrollBar)
		.Orientation(Orient_Horizontal);

	const TSharedRef<SScrollBar> VerticalScrollBar = SNew(SScrollBar)
		.Orientation(Orient_Vertical);

	const FLinearColor BackgroundTint(0.6f, 0.6f, 0.6f, 1.0f);
	ChildSlot			
		[
			SNew(SBox)
			.HAlign(HAlign_Left)
			.ToolTipText(this, &SDMXFixturePatcher::GetTooltipText)
			[
				SNew(SVerticalBox)				

				+ SVerticalBox::Slot()
				.HAlign(HAlign_Fill)
				.AutoHeight()
				[
					SNew(SBorder)					
					.HAlign(HAlign_Fill)
					.BorderBackgroundColor(BackgroundTint)
					.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
					[
						CreateToolbar()
					]
				]

				+ SVerticalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Fill)
				[	
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					[
						SAssignNew(PatchedUniverseHorizontalScrollBox, SScrollBox)
						.Orientation(EOrientation::Orient_Horizontal)
						.ExternalScrollbar(HorizontalScrollBar)

						+SScrollBox::Slot()
						.FillSize(1.f)
						[
							SAssignNew(PatchedUniverseVerticalScrollBox, SScrollBox)
							.Orientation(EOrientation::Orient_Vertical)	
							.ExternalScrollbar(VerticalScrollBar)
						]
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						VerticalScrollBar
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					HorizontalScrollBar
				]
			]
		];

	// Bind to selection changes
	SharedData->OnFixturePatchSelectionChanged.AddSP(this, &SDMXFixturePatcher::OnFixturePatchSelectionChanged);
	SharedData->OnUniverseSelectionChanged.AddSP(this, &SDMXFixturePatcher::OnUniverseSelectionChanged);

	// If the selected universe has no patches, try to find one with patches instead
	UDMXLibrary* Library = GetDMXLibrary();
	check(Library);

	// Bind to object changes
	UDMXEntityFixtureType::GetOnFixtureTypeChanged().AddSP(this, &SDMXFixturePatcher::OnFixtureTypeChanged);
	Library->GetOnEntitiesAdded().AddSP(this, &SDMXFixturePatcher::OnEntitiesAddedOrRemoved);
	Library->GetOnEntitiesRemoved().AddSP(this, &SDMXFixturePatcher::OnEntitiesAddedOrRemoved);

	TArray<UDMXEntityFixturePatch*> Patches = Library->GetEntitiesTypeCast<UDMXEntityFixturePatch>();
	UDMXEntityFixturePatch** ExistingPatchPtr = Patches.FindByPredicate([&](UDMXEntityFixturePatch* Patch) {
		return Patch->GetUniverseID() == SharedData->GetSelectedUniverse();
		});
	if (!ExistingPatchPtr && Patches.Num() > 0)
	{
		SharedData->SelectUniverse(Patches[0]->GetUniverseID());
	}		

	ShowSelectedUniverse();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SDMXFixturePatcher::RefreshFromProperties()
{
	RefreshFixturePatchState = EDMXRefreshFixturePatcherState::RefreshFromProperties;
}

void SDMXFixturePatcher::RefreshFromLibrary()
{
	RefreshFixturePatchState = EDMXRefreshFixturePatcherState::RefreshFromLibrary;
}

void SDMXFixturePatcher::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Refresh if requested
	if (RefreshFixturePatchState != EDMXRefreshFixturePatcherState::NoRefreshRequested)
	{
		if (IsUniverseSelectionEnabled())
		{
			ShowSelectedUniverse();
		}
		else
		{
			ShowAllPatchedUniverses();
		}

		RefreshFixturePatchState = EDMXRefreshFixturePatcherState::NoRefreshRequested;
	}

	// Set Universe if requested
	if (UniverseToSetNextTick != INDEX_NONE)
	{
		SharedData->SelectUniverse(UniverseToSetNextTick);
		UniverseToSetNextTick = INDEX_NONE;

		for (const TTuple<int32, TSharedPtr<SDMXPatchedUniverse>>& UniverseToWidgetPair : PatchedUniversesByID)
		{
			if (UniverseToWidgetPair.Value.IsValid())
			{
				UniverseToWidgetPair.Value->RequestRefresh();
			}
		}
	}
}

FReply SDMXFixturePatcher::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (!Operation.IsValid())
	{
		return FReply::Unhandled();
	}

	if (TSharedPtr<FDMXEntityDragDropOperation> EntityDragDropOp = DragDropEvent.GetOperationAs<FDMXEntityDragDropOperation>())
	{
		return FReply::Handled().EndDragDrop();
	}

	return FReply::Unhandled();
}

TSharedRef<SWidget> SDMXFixturePatcher::CreateToolbar()
{
	TSharedPtr<FUICommandList> CommandList = nullptr;
	FToolBarBuilder	ToolbarBuilder(CommandList, FMultiBoxCustomization::None);

	ToolbarBuilder.BeginSection("UniverseSelection");
	{
		ToolbarBuilder.AddWidget(
			SNew(SBox)
			.Padding(4.f, 0.f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.MinDesiredWidth(75.0f)
				.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
				.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
				.IsEnabled(this, &SDMXFixturePatcher::IsUniverseSelectionEnabled)
				.Text(LOCTEXT("UniverseSelectorLabel", "Local Universe"))
			]
		);

		ToolbarBuilder.AddWidget(
			SNew(SBox)
			.Padding(4.f, 0.f)
			.MinDesiredWidth(210.0f)
			.MaxDesiredWidth(420.0f)
			[
				SNew(SSpinBox<int32>)
				.SliderExponent(1000.0f)
				.MinSliderValue(0)
				.MaxSliderValue(DMX_MAX_UNIVERSE - 1)
				.MinValue(0)
				.MaxValue(DMX_MAX_UNIVERSE - 1)
				.IsEnabled(this, &SDMXFixturePatcher::IsUniverseSelectionEnabled)
				.Value(this, &SDMXFixturePatcher::GetSelectedUniverse)
				.OnValueChanged(this, &SDMXFixturePatcher::SelectUniverse)
			]
		);
	}
	ToolbarBuilder.EndSection();

	ToolbarBuilder.BeginSection("ExpandedSettings");
	{		
		ToolbarBuilder.AddWidget(
			SAssignNew(ShowAllUniversesCheckBox, SCheckBox)
			.IsChecked(false)
			.OnCheckStateChanged(this, &SDMXFixturePatcher::OnToggleDisplayAllUniverses)
			[
				SNew(STextBlock)
				.Margin(FMargin(4.f, 0.f))
				.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
				.Text(LOCTEXT("ShowAllPatchedUniversesLabel", "Show all patched Universes"))
			]
		);

		ToolbarBuilder.AddSeparator();

		const UDMXEditorSettings* DMXEditorSettings = GetDefault<UDMXEditorSettings>();
		const ECheckBoxState InitialDMXMonitorEnabledCheckBoxState = DMXEditorSettings->FixturePatcherSettings.bMonitorEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;

		ToolbarBuilder.AddWidget(
			SAssignNew(EnableDMXMonitorCheckBox, SCheckBox)
			.IsChecked(InitialDMXMonitorEnabledCheckBoxState)
			.OnCheckStateChanged(this, &SDMXFixturePatcher::OnToggleDMXMonitorEnabled)
			[
				SNew(STextBlock)
				.Margin(FMargin(4.f, 0.f))
				.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
				.Text(LOCTEXT("EnableInputMonitorLabel", "Monitor DMX Inputs"))
				.ToolTipText(LOCTEXT("EnableInputMonitorTooltip", "If checked, monitors DMX Input Ports used in this DMX Library"))
			]
		);
	}
	ToolbarBuilder.EndSection();

	const TSharedRef<SWidget> Toolbar =	SNew(SHorizontalBox)
		
		// Toolbar menu
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.AutoWidth()
		[	
			ToolbarBuilder.MakeWidget()
		]

		// Settings button
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		[
			SNew(SComboButton)
			.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButtonWithIcon") // Use the tool bar item style for this button
			.OnGetMenuContent(this, &SDMXFixturePatcher::CreateSettingsMenu)
			.HasDownArrow(false)
			.ButtonContent()
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
			]
		];

	return Toolbar;
}

TSharedRef<SWidget> SDMXFixturePatcher::CreateSettingsMenu()
{
	constexpr bool bCloseMenuWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bCloseMenuWindowAfterMenuSelection, nullptr);

	MenuBuilder.BeginSection("PatchNameSection", LOCTEXT("PatchNameSectionLabel", "Displayed Patch Name"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("FixtureID", "Fixture ID"),
			LOCTEXT("FixtureIDTooltip", "Displays the Fixture IDs of the Patches"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SDMXFixturePatcher::SelectFixturePatchNameDisplayMode, EDMXFixturePatcherNameDisplayMode::FixtureID),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SDMXFixturePatcher::IsUsingFixturePatchNameDisplayMode, EDMXFixturePatcherNameDisplayMode::FixtureID)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("DisplayName", "Patch Name"),
			LOCTEXT("DisplayNameTooltip", "Displays the Names of the Patches"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SDMXFixturePatcher::SelectFixturePatchNameDisplayMode, EDMXFixturePatcherNameDisplayMode::FixturePatchName),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SDMXFixturePatcher::IsUsingFixturePatchNameDisplayMode, EDMXFixturePatcherNameDisplayMode::FixturePatchName)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("FixtureIDAndName", "Fixture ID and Patch Name"),
			LOCTEXT("FixtureIDAndNameTooltip", "Displays the Fixture IDs and Names of the Patches"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SDMXFixturePatcher::SelectFixturePatchNameDisplayMode, EDMXFixturePatcherNameDisplayMode::FixtureIDAndFixturePatchName),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SDMXFixturePatcher::IsUsingFixturePatchNameDisplayMode, EDMXFixturePatcherNameDisplayMode::FixtureIDAndFixturePatchName)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SDMXFixturePatcher::SelectFixturePatchNameDisplayMode(EDMXFixturePatcherNameDisplayMode DisplayMode)
{
	UDMXEditorSettings* EditorSettings = GetMutableDefault<UDMXEditorSettings>();
	EditorSettings->FixturePatcherSettings.FixturePatchNameDisplayMode = DisplayMode;
	EditorSettings->SaveConfig();
}

bool SDMXFixturePatcher::IsUsingFixturePatchNameDisplayMode(EDMXFixturePatcherNameDisplayMode DisplayMode) const
{
	const UDMXEditorSettings* EditorSettings = GetDefault<UDMXEditorSettings>();
	return DisplayMode == EditorSettings->FixturePatcherSettings.FixturePatchNameDisplayMode;
}

void SDMXFixturePatcher::OnDragEnterChannel(int32 UniverseID, int32 ChannelID, const FDragDropEvent& DragDropEvent)
{
	// Same as dropping onto a channel, but without FReply
	OnDropOntoChannel(UniverseID, ChannelID, DragDropEvent);
}

FReply SDMXFixturePatcher::OnDropOntoChannel(int32 UniverseID, int32 ChannelID, const FDragDropEvent& DragDropEvent)
{
	const TMap<TSharedRef<FDMXFixturePatchNode>, int64> DraggedNodesToAbsoluteChannelOffsetMap = GetDraggedNodesToAbsoluteChannelOffsetMap(DragDropEvent);
	if (DraggedNodesToAbsoluteChannelOffsetMap.IsEmpty())
	{
		return FReply::Handled().EndDragDrop();
	}

	int32 PadOffset = 0;
	for (const TTuple<TSharedRef<FDMXFixturePatchNode>, int64>& DraggedNodesToChannelOffsetPair : DraggedNodesToAbsoluteChannelOffsetMap)
	{
		const TSharedRef<FDMXFixturePatchNode>& PatchedNode = DraggedNodesToChannelOffsetPair.Key;

		if (UDMXEntityFixturePatch* FixturePatch = DraggedNodesToChannelOffsetPair.Key->GetFixturePatch().Get())
		{
			const int32 ChannelSpan = FixturePatch->GetChannelSpan();
			const int32 AbsoluteOffset = DraggedNodesToChannelOffsetPair.Value + PadOffset;

			int32 NewUniverseID = UniverseID;
			int32 DesiredStartingChannel = ChannelID;
			if (IsUniverseSelectionEnabled())
			{
				// Stay within the current Universe if only one universe is displayed
				if (FixturePatch->GetUniverseID() != UniverseID)
				{
					continue;
				}

				DesiredStartingChannel = ChannelID - AbsoluteOffset % DMX_MAX_ADDRESS;
			}
			else
			{
				int64 AbsoluteDesiredStartingChannel = UniverseID * DMX_MAX_ADDRESS + ChannelID - AbsoluteOffset;

				// Allow drag to another universe if multiple universes are displayed
				int32 DesiredUniverse = AbsoluteDesiredStartingChannel / DMX_MAX_ADDRESS;
				if (DesiredUniverse < 1)
				{
					DesiredUniverse = 1;
					AbsoluteDesiredStartingChannel = ChannelID - AbsoluteOffset % DMX_MAX_ADDRESS;
				}
				else if (DesiredUniverse > DMX_MAX_UNIVERSE)
				{
					DesiredUniverse = DMX_MAX_UNIVERSE;
					AbsoluteDesiredStartingChannel = DMX_MAX_UNIVERSE * DMX_MAX_ADDRESS - ChannelSpan + 1;
				}
				else
				{
					NewUniverseID = DesiredUniverse;
				}

				// Remove from previous universe if needed
				if (NewUniverseID != PatchedNode->GetUniverseID())
				{
					const TSharedPtr<SDMXPatchedUniverse>* OldUniverseWidgetPtr = PatchedUniversesByID.Find(PatchedNode->GetUniverseID());
					if (OldUniverseWidgetPtr)
					{
						(*OldUniverseWidgetPtr)->Remove(PatchedNode);
						(*OldUniverseWidgetPtr)->RequestRefresh();
					}
				}

				DesiredStartingChannel = AbsoluteDesiredStartingChannel % DMX_MAX_ADDRESS;
			}

			const int32 NewStartingChannel = ClampStartingChannel(DesiredStartingChannel, ChannelSpan);
			DraggedNodesToChannelOffsetPair.Key->SetAddresses(NewUniverseID, NewStartingChannel, ChannelSpan);

			const TSharedPtr<SDMXPatchedUniverse>* UniverseWidgetPtr = PatchedUniversesByID.Find(NewUniverseID);
			if (!UniverseWidgetPtr)
			{
				AddUniverse(NewUniverseID);
				UniverseWidgetPtr = &PatchedUniversesByID.FindChecked(NewUniverseID);
			}
			check(UniverseWidgetPtr);
			(*UniverseWidgetPtr)->FindOrAdd(DraggedNodesToChannelOffsetPair.Key);

			// Pad further patches so they can't be offset more than the currently dropped patch
			PadOffset = FMath::Min(PadOffset, DesiredStartingChannel - NewStartingChannel);
		}
	}

	return FReply::Handled().EndDragDrop();
}

TMap<TSharedRef<FDMXFixturePatchNode>, int64> SDMXFixturePatcher::GetDraggedNodesToAbsoluteChannelOffsetMap(const FDragDropEvent& DragDropEvent) const
{
	TMap<TSharedRef<FDMXFixturePatchNode>, int64> DraggedNodesToAbsoluteChannelOffsetMap;

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (!Operation.IsValid())
	{
		return DraggedNodesToAbsoluteChannelOffsetMap;
	}

	const TSharedPtr<FDMXEntityFixturePatchDragDropOperation> FixturePatchDragDropOp = DragDropEvent.GetOperationAs<FDMXEntityFixturePatchDragDropOperation>();
	if (!FixturePatchDragDropOp.IsValid())
	{
		return DraggedNodesToAbsoluteChannelOffsetMap;
	}

	const TMap<TWeakObjectPtr<UDMXEntityFixturePatch>, int64>& FixturePatchToAbsoluteChannelOffsetMap = FixturePatchDragDropOp->GetFixturePatchToAbsoluteChannelOffsetMap();
	for (const TTuple<TWeakObjectPtr<UDMXEntityFixturePatch>, int64>& FixturePatchToAbsoluteChannelOffsetPair : FixturePatchToAbsoluteChannelOffsetMap)
	{
		UDMXEntityFixturePatch* FixturePatch = FixturePatchToAbsoluteChannelOffsetPair.Key.Get();
		if (!FixturePatch)
		{
			continue;
		}

		TSharedPtr<FDMXFixturePatchNode> DraggedNode = FindPatchNode(FixturePatch);
		if (!DraggedNode.IsValid())
		{
			DraggedNode = FDMXFixturePatchNode::Create(DMXEditorPtr, FixturePatch);
		}

		if (DraggedNode.IsValid())
		{
			DraggedNodesToAbsoluteChannelOffsetMap.Add(DraggedNode.ToSharedRef(), FixturePatchToAbsoluteChannelOffsetPair.Value);
		}
	}

	return DraggedNodesToAbsoluteChannelOffsetMap;
}

void SDMXFixturePatcher::PostUndo(bool bSuccess)
{
	UDMXLibrary* DMXLibrary = GetDMXLibrary();
	if(DMXLibrary)
	{
		DMXLibrary->Modify();
	}

	RefreshFromLibrary();
}

void SDMXFixturePatcher::PostRedo(bool bSuccess)
{
	RefreshFromLibrary();
}

TSharedPtr<FDMXFixturePatchNode> SDMXFixturePatcher::FindPatchNode(const TWeakObjectPtr<UDMXEntityFixturePatch> FixturePatch) const
{
	TSharedPtr<FDMXFixturePatchNode> ExistingNode;
	for (const TPair<int32, TSharedPtr<SDMXPatchedUniverse>>& UniverseByID : PatchedUniversesByID)
	{
		ExistingNode = UniverseByID.Value->FindPatchNode(FixturePatch);

		if (ExistingNode.IsValid())
		{
			return ExistingNode;
		}
	}

	return nullptr;
}

TSharedPtr<FDMXFixturePatchNode> SDMXFixturePatcher::FindPatchNodeOfType(UDMXEntityFixtureType* Type, const TSharedPtr<FDMXFixturePatchNode>& IgoredNode) const
{
	if (Type)
	{
		if (const TPair<int32, TSharedPtr<SDMXPatchedUniverse>>* UniverseByID = PatchedUniversesByID.FindArbitraryElement())
		{		
			return UniverseByID->Value->FindPatchNodeOfType(Type, IgoredNode);
		}
	}
	return nullptr;
}

void SDMXFixturePatcher::OnEntitiesAddedOrRemoved(UDMXLibrary* DMXLibrary, TArray<UDMXEntity*> Entities)
{
	if (DMXLibrary == GetDMXLibrary())
	{
		RefreshFromLibrary();
	}
}

void SDMXFixturePatcher::OnFixtureTypeChanged(const UDMXEntityFixtureType* FixtureType)
{
	RefreshFromLibrary();
}

void SDMXFixturePatcher::OnFixturePatchSelectionChanged()
{
	const int32 SelectedUniverse = SharedData->GetSelectedUniverse();
	if (PatchedUniversesByID.Contains(SelectedUniverse))
	{
		return;
	}

	const TSharedPtr<SDMXPatchedUniverse>* const UniverseWidgetPtr = PatchedUniversesByID.Find(SelectedUniverse);
	if (IsUniverseSelectionEnabled() || !UniverseWidgetPtr)
	{
		// Refresh if all universes are displayed, or if the Universe was empty previously and now needs to be displayed.
		ShowSelectedUniverse();
	}
	else
	{
		PatchedUniverseHorizontalScrollBox->ScrollDescendantIntoView(*UniverseWidgetPtr);
		PatchedUniverseVerticalScrollBox->ScrollDescendantIntoView(*UniverseWidgetPtr);
	}
}

void SDMXFixturePatcher::OnUniverseSelectionChanged()
{
	if (IsUniverseSelectionEnabled())
	{
		GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateSP(this, &SDMXFixturePatcher::ShowSelectedUniverse));
	}
	else 
	{
		constexpr bool bReconstructWidgets = false;
		GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateSP(this, &SDMXFixturePatcher::ShowAllPatchedUniverses, bReconstructWidgets));
	}
}

void SDMXFixturePatcher::SelectUniverse(int32 NewUniverseID)
{
	UniverseToSetNextTick = FMath::Clamp(NewUniverseID, 1, DMX_MAX_UNIVERSE);
}

int32 SDMXFixturePatcher::GetSelectedUniverse() const
{
	check(SharedData.IsValid());
	return UniverseToSetNextTick != INDEX_NONE ? UniverseToSetNextTick : SharedData->GetSelectedUniverse();
}

void SDMXFixturePatcher::ShowSelectedUniverse()
{
	PatchedUniverseVerticalScrollBox->ClearChildren();
	PatchedUniversesByID.Reset();
	
	const int32 SelectedUniverseID = GetSelectedUniverse();
	const TSharedRef<SDMXPatchedUniverse> NewPatchedUniverse =
		SNew(SDMXPatchedUniverse)
		.DMXEditor(DMXEditorPtr)
		.UniverseID(SelectedUniverseID)
		.OnDragEnterChannel(this, &SDMXFixturePatcher::OnDragEnterChannel)
		.OnDropOntoChannel(this, &SDMXFixturePatcher::OnDropOntoChannel);

	PatchedUniverseVerticalScrollBox->AddSlot()
		.Padding(FMargin(0.0f, 3.0f, 0.0f, 12.0f))			
		[
			NewPatchedUniverse
		];

	PatchedUniversesByID.Add(SelectedUniverseID, NewPatchedUniverse);

	// Update monitor
	UDMXEditorSettings* DMXEditorSettings = GetMutableDefault<UDMXEditorSettings>();
	SetDMXMonitorEnabled(DMXEditorSettings->FixturePatcherSettings.bMonitorEnabled);
}

void SDMXFixturePatcher::ShowAllPatchedUniverses(bool bForceReconstructWidget)
{
	check(PatchedUniverseHorizontalScrollBox.IsValid() && PatchedUniverseVerticalScrollBox.IsValid());

	if (bForceReconstructWidget)
	{
		PatchedUniverseVerticalScrollBox->ClearChildren();
		PatchedUniversesByID.Reset();
	}

	UDMXLibrary* Library = GetDMXLibrary();
	if (Library)
	{
		TArray<UDMXEntityFixturePatch*> FixturePatches = Library->GetEntitiesTypeCast<UDMXEntityFixturePatch>();

		// Sort by universe ID
		FixturePatches.Sort([](const UDMXEntityFixturePatch& Patch, const UDMXEntityFixturePatch& Other) {
			return Patch.GetUniverseID() < Other.GetUniverseID();
			});

		// Create widgets for all universe with patches		
		for(UDMXEntityFixturePatch* Patch : FixturePatches)
		{
			check(Patch);

			// Ignore patches that are not residing in a universe
			if (Patch->GetUniverseID() < 0)
			{
				continue;
			}

			if (!PatchedUniversesByID.Contains(Patch->GetUniverseID()))
			{
				AddUniverse(Patch->GetUniverseID());
			}
		}

		TMap<int32, TSharedPtr<SDMXPatchedUniverse>> CachedPatchedUniversesByID = PatchedUniversesByID;
		for (const TPair<int32, TSharedPtr<SDMXPatchedUniverse>>& UniverseByIDKvp : CachedPatchedUniversesByID)
		{
			check(UniverseByIDKvp.Value.IsValid());

			if (UniverseByIDKvp.Value->GetPatchedNodes().Num() == 0)
			{
				// Remove universe widgets without patches
				PatchedUniversesByID.Remove(UniverseByIDKvp.Key);
				PatchedUniverseVerticalScrollBox->RemoveSlot(UniverseByIDKvp.Value.ToSharedRef());
			}
		}

		// Show last patched universe +1 for convenience of adding patches to a new universe
		TArray<int32> UniverseIDs;
		PatchedUniversesByID.GetKeys(UniverseIDs);
		UniverseIDs.Sort([](int32 FirstID, int32 SecondID) {
			return FirstID > SecondID;
			});

		int32 LastPatchedUniverseID = 0;
		if (UniverseIDs.Num() > 0)
		{
			LastPatchedUniverseID = UniverseIDs[0];
		}
	
		int32 FirstEmptyUniverse = LastPatchedUniverseID + 1;
		AddUniverse(FirstEmptyUniverse);

		const int32 SelectedUniverse = SharedData->GetSelectedUniverse();
		if (PatchedUniversesByID.Contains(SelectedUniverse))
		{
			PatchedUniverseHorizontalScrollBox->ScrollDescendantIntoView(PatchedUniversesByID[SelectedUniverse]);
			PatchedUniverseVerticalScrollBox->ScrollDescendantIntoView(PatchedUniversesByID[SelectedUniverse]);
		}
	}

	// Update monitor
	UDMXEditorSettings* DMXEditorSettings = GetMutableDefault<UDMXEditorSettings>();
	SetDMXMonitorEnabled(DMXEditorSettings->FixturePatcherSettings.bMonitorEnabled);
}

void SDMXFixturePatcher::AddUniverse(int32 UniverseID)
{
	TSharedRef<SDMXPatchedUniverse> PatchedUniverse =
		SNew(SDMXPatchedUniverse)
		.DMXEditor(DMXEditorPtr)
		.UniverseID(UniverseID)
		.OnDragEnterChannel(this, &SDMXFixturePatcher::OnDragEnterChannel)
		.OnDropOntoChannel(this, &SDMXFixturePatcher::OnDropOntoChannel);

	PatchedUniverseVerticalScrollBox->AddSlot()
		.Padding(FMargin(0.0f, 3.0f, 0.0f, 12.0f))
		[
			PatchedUniverse
		];

	PatchedUniversesByID.Add(UniverseID, PatchedUniverse);
}

void SDMXFixturePatcher::OnToggleDisplayAllUniverses(ECheckBoxState CheckboxState)
{
	const int32 SelectedUniverse = SharedData->GetSelectedUniverse();

	switch (ShowAllUniversesCheckBox->GetCheckedState())
	{
	case ECheckBoxState::Checked:
		ShowAllPatchedUniverses();
		return;

	case ECheckBoxState::Unchecked:
		SelectUniverse(SelectedUniverse);
		ShowSelectedUniverse();
		return;

	case ECheckBoxState::Undetermined:
	default:
		checkNoEntry();
	}
}

void SDMXFixturePatcher::SetDMXMonitorEnabled(bool bEnabled)
{
	UDMXEditorSettings* DMXEditorSettings = GetMutableDefault<UDMXEditorSettings>();
	DMXEditorSettings->FixturePatcherSettings.bMonitorEnabled = bEnabled;
	DMXEditorSettings->SaveConfig();

	for (const TTuple<int32, TSharedPtr<SDMXPatchedUniverse>>& UniverseToWidgetPair : PatchedUniversesByID)
	{
		UniverseToWidgetPair.Value->SetMonitorInputsEnabled(bEnabled);
	}
}

void SDMXFixturePatcher::OnToggleDMXMonitorEnabled(ECheckBoxState CheckboxState)
{
	switch (EnableDMXMonitorCheckBox->GetCheckedState())
	{
	case ECheckBoxState::Checked:
		SetDMXMonitorEnabled(true);
		return;

	case ECheckBoxState::Unchecked:
		SetDMXMonitorEnabled(false);
		return;

	case ECheckBoxState::Undetermined:
	default:
		checkNoEntry();
	}
}

bool SDMXFixturePatcher::IsUniverseSelectionEnabled() const
{
	check(ShowAllUniversesCheckBox.IsValid());

	switch (ShowAllUniversesCheckBox->GetCheckedState())
	{
	case ECheckBoxState::Checked:
		return false;

	case ECheckBoxState::Unchecked:
		return true;
		
	case ECheckBoxState::Undetermined:
	default:
		checkNoEntry();
	}

	return false;
}

bool SDMXFixturePatcher::HasAnyPorts() const
{
	UDMXLibrary* Library = GetDMXLibrary();
	if (Library)
	{
		const bool bHasAnyPorts =  Library->GetInputPorts().Num() > 0 || Library->GetOutputPorts().Num() > 0;

		return bHasAnyPorts;
	}
	return false;
}

FText SDMXFixturePatcher::GetTooltipText() const
{
	if (!HasAnyPorts())
	{
		return LOCTEXT("NoPortsTooltip", "No ports available. Please create ports in Project Settings -> Plugins -> DMX.");
	}

	return FText::GetEmpty();
}

int32 SDMXFixturePatcher::ClampStartingChannel(int32 StartingChannel, int32 ChannelSpan) const
{
	if (StartingChannel < 1)
	{
		StartingChannel = 1;
	}

	if (StartingChannel + ChannelSpan > DMX_UNIVERSE_SIZE)
	{
		StartingChannel = DMX_UNIVERSE_SIZE - ChannelSpan + 1;
	}

	return StartingChannel;
}

UDMXLibrary* SDMXFixturePatcher::GetDMXLibrary() const
{
	if (TSharedPtr<FDMXEditor> DMXEditor = DMXEditorPtr.Pin())
	{
		return DMXEditor->GetDMXLibrary();
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
