// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "Styling/SlateColor.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

enum class EDMXFixturePatcherNameDisplayMode : uint8;
class FDMXEditor;
class FDMXEntityFixturePatchDragDropOperation;
class FDMXFixturePatchNode;
class FDMXFixturePatchSharedData;
class FDMXTreeNodeBase;
class SDMXPatchedUniverse;
class UDMXLibrary;
class UDMXEntity;
class UDMXEntityFixturePatch;
class UDMXEntityFixtureType;

class SCheckBox;
class SScrollBox;
class SDockTab;
enum class ECheckBoxState : uint8;


/** Widget to assign fixture patches to universes and their channels */
class SDMXFixturePatcher
	: public SCompoundWidget
	, public FSelfRegisteringEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(SDMXFixturePatcher)
		: _DMXEditor(nullptr)
	{}

		SLATE_ARGUMENT(TWeakPtr<FDMXEditor>, DMXEditor)

	SLATE_END_ARGS()

	/** Constructs this widget */
	void Construct(const FArguments& InArgs);

	/** Refreshes the whole view from properties, does not consider changes in the library */
	void RefreshFromProperties();

	/** Refreshes the whole view from the library, considers library changes */
	void RefreshFromLibrary();

protected:
	// Begin SWidget Interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	// ~End SWidget Interface

private:
	/** */
	TSharedRef<SWidget> CreateToolbar();

	/** Creates the settings menu */
	TSharedRef<SWidget> CreateSettingsMenu();

	/** Selects the fixture patch name display mode */
	void SelectFixturePatchNameDisplayMode(EDMXFixturePatcherNameDisplayMode DisplayMode);
	
	/** Returns true if the widget is currently using specified name display mode */
	bool IsUsingFixturePatchNameDisplayMode(EDMXFixturePatcherNameDisplayMode DisplayMode) const;

	/** Called when drag enters a channel */
	void OnDragEnterChannel(int32 UniverseID, int32 ChannelID, const FDragDropEvent& DragDropEvent);

	/** Called when drag dropped onto a channel */
	FReply OnDropOntoChannel(int32 UniverseID, int32 ChannelID, const FDragDropEvent& DragDropEvent);

	/** Returns a map of nodes being dragged along with their absolute channel offset from the Drag Drop Event */
	TMap<TSharedRef<FDMXFixturePatchNode>, int64> GetDraggedNodesToAbsoluteChannelOffsetMap(const FDragDropEvent& DragDropEvent) const;
	
	//~ Begin FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	// End of FEditorUndoClient

	/** Finds specified fixture patch in all universes */
	TSharedPtr<FDMXFixturePatchNode> FindPatchNode(const TWeakObjectPtr<UDMXEntityFixturePatch> FixturePatch) const;

	/** Returns first node with same fixture type */
	TSharedPtr<FDMXFixturePatchNode> FindPatchNodeOfType(UDMXEntityFixtureType* Type, const TSharedPtr<FDMXFixturePatchNode>& IgoredNode) const;

	/** Called when entities were added to or removed from a DMX Library */
	void OnEntitiesAddedOrRemoved(UDMXLibrary* DMXLibrary, TArray<UDMXEntity*> Entities);

	/** Called when a fixture type changed */
	void OnFixtureTypeChanged(const UDMXEntityFixtureType* FixtureType);

	/** Called when a fixture patch was selected */
	void OnFixturePatchSelectionChanged();

	/** Called when a universe was selected */
	void OnUniverseSelectionChanged();

	/** Selects a universe */
	void SelectUniverse(int32 NewUniverseID);

	/** Returns the selected universe */
	int32 GetSelectedUniverse() const;

	/** Shows the selected universe only */
	void ShowSelectedUniverse();

	/** Shows all universes */
	void ShowAllPatchedUniverses(bool bForceReconstructWidgets = false);

	/** Adds a universe widget */
	void AddUniverse(int32 UniverseID);

	/** Called when the display all universe option is turned on or off */
	void OnToggleDisplayAllUniverses(ECheckBoxState CheckboxState);

	/** Sets if the DMX Monitor feature is enabled */
	void SetDMXMonitorEnabled(bool bEnabled);

	/** Called when the monitor feature is turned on or off */
	void OnToggleDMXMonitorEnabled(ECheckBoxState CheckboxState);

	/** Returns true if the patcher allows for the selection of a single universe and show only that */
	bool IsUniverseSelectionEnabled() const;

	/** Returns true if the library has any ports */
	bool HasAnyPorts() const;

	/** Enum for states of refreshing required */
	enum class EDMXRefreshFixturePatcherState : uint8
	{
		RefreshFromProperties,
		RefreshFromLibrary,
		NoRefreshRequested
	};

	/** True when a refresh was requested */
	EDMXRefreshFixturePatcherState RefreshFixturePatchState = EDMXRefreshFixturePatcherState::NoRefreshRequested;

	/** If true updates selection once during tick and resets */
	int32 UniverseToSetNextTick = INDEX_NONE;

protected:
	/** Returns a tooltip for the whole widget, used to hint why patching is not possible */
	FText GetTooltipText() const;

protected:
	/** Creates a valid address given Universe, Starting Channel and Channel Span. Returns false if no valid Address can be found. */
	int32 ClampStartingChannel(int32 StartingChannel, int32 ChannelSpan) const;

	/** Returns the DMXLibrary or nullptr if not available */
	UDMXLibrary* GetDMXLibrary() const;

	/** Checkbox to determine if all patched Universes should be displayed */
	TSharedPtr<SCheckBox> ShowAllUniversesCheckBox;

	/** Checkbox to determine if the DMX Monitor functionality should be enabled */
	TSharedPtr<SCheckBox> EnableDMXMonitorCheckBox;

	/** Horizontal scrollbox containing all the patched universes */
	TSharedPtr<SScrollBox> PatchedUniverseHorizontalScrollBox;

	/** Vertical scrollbox containing all the patched universes */
	TSharedPtr<SScrollBox> PatchedUniverseVerticalScrollBox;

	/** Universe widgets by ID */
	TMap<int32, TSharedPtr<SDMXPatchedUniverse>> PatchedUniversesByID;

	/** Shared data for fixture patch editors */
	TSharedPtr<FDMXFixturePatchSharedData> SharedData;

	/** The owning editor */
	TWeakPtr<FDMXEditor> DMXEditorPtr;
};
