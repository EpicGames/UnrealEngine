// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXPatchedUniverse.h"

#include "Algo/ForEach.h"
#include "Algo/RemoveIf.h"
#include "Commands/DMXEditorCommands.h"
#include "DMXEditor.h"
#include "DMXFixturePatchEditorDefinitions.h"
#include "DMXFixturePatchNode.h"
#include "DMXFixturePatchSharedData.h"
#include "DragDrop/DMXEntityFixturePatchDragDropOp.h"
#include "FixturePatchAutoAssignUtility.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "IO/DMXInputPort.h"
#include "IO/DMXOutputPort.h"
#include "IO/DMXPortManager.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXLibrary.h"
#include "ScopedTransaction.h"
#include "SDMXChannelConnector.h"
#include "SDMXFixturePatchFragment.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SCompoundWidget.h"


#define LOCTEXT_NAMESPACE "SDMXPatchedUniverse"

namespace UE::DMXEditor::FixturePatchEditor::Private
{
	/** Helper to raise pre and post edit change events on an array of fixture patches */
	class FScopedEditChangeFixturePatches
	{
	public:
		FScopedEditChangeFixturePatches(TArray<UDMXEntityFixturePatch*> InFixturePatches)
			: FixturePatches(InFixturePatches)
		{
			for (UDMXEntityFixturePatch* Patch : FixturePatches)
			{
				Patch->PreEditChange(nullptr);
			}
		}

		~FScopedEditChangeFixturePatches()
		{
			for (UDMXEntityFixturePatch* Patch : FixturePatches)
			{
				Patch->PostEditChange();
			}
		}

	private:
		TArray<UDMXEntityFixturePatch*> FixturePatches;
	};
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SDMXPatchedUniverse::Construct(const FArguments& InArgs)
{
	check(InArgs._UniverseID != INDEX_NONE);

	SetCanTick(true);

	UniverseID = InArgs._UniverseID;

	WeakDMXEditor = InArgs._DMXEditor;
	OnDragEnterChannel = InArgs._OnDragEnterChannel;
	OnDragLeaveChannel = InArgs._OnDragLeaveChannel;
	OnDropOntoChannel = InArgs._OnDropOntoChannel;

	const TSharedPtr<FDMXEditor> DMXEditor = WeakDMXEditor.Pin();
	if (!DMXEditor.IsValid())
	{
		return;
	}
	SharedData = DMXEditor->GetFixturePatchSharedData();
	check(SharedData.IsValid());

	ChildSlot
	[
		SNew (SVerticalBox)

		+ SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.AutoHeight()
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SAssignNew(UniverseName, SBorder)
				.Padding(4.f, 2.f)
				.VAlign(VAlign_Top)
				.HAlign(HAlign_Fill)
				.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 0.3f))
				.ToolTipText(FText::Format(LOCTEXT("UniverseListCategoryTooltip", "Patches assigned to Universe {0}"), UniverseID))
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
					.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
					.Text(this, &SDMXPatchedUniverse::GetHeaderText)
				]
			]

			+ SOverlay::Slot()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("Graph.Node.DevelopmentBanner"))
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.Visibility(this, &SDMXPatchedUniverse::GetPatchedUniverseReachabilityBannerVisibility)
			]
		]
			
		+ SVerticalBox::Slot()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Fill)		
		[					
			SNew(SBox)
			.HAlign(HAlign_Center)
			.MinDesiredWidth(920.0f) // Avoids a slate issue, where SScaleBox ignores border size hence scales			
			.Padding(FMargin(4.0f, 4.0f, 4.0f, 8.0f))
			[
				SAssignNew(Grid, SGridPanel)										
			]
		]
	];

	CreateChannelConnectors();

	SetUniverseIDInternal(UniverseID);

	UpdateZOrderOfNodes();
	UpdatePatchedUniverseReachability();

	RegisterCommands();

	// Handle external changes
	UDMXEntityFixturePatch::GetOnFixturePatchChanged().AddSP(this, &SDMXPatchedUniverse::OnFixturePatchChanged);
	SharedData->OnFixturePatchSelectionChanged.AddSP(this, &SDMXPatchedUniverse::OnSelectionChanged);
	FDMXPortManager::Get().OnPortsChanged.AddSP(this, &SDMXPatchedUniverse::UpdatePatchedUniverseReachability);

	UDMXProtocolSettings* ProtocolSettings = GetMutableDefault<UDMXProtocolSettings>();
	ProtocolSettings->GetOnSetReceiveDMXEnabled().AddSP(this, &SDMXPatchedUniverse::OnReceiveDMXEnabledChanged);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SDMXPatchedUniverse::RequestRefresh()
{
	if (!RequestRefreshTimerHandle.IsValid())
	{
		RequestRefreshTimerHandle = GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateSP(this, &SDMXPatchedUniverse::RefreshInternal));
	}
}

void SDMXPatchedUniverse::SetMonitorInputsEnabled(bool bEnabled)
{
	bMonitorInputs = bEnabled;

	if (bEnabled)
	{
		// Don't show the channel overlay of the fragment, so only the channel connectors display the values
		for (const TSharedRef<SDMXFixturePatchFragment>& Fragment : FixturePatchFragmentWidgets)
		{
			Fragment->SetChannelVisibility(EVisibility::Hidden);
		}
	}
	else
	{
		// Show the channel overlay of the fragment, so the fragments show their hovered state
		for (const TSharedRef<SDMXFixturePatchFragment>& Fragment : FixturePatchFragmentWidgets)
		{
			Fragment->SetChannelVisibility(EVisibility::Visible);
		}

		ResetMonitor();
	}
}

void SDMXPatchedUniverse::CreateChannelConnectors()
{
	check(Grid.IsValid());

	for (int32 ChannelIndex = 0; ChannelIndex < DMX_UNIVERSE_SIZE; ++ChannelIndex)
	{
		int32 Column = ChannelIndex % FDMXChannelGridSpecs::NumColumns;
		int32 Row = ChannelIndex / FDMXChannelGridSpecs::NumColumns;
		int32 Channel = ChannelIndex + FDMXChannelGridSpecs::ChannelIDOffset;

		TSharedRef<SDMXChannelConnector> ChannelPatchWidget =
			SNew(SDMXChannelConnector, WeakDMXEditor.Pin())
			.ChannelID(Channel)
			.Value(0)
			.OnHovered(this, &SDMXPatchedUniverse::HandleOnChannelHovered, Channel)
			.OnUnhovered(this, &SDMXPatchedUniverse::HandleOnChannelUnhovered, Channel)
			.OnMouseButtonDownOnChannel(this, &SDMXPatchedUniverse::HandleOnMouseButtonDownOnChannel)
			.OnMouseButtonUpOnChannel(this, &SDMXPatchedUniverse::HandleOnMouseButtonUpOnChannel)
			.OnDragDetectedOnChannel(this, &SDMXPatchedUniverse::HandleOnDragDetectedOnChannel)
			.OnDragEnterChannel(this, &SDMXPatchedUniverse::HandleDragEnterChannel)
			.OnDragLeaveChannel(this, &SDMXPatchedUniverse::HandleDragLeaveChannel)
			.OnDropOntoChannel(this, &SDMXPatchedUniverse::HandleDropOntoChannel);

		ChannelConnectors.Add(ChannelPatchWidget);

		Grid->AddSlot(Column, Row)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Layer(0)
			[			
				ChannelPatchWidget
			];
	}
}

bool SDMXPatchedUniverse::FindOrAdd(const TSharedRef<FDMXFixturePatchNode>& Node)
{
	UDMXEntityFixturePatch* FixturePatch = Node->GetFixturePatch().Get();
	if (!FixturePatch)
	{
		return false;
	}

	PatchedNodes.AddUnique(Node);

	RequestRefresh();

	return true;
}

void SDMXPatchedUniverse::Remove(const TSharedRef<FDMXFixturePatchNode>& Node)
{
	if (PatchedNodes.Contains(Node))
	{
		PatchedNodes.RemoveSingle(Node);

		const TArray<TSharedRef<SDMXFixturePatchFragment>> CachedWidgets = FixturePatchFragmentWidgets;
		for (const TSharedRef<SDMXFixturePatchFragment>& Widget : CachedWidgets)
		{
			if (Widget->GetFixturePatchNode() == Node)
			{
				Grid->RemoveSlot(Widget);
				FixturePatchFragmentWidgets.Remove(Widget);
			}
		}
		RequestRefresh();
	}
}

bool SDMXPatchedUniverse::CanAssignFixturePatch(TWeakObjectPtr<UDMXEntityFixturePatch> FixturePatch) const
{
	if (FixturePatch.IsValid())
	{
		return CanAssignFixturePatch(FixturePatch, FixturePatch->GetStartingChannel());
	}

	return false;
}

bool SDMXPatchedUniverse::CanAssignFixturePatch(TWeakObjectPtr<UDMXEntityFixturePatch> FixturePatch, int32 StartingChannel) const
{
	// Test for a valid and patch that has an active mode
	FText InvalidReason;
	if (!FixturePatch.IsValid() ||
		!FixturePatch->IsValidEntity(InvalidReason) ||
		!FixturePatch->GetActiveMode())
	{
		return false;
	}

	int32 ChannelSpan = FixturePatch->GetChannelSpan();
	if (ChannelSpan == 0 || ChannelSpan > DMX_MAX_ADDRESS)
	{
		// Cannot patch a patch with 0 channel span or exceeding DMX_MAX_ADDRESS
		return false;
	}

	// Only fully valid channels are supported
	check(StartingChannel > 0);
	check(StartingChannel + ChannelSpan - 1 <= DMX_UNIVERSE_SIZE);

	// Test for overlapping nodes in this universe
	for (const TSharedRef<FDMXFixturePatchNode>& Node : PatchedNodes)
	{
		if (Node->GetFixturePatch() == FixturePatch)
		{
			continue;
		}

		if (Node->OccupiesChannels(StartingChannel, ChannelSpan))
		{
			return false;
		}
	}

	return true;
}

bool SDMXPatchedUniverse::CanAssignNode(const TSharedPtr<FDMXFixturePatchNode>& TestedNode, int32 StartingChannel) const
{
	check(TestedNode.IsValid());

	UDMXEntityFixturePatch* FixturePatch = TestedNode->GetFixturePatch().Get();

	return CanAssignFixturePatch(FixturePatch, StartingChannel);
}

TSharedPtr<FDMXFixturePatchNode> SDMXPatchedUniverse::FindPatchNode(TWeakObjectPtr<UDMXEntityFixturePatch> FixturePatch) const
{
	const TSharedRef<FDMXFixturePatchNode>* NodePtr = PatchedNodes.FindByPredicate([&FixturePatch](const TSharedPtr<FDMXFixturePatchNode>& Node)
		{
			if (Node->GetFixturePatch() == FixturePatch)
			{
				return true;
			}
			return false;
		});

	if (NodePtr)
	{
		return *NodePtr;
	}

	return nullptr;
}

TSharedPtr<FDMXFixturePatchNode> SDMXPatchedUniverse::FindPatchNodeOfType(UDMXEntityFixtureType* Type, const TSharedPtr<FDMXFixturePatchNode>& IgoredNode) const
{	
	if (!Type)
	{
		return nullptr;
	}

	for (const TSharedRef<FDMXFixturePatchNode>& PatchNode : PatchedNodes)
	{
		if (PatchNode == IgoredNode)
		{
			continue;
		}

		if (PatchNode->GetFixturePatch().IsValid() &&
			PatchNode->GetFixturePatch()->GetFixtureType() == Type)
		{
			return PatchNode;
		}
	}

	return nullptr;
}

void SDMXPatchedUniverse::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) 
{
	UpdateNodesHoveredState();

	if (bMonitorInputs)
	{
		UpdateMonitor();
	}
}

void SDMXPatchedUniverse::OnFixturePatchChanged(const UDMXEntityFixturePatch* FixturePatch)
{
	if (!FixturePatch || FixturePatch->GetParentLibrary() != GetDMXLibrary())
	{
		return;
	}

	const TSharedRef<FDMXFixturePatchNode>* NodePtr = PatchedNodes.FindByPredicate([FixturePatch](const TSharedPtr<FDMXFixturePatchNode>& Node)
		{
			return Node->GetFixturePatch() == FixturePatch;
		});

	if (NodePtr && (*NodePtr)->GetUniverseID() != UniverseID)
	{
		Remove(*NodePtr);
	}
	else if (FixturePatch->GetUniverseID() == UniverseID)
	{
		// Redraw all if the patch is in this Universe but not found
		SetUniverseIDInternal(UniverseID);
	}
}

void SDMXPatchedUniverse::OnReceiveDMXEnabledChanged(bool bNewEnabled)
{
	if (!bNewEnabled)
	{
		ResetMonitor();
	}
}

void SDMXPatchedUniverse::SetUniverseIDInternal(int32 NewUniverseID)
{
	if (NewUniverseID < 0)
	{
		return;
	}

	// Find patches in new universe
	UDMXLibrary* Library = GetDMXLibrary();
	if (Library)
	{
		// Unpatch all nodes
		TArray<TSharedRef<FDMXFixturePatchNode>> CachedPatchedNodes = PatchedNodes;
		for (const TSharedRef<FDMXFixturePatchNode>& Node : CachedPatchedNodes)
		{
			Remove(Node);
		}
		check(PatchedNodes.Num() == 0);

		// Update what to draw
		UniverseID = NewUniverseID;

		UpdatePatchedUniverseReachability();

		TArray<UDMXEntityFixturePatch*> PatchesInUniverse;
		Library->ForEachEntityOfType<UDMXEntityFixturePatch>([&](UDMXEntityFixturePatch* Patch)
			{
				const bool bResidesInThisUniverse = Patch->GetUniverseID() == UniverseID;
				const bool bHasValidChannelSpan = Patch->GetChannelSpan() > 0 && Patch->GetChannelSpan() < DMX_MAX_ADDRESS;
				if (bResidesInThisUniverse && bHasValidChannelSpan)
				{
					PatchesInUniverse.Add(Patch);
				}
			});

		PatchesInUniverse.Sort([](const UDMXEntityFixturePatch& FixturePatchA, const UDMXEntityFixturePatch& FixturePatchB)
			{
				return FixturePatchA.GetEndingChannel() <= FixturePatchB.GetEndingChannel();
			});

		// Add fixture patches to the grid
		for (UDMXEntityFixturePatch* FixturePatch : PatchesInUniverse)
		{
			TSharedPtr<FDMXFixturePatchNode> Node = FindPatchNode(FixturePatch);
			if (!Node.IsValid() && IsValid(FixturePatch))
			{
				Node = FDMXFixturePatchNode::Create(WeakDMXEditor, FixturePatch);
			}

			if (Node.IsValid())
			{
				FindOrAdd(Node.ToSharedRef());
			}
		}

		// Update the channel connectors' Universe ID
		for (const TSharedRef<SDMXChannelConnector>& Connector : ChannelConnectors)
		{
			Connector->SetUniverseID(NewUniverseID);
		}

		RequestRefresh();
	}
}

void SDMXPatchedUniverse::RefreshInternal()
{
	RequestRefreshTimerHandle.Invalidate();

	UpdateZOrderOfNodes();

	// Remove old widgets from the grid
	for (const TSharedRef<SDMXFixturePatchFragment>& OldWidget : FixturePatchFragmentWidgets)
	{
		Grid->RemoveSlot(OldWidget);
	}
	FixturePatchFragmentWidgets.Reset();

	// Create Groups of Nodes with same Universe/Address
	TMap<int32, TArray<TSharedPtr<FDMXFixturePatchNode>>> AddressToNodeGroupMap;
	for (const TSharedRef<FDMXFixturePatchNode>& Node : PatchedNodes)
	{
		if (Node->GetChannelSpan() == 0 || Node->GetChannelSpan() > DMX_MAX_ADDRESS)
		{
			// Ignore nodes without or with excess channelspan, they're not displayed in UI either
			continue;
		}

		const int32 Address = Node->GetStartingChannel();
		AddressToNodeGroupMap.FindOrAdd(Address).Add(Node);
	}

	// Generate new widgets
	for (const TSharedRef<FDMXFixturePatchNode>& Node : PatchedNodes)
	{
		if (const TArray<TSharedPtr<FDMXFixturePatchNode>>* const NodeGroup = AddressToNodeGroupMap.Find(Node->GetStartingChannel()))
		{
			FixturePatchFragmentWidgets.Append(Node->GenerateWidgets(StaticCastSharedRef<SDMXPatchedUniverse>(AsShared()), *NodeGroup));
		}
	}

	// Add widgets to the grid
	for (const TSharedRef<SDMXFixturePatchFragment>& NewWidget : FixturePatchFragmentWidgets)
	{
		Grid->AddSlot(NewWidget->Column, NewWidget->Row)
			.ColumnSpan(NewWidget->ColumnSpan)
			.Layer(1)
			[
				NewWidget
			];
	}
}

void SDMXPatchedUniverse::UpdateNodesHoveredState()
{
	const int32 HoveredChannelID = HoveredChannel.IsValid() ? HoveredChannel->GetChannelID() + 1 : -1;
	for (const TSharedRef<FDMXFixturePatchNode>& Node : PatchedNodes)
	{
		Node->SetIsHovered(false);

		if (HoveredChannelID > 0)
		{
			constexpr int32 ChannelSpan = 1;
			if (Node->OccupiesChannels(HoveredChannelID, ChannelSpan))
			{
				Node->SetIsHovered(true);
			}
		}
	}
}

void SDMXPatchedUniverse::UpdateMonitor()
{
	UDMXLibrary* DMXLibrary = GetDMXLibrary();
	const UDMXProtocolSettings* ProtocolSettings = GetDefault<UDMXProtocolSettings>();
	if (!DMXLibrary || !ProtocolSettings || !ProtocolSettings->IsReceiveDMXEnabled())
	{
		return;
	}

	const TSet<FDMXInputPortSharedRef>& InputPorts = DMXLibrary->GetInputPorts();
	for (const FDMXInputPortSharedRef& InputPort : InputPorts)
	{
		FDMXSignalSharedPtr Signal;
		InputPort->GameThreadGetDMXSignal(UniverseID, Signal);
		if (Signal.IsValid())
		{
			for (int32 DataIndex = 0; DataIndex < Signal->ChannelData.Num(); DataIndex++)
			{
				if (ensureMsgf(ChannelConnectors.IsValidIndex(DataIndex), TEXT("Missing Channel to display value of Channel %i"), DataIndex))
				{
					ChannelConnectors[DataIndex]->SetValue(Signal->ChannelData[DataIndex]);
				}
			}
		}
	}
}

void SDMXPatchedUniverse::ResetMonitor()
{
	for (const TSharedRef<SDMXChannelConnector>& Channel : ChannelConnectors)
	{
		Channel->SetValue(0);
	}
}


void SDMXPatchedUniverse::HandleOnChannelHovered(int32 Channel)
{
	if (ChannelConnectors.IsValidIndex(Channel - 1))
	{
		HoveredChannel = ChannelConnectors[Channel - 1];
		LastHoveredChannel = Channel;
	}
}

void SDMXPatchedUniverse::HandleOnChannelUnhovered(int32 Channel)
{
	if (ChannelConnectors.IsValidIndex(Channel - 1) && 
		HoveredChannel == ChannelConnectors[Channel - 1])
	{
		HoveredChannel.Reset();
	}
}

FReply SDMXPatchedUniverse::HandleOnMouseButtonDownOnChannel(uint32 Channel, const FPointerEvent& PointerEvent)
{
	if (PointerEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		// Get valid selection, sorted by Universe/Address
		TArray<UDMXEntityFixturePatch*> Selection;
		Algo::TransformIf(SharedData->GetSelectedFixturePatches(), Selection,
			[](TWeakObjectPtr<UDMXEntityFixturePatch> WeakFixturePatch) { return WeakFixturePatch.IsValid(); },
			[](TWeakObjectPtr<UDMXEntityFixturePatch> WeakFixturePatch) { return WeakFixturePatch.Get(); }
		);
		Algo::StableSortBy(Selection, [](UDMXEntityFixturePatch* FixturePatch)
			{
				return FixturePatch->GetUniverseID() * DMX_MAX_ADDRESS + FixturePatch->GetStartingChannel();
			});

		const TArray<UDMXEntityFixturePatch*> FixturePatchesOnChannel = GetFixturePatchesOnChannel(Channel);
		UDMXEntityFixturePatch* ClickedPatch = GetTopmostFixturePatchOnChannel(Channel);

		// Shift Select
		if (FSlateApplication::Get().GetModifierKeys().IsShiftDown() &&
			!Selection.IsEmpty())
		{
			ShiftSelectAnchorPatch = ShiftSelectAnchorPatch.IsValid() ? ShiftSelectAnchorPatch : Selection[0];

			UDMXLibrary* DMXLibrary = ShiftSelectAnchorPatch->GetParentLibrary();
			if (!DMXLibrary)
			{
				return FReply::Handled();
			}
			TArray<UDMXEntityFixturePatch*> FixturePatches = DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>();
			const int64 AbsoluteClickedChannel = UniverseID * DMX_MAX_ADDRESS + Channel;
			const int64 ShiftSelectAbsoluteStartingChannel = ShiftSelectAnchorPatch->GetUniverseID() * DMX_MAX_ADDRESS + ShiftSelectAnchorPatch->GetStartingChannel();
			const int64 ShiftSelectAbsoluteEndingChannel = ShiftSelectAnchorPatch->GetUniverseID() * DMX_MAX_ADDRESS + ShiftSelectAnchorPatch->GetEndingChannel();

			const TRange<int64> SelectedRange(FMath::Min(ShiftSelectAbsoluteStartingChannel, AbsoluteClickedChannel), FMath::Max(ShiftSelectAbsoluteStartingChannel, AbsoluteClickedChannel + 1));

			TArray<TWeakObjectPtr<UDMXEntityFixturePatch>> ShiftSelection;
			Algo::TransformIf(FixturePatches, ShiftSelection,
				[&SelectedRange, Channel, this](UDMXEntityFixturePatch* FixturePatch)
				{
					const int64 PatchAbsoluteStartingChannel = FixturePatch->GetUniverseID() * DMX_MAX_ADDRESS + FixturePatch->GetStartingChannel();
					const int64 PatchAbsoluteEndingChannel = FixturePatch->GetUniverseID() * DMX_MAX_ADDRESS + FixturePatch->GetEndingChannel();
					const TRange<int64> PatchRange(PatchAbsoluteStartingChannel, PatchAbsoluteEndingChannel + 1);
					return
						SelectedRange.Overlaps(PatchRange);
				},
				[](UDMXEntityFixturePatch* FixturePatch)
				{
					return FixturePatch;
				});

			SharedData->SelectFixturePatches(ShiftSelection);

			return FReply::Handled();
		}
		ShiftSelectAnchorPatch.Reset();

		// Ctrl-Select
		if (FSlateApplication::Get().GetModifierKeys().IsControlDown() &&
			ClickedPatch)
		{
			if (const bool bUnselect = Selection.Num() > 1 && Selection.Contains(ClickedPatch))
			{
				TArray<TWeakObjectPtr<UDMXEntityFixturePatch>> WeakSelection = SharedData->GetSelectedFixturePatches();
				WeakSelection.Remove(ClickedPatch);
				SharedData->SelectFixturePatches(WeakSelection);
			}
			else
			{
				SharedData->AddFixturePatchToSelection(ClickedPatch);
			}

			return FReply::Handled();
		}

		// Normal Select and Detect Drag
		if (ClickedPatch &&
			ensureMsgf(ChannelConnectors.IsValidIndex(Channel - 1), TEXT("Trying to drag Fixture Patch, but the dragged channel is not valid.")))
		{
			ShiftSelectAnchorPatch.Reset();

			// Replace selection with clicked patch if it wasn't selected
			if (!Selection.Contains(ClickedPatch))
			{
				SharedData->SelectFixturePatch(ClickedPatch);
			}

			return FReply::Handled();
		}
	}

	// Select this universe
	SharedData->SelectUniverse(UniverseID);

	return FReply::Unhandled();
}

FReply SDMXPatchedUniverse::HandleOnMouseButtonUpOnChannel(uint32 Channel, const FPointerEvent& PointerEvent)
{
	if (PointerEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		UDMXEntityFixturePatch* Patch = GetTopmostFixturePatchOnChannel(Channel);
		if (!Patch)
		{
			return FReply::Unhandled();
		}

		// Normal Select
		if (!FSlateApplication::Get().GetModifierKeys().IsControlDown() && !FSlateApplication::Get().GetModifierKeys().IsShiftDown())
		{
			UDMXEntityFixturePatch* ClickedPatch = GetTopmostFixturePatchOnChannel(Channel);
			if (ClickedPatch)
			{
				SharedData->SelectFixturePatch(ClickedPatch);
			}
		}
	}
	else if (PointerEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		// Select the patchh under the cursor if the current selection is not under the cursor
		UDMXEntityFixturePatch* FixturePatchesOnChannel = GetTopmostFixturePatchOnChannel(Channel);
		const TArray<TWeakObjectPtr<UDMXEntityFixturePatch>>& SelectedFixturePatches = SharedData->GetSelectedFixturePatches();
		if (FixturePatchesOnChannel && !SelectedFixturePatches.Contains(FixturePatchesOnChannel))
		{
			SharedData->SelectFixturePatch(FixturePatchesOnChannel);
		}

		// Open context menu
		const FVector2D& ContextMenuSummonLocation = PointerEvent.GetScreenSpacePosition();

		TSharedRef<SWidget> MenuContent = CreateContextMenu(Channel);
		FWidgetPath WidgetPath = PointerEvent.GetEventPath() ? *PointerEvent.GetEventPath() : FWidgetPath();
		FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuContent, ContextMenuSummonLocation, FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
	}

	return FReply::Unhandled();
}

FReply SDMXPatchedUniverse::HandleOnDragDetectedOnChannel(uint32 Channel, const FPointerEvent& PointerEvent)
{
	TArray<TWeakObjectPtr<UDMXEntityFixturePatch>> SelectedFixturePatches = SharedData->GetSelectedFixturePatches();

	// Only valid patches, sorted by Universe/Address
	const int32 NumValidPatches = Algo::RemoveIf(SelectedFixturePatches, [this](TWeakObjectPtr<UDMXEntityFixturePatch> FixturePatch) 
		{ 
			return !FixturePatch.IsValid();
		});
	SelectedFixturePatches.SetNum(NumValidPatches);
	Algo::StableSortBy(SelectedFixturePatches, [](TWeakObjectPtr<UDMXEntityFixturePatch> FixturePatch)
		{
			return (int64)FixturePatch->GetUniverseID() * DMX_MAX_ADDRESS + FixturePatch->GetStartingChannel();
		}
	);

	if (SelectedFixturePatches.IsEmpty())
	{
		return FReply::Unhandled();
	}

	TMap<TWeakObjectPtr<UDMXEntityFixturePatch>, int64> FixturePatchToAbsoluteChannelOffsetMap;
	for (TWeakObjectPtr<UDMXEntityFixturePatch> FixturePatch : SelectedFixturePatches)
	{
		if (!FixturePatch.IsValid())
		{
			continue;
		}

		FixturePatch->Modify();

		const int64 AbsoluteOffset = UniverseID * DMX_MAX_ADDRESS + Channel - (FixturePatch->GetUniverseID() * DMX_MAX_ADDRESS + FixturePatch->GetStartingChannel());
		FixturePatchToAbsoluteChannelOffsetMap.Add(FixturePatch, AbsoluteOffset);
	}

	const TSharedRef<FDMXEntityFixturePatchDragDropOperation> DragDropOp = MakeShared<FDMXEntityFixturePatchDragDropOperation>(FixturePatchToAbsoluteChannelOffsetMap);
	return FReply::Handled().BeginDragDrop(DragDropOp);
}

void SDMXPatchedUniverse::HandleDragEnterChannel(uint32 Channel, const FDragDropEvent& DragDropEvent)
{
	OnDragEnterChannel.ExecuteIfBound(UniverseID, Channel, DragDropEvent);
}

void SDMXPatchedUniverse::HandleDragLeaveChannel(uint32 Channel, const FDragDropEvent& DragDropEvent)
{
	OnDragLeaveChannel.ExecuteIfBound(UniverseID, Channel, DragDropEvent);
}

FReply SDMXPatchedUniverse::HandleDropOntoChannel(uint32 Channel, const FDragDropEvent& DragDropEvent)
{
	return OnDropOntoChannel.Execute(UniverseID, Channel, DragDropEvent);
}

FText SDMXPatchedUniverse::GetHeaderText() const
{
	switch (PatchedUniverseReachability)
	{
		case EDMXPatchedUniverseReachability::Reachable:
			return FText::Format(LOCTEXT("UniverseIDLabel", "Universe {0}"), UniverseID);

		case EDMXPatchedUniverseReachability::UnreachableForInputPorts:
			return FText::Format(LOCTEXT("UnreachableForInputPorts", "Universe {0} - Unreachable by Input Ports"), UniverseID);

		case EDMXPatchedUniverseReachability::UnreachableForOutputPorts:
			return FText::Format(LOCTEXT("UnreachableForOutputPorts", "Universe {0} - Unreachable by Output Ports"), UniverseID);

		case EDMXPatchedUniverseReachability::UnreachableForInputAndOutputPorts:
			return FText::Format(LOCTEXT("UnreachableForInputAndOutputPorts", "Universe {0} - Unreachable by Input and Output Ports"), UniverseID);

		default:
			// Unhandled enum value
			checkNoEntry();
	}

	return FText::GetEmpty();
}

void SDMXPatchedUniverse::UpdateZOrderOfNodes()
{
	if (PatchedNodes.Num() == 0)
	{
		return;
	}

	TArray<TSharedRef<FDMXFixturePatchNode>> Nodes = PatchedNodes;
	// Sort by ZOrder descending
	Nodes.StableSort([](const TSharedRef<FDMXFixturePatchNode>& NodeA, const TSharedRef<FDMXFixturePatchNode>& NodeB)
		{
			if (NodeA->GetStartingChannel() > NodeB->GetStartingChannel())
			{
				return true;
			}
			else if (NodeA->GetStartingChannel() == NodeB->GetStartingChannel())
			{
				const bool bNodeASelected = NodeA->IsSelected();
				if (bNodeASelected)
				{
					return
						!NodeB->IsSelected() ||
						NodeA->GetZOrder() > NodeB->GetZOrder();
				}
				else
				{
					return
						!NodeB->IsSelected() &&
						NodeA->GetZOrder() > NodeB->GetZOrder();
				}
			}
			return false;
		});

	int32 ZOrder = Nodes.Num();
	for (const TSharedRef<FDMXFixturePatchNode>& Node : Nodes)
	{
		Node->SetZOrder(ZOrder);
		--ZOrder;
	}
}

EVisibility SDMXPatchedUniverse::GetPatchedUniverseReachabilityBannerVisibility() const
{
	if (PatchedUniverseReachability == EDMXPatchedUniverseReachability::Reachable)
	{
		return EVisibility::Hidden;
	}

	return EVisibility::Visible;
}

void SDMXPatchedUniverse::UpdatePatchedUniverseReachability()
{
	PatchedUniverseReachability = EDMXPatchedUniverseReachability::Reachable;

	UDMXLibrary* Library = GetDMXLibrary();
	if (Library)
	{
		bool bReachableForAnyInput = false;
		for (const FDMXInputPortSharedRef& InputPort : Library->GetInputPorts())
		{
			if (InputPort->IsLocalUniverseInPortRange(UniverseID))
			{
				bReachableForAnyInput = true;
				break;
			}
		}

		bool bReachableForAnyOutput = false;
		for (const FDMXOutputPortSharedRef& OutputPort : Library->GetOutputPorts())
		{
			if (OutputPort->IsLocalUniverseInPortRange(UniverseID))
			{
				bReachableForAnyOutput = true;
				break;
			}
		}

		if (!bReachableForAnyInput && !bReachableForAnyOutput)
		{
			PatchedUniverseReachability = EDMXPatchedUniverseReachability::UnreachableForInputAndOutputPorts;
		}
		else if (!bReachableForAnyOutput)
		{
			PatchedUniverseReachability = EDMXPatchedUniverseReachability::UnreachableForOutputPorts;
		}
		else if (!bReachableForAnyInput)
		{
			PatchedUniverseReachability = EDMXPatchedUniverseReachability::UnreachableForInputPorts;
		}
	}
}

void SDMXPatchedUniverse::OnSelectionChanged()
{
	const TArray<TWeakObjectPtr<UDMXEntityFixturePatch>> FixturePatches = SharedData->GetSelectedFixturePatches();
	const bool bSelectionChanged = [&FixturePatches, this]()
		{
			for (const TSharedRef<FDMXFixturePatchNode>& Node : PatchedNodes)
			{
				if (Node->IsSelected() ^ FixturePatches.Contains(Node->GetFixturePatch()))
				{
					return true;
				}
			}
			return false;
		}();
	if (bSelectionChanged)
	{
		RequestRefresh();
	}
}

void SDMXPatchedUniverse::AutoAssignFixturePatches(UE::DMXEditor::AutoAssign::EAutoAssignMode AutoAssignMode)
{
	const TSharedPtr<FDMXEditor> DMXEditor = WeakDMXEditor.Pin();
	TArray<UDMXEntityFixturePatch*> FixturePatchesToAutoAssign;
	if (!DMXEditor.IsValid() || !GetSelectedFixturePatches(FixturePatchesToAutoAssign))
	{
		return;
	}

	const FText TransactionText = FText::Format(LOCTEXT("AutoAssignTransaction", "Auto-Assign Fixture {0}|plural(one=Patch, other=Patches)"), FixturePatchesToAutoAssign.Num());
	const FScopedTransaction AutoAssignTransaction(TransactionText);

	UE::DMXEditor::FixturePatchEditor::Private::FScopedEditChangeFixturePatches ScopedEditChange(FixturePatchesToAutoAssign);
	const int32 AssignedToUniverse = UE::DMXEditor::AutoAssign::FAutoAssignUtility::AutoAssign(AutoAssignMode, DMXEditor.ToSharedRef(), FixturePatchesToAutoAssign, UniverseID, LastHoveredChannel);

	SharedData->SelectUniverse(AssignedToUniverse);

	RequestRefresh();
}

void SDMXPatchedUniverse::AssignFixturePatches()
{
	TArray<UDMXEntityFixturePatch*> FixturePatchesToAssign;
	if (!GetSelectedFixturePatches(FixturePatchesToAssign))
	{
		return;
	}

	const FText TransactionText = FText::Format(LOCTEXT("AssignTransaction", "Assign Fixture {0}|plural(one=Patch, other=Patches)"), FixturePatchesToAssign.Num());
	const FScopedTransaction AssignTransaction(TransactionText);

	UE::DMXEditor::FixturePatchEditor::Private::FScopedEditChangeFixturePatches ScopedEditChange(FixturePatchesToAssign);
	const int32 FirstPatchedUniverse = UE::DMXEditor::AutoAssign::FAutoAssignUtility::Assign(FixturePatchesToAssign, UniverseID, LastHoveredChannel);

	SharedData->SelectUniverse(FirstPatchedUniverse);
	
	RequestRefresh();
}

void SDMXPatchedUniverse::AlignFixturePatches()
{
	TArray<UDMXEntityFixturePatch*> FixturePatchesToAlign;
	if (!GetSelectedFixturePatches(FixturePatchesToAlign))
	{
		return;
	}

	const FText TransactionText = FText::Format(LOCTEXT("AlignTransaction", "Align Fixture {0}|plural(one=Patch, other=Patches)"), FixturePatchesToAlign.Num());
	const FScopedTransaction AlignTransaction(TransactionText);
	
	UE::DMXEditor::FixturePatchEditor::Private::FScopedEditChangeFixturePatches ScopedEditChange(FixturePatchesToAlign);
	UE::DMXEditor::AutoAssign::FAutoAssignUtility::Align(FixturePatchesToAlign);

	RequestRefresh();
}

void SDMXPatchedUniverse::StackFixturePatches()
{
	TArray<UDMXEntityFixturePatch*> FixturePatchesToStack;
	if (!GetSelectedFixturePatches(FixturePatchesToStack))
	{
		return;
	}

	const FText TransactionText = FText::Format(LOCTEXT("StackTransaction", "Stack Fixture {0}|plural(one=Patch, other=Patches)"), FixturePatchesToStack.Num());
	const FScopedTransaction StackTransaction(TransactionText);

	UE::DMXEditor::FixturePatchEditor::Private::FScopedEditChangeFixturePatches ScopedEditChange(FixturePatchesToStack);
	UE::DMXEditor::AutoAssign::FAutoAssignUtility::Stack(FixturePatchesToStack);

	RequestRefresh();
}

void SDMXPatchedUniverse::SpreadFixturePatchesOverUniverses()
{
	TArray<UDMXEntityFixturePatch*> FixturePatchesToSpread;
	if (!GetSelectedFixturePatches(FixturePatchesToSpread))
	{
		return;
	}

	const FText TransactionText = FText::Format(LOCTEXT("SpreadOverUniverseTransaction", "Spread Fixture {0}|plural(one=Patch, other=Patches) over Universes"), FixturePatchesToSpread.Num());
	const FScopedTransaction SpreadOverUniverseTransaction(TransactionText);

	UE::DMXEditor::FixturePatchEditor::Private::FScopedEditChangeFixturePatches ScopedEditChange(FixturePatchesToSpread);
	UE::DMXEditor::AutoAssign::FAutoAssignUtility::SpreadOverUniverses(FixturePatchesToSpread);

	RequestRefresh();
}

bool SDMXPatchedUniverse::GetSelectedFixturePatches(TArray<UDMXEntityFixturePatch*>& OutFixturePatchArray) const
{
	OutFixturePatchArray.Reset();

	Algo::TransformIf(SharedData->GetSelectedFixturePatches(), OutFixturePatchArray,
		[](const TWeakObjectPtr<UDMXEntityFixturePatch>& WeakFixturePatch) { return WeakFixturePatch.IsValid(); },
		[](const TWeakObjectPtr<UDMXEntityFixturePatch>& WeakFixturePatch) { return WeakFixturePatch.Get(); });

	return !OutFixturePatchArray.IsEmpty();
}

bool SDMXPatchedUniverse::DoesDMXLibraryHaveReachableUniverses() const
{
	if (UDMXLibrary* DMXLibrary = GetDMXLibrary())
	{
		return !DMXLibrary->GetInputPorts().IsEmpty() && !DMXLibrary->GetOutputPorts().IsEmpty();
	}
	return false;
}

UDMXEntityFixturePatch* SDMXPatchedUniverse::GetTopmostFixturePatchOnChannel(uint32 Channel) const
{
	TArray<TSharedPtr<FDMXFixturePatchNode>> Nodes(PatchedNodes);
	Nodes.RemoveAll([Channel](const TSharedPtr<FDMXFixturePatchNode>& Node)
		{
			if (UDMXEntityFixturePatch* FixturePatch = Node->GetFixturePatch().Get())
			{
				const uint32 PatchStartingChannel = FixturePatch->GetStartingChannel();
				const TRange<uint32> FixturePatchRange(PatchStartingChannel, PatchStartingChannel + FixturePatch->GetChannelSpan());

				return !FixturePatchRange.Contains(Channel);
			}
			return true;
		});

	Nodes.Sort([](const TSharedPtr<FDMXFixturePatchNode>& NodeA, const TSharedPtr<FDMXFixturePatchNode>& NodeB)
		{
			return NodeA->GetZOrder() > NodeB->GetZOrder();
		});

	if (Nodes.IsEmpty())
	{
		return nullptr;
	}
	else
	{
		return Nodes[0]->GetFixturePatch().Get();
	}
}

TArray<UDMXEntityFixturePatch*> SDMXPatchedUniverse::GetFixturePatchesOnChannel(uint32 Channel) const
{
	TArray<TSharedRef<FDMXFixturePatchNode>> Nodes(PatchedNodes);
	Nodes.RemoveAll([Channel](const TSharedRef<FDMXFixturePatchNode>& Node)
		{
			if (UDMXEntityFixturePatch* FixturePatch = Node->GetFixturePatch().Get())
			{
				const uint32 PatchStartingChannel = FixturePatch->GetStartingChannel();
				const TRange<uint32> FixturePatchRange(PatchStartingChannel, PatchStartingChannel + FixturePatch->GetChannelSpan());

				return !FixturePatchRange.Contains(Channel);
			}
			return true;
		});

	TArray<UDMXEntityFixturePatch*> Result;
	for (const TSharedRef<FDMXFixturePatchNode>& Node : Nodes)
	{
		if (UDMXEntityFixturePatch* FixturePatch = Node->GetFixturePatch().Get())
		{
			Result.Add(FixturePatch);
		}
	}

	return Result;
}

UDMXLibrary* SDMXPatchedUniverse::GetDMXLibrary() const
{
	if (TSharedPtr<FDMXEditor> DMXEditor = WeakDMXEditor.Pin())
	{
		return DMXEditor->GetDMXLibrary();
	}
	return nullptr;
}

void SDMXPatchedUniverse::RegisterCommands()
{
	if (CommandList.IsValid())
	{
		return;
	}

	// listen to common editor shortcuts for copy/paste etc
	CommandList = MakeShared<FUICommandList>();

	using namespace UE::DMXEditor::AutoAssign;
	CommandList->MapAction
	(
		FDMXEditorCommands::Get().AutoAssignSelectedUniverse,
		FExecuteAction::CreateSP(this, &SDMXPatchedUniverse::AutoAssignFixturePatches, EAutoAssignMode::SelectedUniverse)
	);
	CommandList->MapAction
	(
		FDMXEditorCommands::Get().Align,
		FExecuteAction::CreateSP(this, &SDMXPatchedUniverse::AlignFixturePatches)
	);
	CommandList->MapAction
	(
		FDMXEditorCommands::Get().Stack,
		FExecuteAction::CreateSP(this, &SDMXPatchedUniverse::StackFixturePatches)
	);
}

TSharedRef<SWidget> SDMXPatchedUniverse::CreateContextMenu(int32 Channel)
{
	UDMXLibrary* DMXLibrary = GetDMXLibrary();
	if (!DMXLibrary)
	{
		return SNullWidget::NullWidget;
	}

	constexpr bool bCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bCloseWindowAfterMenuSelection, CommandList);

	if (!SharedData->GetSelectedFixturePatches().IsEmpty())
	{
		using namespace UE::DMXEditor::AutoAssign;

		// Auto assign section
		MenuBuilder.BeginSection("AutoAssignSection", LOCTEXT("AutoAssignActionsSection", "Auto-Assign"));
		{
			MenuBuilder.AddMenuEntry(FDMXEditorCommands::Get().AutoAssignSelectedUniverse);

			const FText AutoAssignUnderMouseAssignLabel = FText::Format(LOCTEXT("AutoAssignUnderMouseAssignLabel", "Auto-Assign at {0}.{1}"), FText::FromString(FString::FromInt(UniverseID)), FText::FromString(FString::FromInt(LastHoveredChannel)));
			MenuBuilder.AddMenuEntry
			(
				AutoAssignUnderMouseAssignLabel,
				LOCTEXT("AutoAssignTooltip", "Auto-assigns selected patches to first consecutive range of free channels, starting from the current mouse position."),
				FSlateIcon(),
				FUIAction
				(
					FExecuteAction::CreateSP(this, &SDMXPatchedUniverse::AutoAssignFixturePatches, EAutoAssignMode::UserDefinedChannel),
					FCanExecuteAction::CreateLambda([this, SharedThis = AsShared()] { return !SharedData->GetSelectedFixturePatches().IsEmpty(); })
				),
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
		MenuBuilder.EndSection();

		// Assign section
		MenuBuilder.BeginSection("AssignSection", LOCTEXT("AssignSection", "Assign"));
		{
			const FText AssignLabel = FText::Format(LOCTEXT("AssignMenuLabel", "Assign to {0}.{1}"), FText::FromString(FString::FromInt(UniverseID)), FText::FromString(FString::FromInt(LastHoveredChannel)));
			MenuBuilder.AddMenuEntry
			(
				AssignLabel,
				LOCTEXT("AssignTooltip", "Assigns selected patches to the hovered channel"),
				FSlateIcon(),
				FUIAction
				(
					FExecuteAction::CreateSP(this, &SDMXPatchedUniverse::AssignFixturePatches),
					FCanExecuteAction::CreateLambda([this, SharedThis = AsShared()] { return !SharedData->GetSelectedFixturePatches().IsEmpty(); })
				),
				NAME_None,
				EUserInterfaceActionType::Button
			);
			MenuBuilder.AddMenuEntry(FDMXEditorCommands::Get().Align);
			MenuBuilder.AddMenuEntry(FDMXEditorCommands::Get().Stack);
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("SpreadOverUniversesLabel", "Spread over Universes"),
				LOCTEXT("SpreadOverUniversesTooltip", "Assigns each patch to its own Universe"),
				FSlateIcon(),
				FUIAction
				(
					FExecuteAction::CreateSP(this, &SDMXPatchedUniverse::SpreadFixturePatchesOverUniverses),
					FCanExecuteAction::CreateLambda([this, SharedThis = AsShared()] { return SharedData->GetSelectedFixturePatches().Num() > 1; })
				),
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
		MenuBuilder.EndSection();
	}

	TArray<UDMXEntityFixturePatch*> FixturePatchesOnChannel = GetFixturePatchesOnChannel(Channel);
	if (FixturePatchesOnChannel.Num() > 1)
	{
		// Select section
		MenuBuilder.BeginSection("SelectSection", LOCTEXT("SelectSection", "Select"));
		{
			// Select all
			{
				const FUIAction Action(
					FExecuteAction::CreateLambda([FixturePatchesOnChannel, this]()
						{
							if (SharedData.IsValid())
							{
								TArray<TWeakObjectPtr<UDMXEntityFixturePatch>> WeakFixturePatchesOnChannel;
								for (UDMXEntityFixturePatch* FixturePatch : FixturePatchesOnChannel)
								{
									WeakFixturePatchesOnChannel.Add(FixturePatch);
								}
								SharedData->SelectFixturePatches(WeakFixturePatchesOnChannel);
							}
						}));

				const FText SelectAllText = LOCTEXT("SelectAllFixturePatchesContextMenuEntry", "Select All");
				const TSharedRef<SWidget> Widget =
					SNew(STextBlock)
					.Text(SelectAllText);

				MenuBuilder.AddMenuEntry(Action, Widget);
			}

			MenuBuilder.AddSeparator();

			// Select specific patch
			{
				const TArray<UDMXEntityFixturePatch*> DMXFixturePatchesInLibrary = DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>();
				FixturePatchesOnChannel.Sort([&DMXFixturePatchesInLibrary](const UDMXEntityFixturePatch& FixturePatchA, const UDMXEntityFixturePatch& FixturePatchB)
					{
						return DMXFixturePatchesInLibrary.IndexOfByKey(&FixturePatchA) < DMXFixturePatchesInLibrary.IndexOfByKey(&FixturePatchB);
					});

				for (TWeakObjectPtr<UDMXEntityFixturePatch> WeakFixturePatch : FixturePatchesOnChannel)
				{
					const FUIAction Action(
						FExecuteAction::CreateLambda([this, WeakFixturePatch]()
							{
								if (SharedData.IsValid() && WeakFixturePatch.IsValid())
								{
									SharedData->SelectFixturePatch(WeakFixturePatch.Get());
								}
							})
					);

					const FText FixturePatchName = FText::FromString(WeakFixturePatch->Name);
					const TSharedRef<SWidget> Widget =
						SNew(STextBlock)
						.Text(FixturePatchName);

					MenuBuilder.AddMenuEntry(Action, Widget);
				}
			}
		}

		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
