// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SDMXChannelConnector.h"
#include "Library/DMXEntityFixtureType.h"

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "Engine/EngineTypes.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FDMXEditor;
class FDMXFixturePatchNode;
class FDMXFixturePatchSharedData;
class FDragDropEvent;
struct FTimerHandle;
class FUICommandList;
class SBorder;
class SDMXChannelConnector;
class SDMXFixturePatchFragment;
class SGridPanel;
class UDMXLibrary;
class UDMXEntityFixturePatch;
namespace UE::DMXEditor::AutoAssign { enum class EAutoAssignMode : uint8; }

enum class EDMXPatchedUniverseReachability
{
	Reachable,
	UnreachableForInputPorts,
	UnreachableForOutputPorts,
	UnreachableForInputAndOutputPorts
};


/** A universe with assigned patches */
class SDMXPatchedUniverse
	: public SCompoundWidget
{
	DECLARE_DELEGATE_ThreeParams(FOnDragOverChannel, int32 /** UniverseID */, int32 /** Channel */, const FDragDropEvent&);

	DECLARE_DELEGATE_RetVal_ThreeParams(FReply, FOnDropOntoChannel, int32 /** UniverseID */, int32 /** Channel */, const FDragDropEvent&);

public:
	SDMXPatchedUniverse()
		: PatchedUniverseReachability(EDMXPatchedUniverseReachability::UnreachableForInputAndOutputPorts)
	{}

	SLATE_BEGIN_ARGS(SDMXPatchedUniverse)
		: _UniverseID(0)
		, _DMXEditor(nullptr)
		, _OnDragEnterChannel()
		, _OnDragLeaveChannel()
		, _OnDropOntoChannel()
	{}
		SLATE_ARGUMENT(int32, UniverseID)

		SLATE_ARGUMENT(TWeakPtr<FDMXEditor>, DMXEditor)

		/** Called when drag enters a channel in this universe */
		SLATE_EVENT(FOnDragOverChannel, OnDragEnterChannel)

		/** Called when drag leaves a channel in this universe */
		SLATE_EVENT(FOnDragOverChannel, OnDragLeaveChannel)

		/** Called when dropped onto a channel in this universe */
		SLATE_EVENT(FOnDropOntoChannel, OnDropOntoChannel)

	SLATE_END_ARGS()

	/** Constructs this widget */
	void Construct(const FArguments& InArgs);

	/** Requests to refresh the Widget on the next tick */
	void RequestRefresh();

	/** Sets if this universe should monitor inputs */
	void SetMonitorInputsEnabled(bool bEnabled);

	/** Finds or adds a Node to this Universe */
	bool FindOrAdd(const TSharedRef<FDMXFixturePatchNode>& Node);

	/** Removes a Node from this Universe */
	void Remove(const TSharedRef<FDMXFixturePatchNode>& Node);

	/** Returns wether the patch can be patched to its current channels */
	bool CanAssignFixturePatch(TWeakObjectPtr<UDMXEntityFixturePatch> TestedPatch) const;

	/** Returns wether the patch can be patched to specified channel */
	bool CanAssignFixturePatch(TWeakObjectPtr<UDMXEntityFixturePatch> TestedPatch, int32 StartingChannel) const;

	/** Returns wether the node can be patched to specified channel */
	bool CanAssignNode(const TSharedPtr<FDMXFixturePatchNode>& TestedNode, int32 StartingChannel) const;

	/** Returns if the node is patched in the unvierse */
	TSharedPtr<FDMXFixturePatchNode> FindPatchNode(TWeakObjectPtr<UDMXEntityFixturePatch> FixturePatch) const;

	/** Returns first node with same fixture type as specified node */
	TSharedPtr<FDMXFixturePatchNode> FindPatchNodeOfType(UDMXEntityFixtureType* Type, const TSharedPtr<FDMXFixturePatchNode>& IgoredNode) const;

	/** Returns the ID of the universe */
	int32 GetUniverseID() const { return UniverseID; }

	/** Gets all nodes patched to this universe */
	const TArray<TSharedRef<FDMXFixturePatchNode>>& GetPatchedNodes() const { return PatchedNodes; }

protected:
	//~ Begin SWidget Interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget Interface

private:
	/** Called when a Fixture Patch changed */
	void OnFixturePatchChanged(const UDMXEntityFixturePatch* FixturePatch);

	/** Called when receive DMX was enabled or disabled */
	void OnReceiveDMXEnabledChanged(bool bNewEnabled);

	/** Shows the universe directly */
	void SetUniverseIDInternal(int32 NewUniverseID);

	/** Refreshes the widget directly */
	void RefreshInternal();

	/** Updates the hover state of all nodes */
	void UpdateNodesHoveredState();

	/** Updates the monitor */
	void UpdateMonitor();

	/** Resets the Monitor */
	void ResetMonitor();

	/** Creates a a new grid of channels */
	void CreateChannelConnectors();

	/** Returns the name of the universe displayed */
	FText GetHeaderText() const;
	
	/** Called when a channel is hovered */
	void HandleOnChannelHovered(int32 Channel);

	/** Called when a channel is unhovered */
	void HandleOnChannelUnhovered(int32 Channel);

	/** Handles when a mouse button was pressed on a Channel */
	FReply HandleOnMouseButtonDownOnChannel(uint32 Channel, const FPointerEvent& PointerEvent);

	/** Handles when a mouse button was released on a Channel */
	FReply HandleOnMouseButtonUpOnChannel(uint32 Channel, const FPointerEvent& PointerEvent);

	/** Handles when a Channel was dragged */
	FReply HandleOnDragDetectedOnChannel(uint32 Channel, const FPointerEvent& PointerEvent);

	/** Handles when drag enters a channel */
	void HandleDragEnterChannel(uint32 Channel, const FDragDropEvent& DragDropEvent);

	/** Handles when drag leaves a channel */
	void HandleDragLeaveChannel(uint32 Channel, const FDragDropEvent& DragDropEvent);

	/** Handles when drag dropped onto a channel */
	FReply HandleDropOntoChannel(uint32 Channel, const FDragDropEvent& DragDropEvent);

	/** Updates the ZOrder of all Nodes */
	void UpdateZOrderOfNodes();

	/** Returns wether the out of controllers' ranges banner should be visible */
	EVisibility GetPatchedUniverseReachabilityBannerVisibility() const;

	/** Updates bOutOfControllersRanges member */
	void UpdatePatchedUniverseReachability();

	/** Called when selection changed */
	void OnSelectionChanged();

	/** Auto assigns selected Fixture Patches */
	void AutoAssignFixturePatches(UE::DMXEditor::AutoAssign::EAutoAssignMode AutoAssignMode);

	/** Assign selected Fixture Patches */
	void AssignFixturePatches();

	/** Aligns selected Fixture Patches */
	void AlignFixturePatches();

	/** Stacks selected Fixture Patches */
	void StackFixturePatches();

	/** Spreads selected Fixture Patches over Universes */
	void SpreadFixturePatchesOverUniverses();

	/** Gets selected fixture patches. Returns false if no patches are selected */
	bool GetSelectedFixturePatches(TArray<UDMXEntityFixturePatch*>& OutFixturePatchArray) const;

	/** Returns true if the DMX Library has reachable Universes */
	bool DoesDMXLibraryHaveReachableUniverses() const;

	/** Returns the Fixture Patch that is topmost under Channel */
	UDMXEntityFixturePatch* GetTopmostFixturePatchOnChannel(uint32 Channel) const;

	/** Returns all Fixture Patches on a Channel ID */
	TArray<UDMXEntityFixturePatch*> GetFixturePatchesOnChannel(uint32 Channel) const;

	/** Returns the DMXLibrary or nullptr if not available */
	UDMXLibrary* GetDMXLibrary() const;

	/** The universe being displayed */
	int32 UniverseID;

	/** The hovered channel, or unset if no channel is hovered */
	TSharedPtr<SDMXChannelConnector> HoveredChannel;

	/** The channel that was last hovered */
	int32 LastHoveredChannel = 1;

	/** If true, monitors inputs */
	bool bMonitorInputs = false;

	/** If true the universe ID is out of controllers' ranges */
	EDMXPatchedUniverseReachability PatchedUniverseReachability;
	
	/** Widget showing the Name of the Universe */
	TSharedPtr<SBorder> UniverseName; 

	/** Grid laying out available channels */
	TSharedPtr<SGridPanel> Grid;

	/** Patches in the grid */
	TArray<TSharedRef<FDMXFixturePatchNode>> PatchedNodes;

	/** The Channel connectors in this universe */
	TArray<TSharedRef<SDMXChannelConnector>> ChannelConnectors;

	/** Delegate executed when drag enters a Channel */
	FOnDragOverChannel OnDragEnterChannel;

	/** Delegate executed when drag leaves a Channel */
	FOnDragOverChannel OnDragLeaveChannel;

	/** Delegate executed when a Drag Drop event was dropped onto a Channel */
	FOnDropOntoChannel OnDropOntoChannel;

	/** Anchor of Shift-Select while shift select is ongoing */
	TWeakObjectPtr<UDMXEntityFixturePatch> ShiftSelectAnchorPatch;

	/** The Fixture Patch Widgets that are currently being displalyed */
	TArray<TSharedRef<SDMXFixturePatchFragment>> FixturePatchFragmentWidgets;

	/** Timer handle for the Request Refresh method */
	FTimerHandle RequestRefreshTimerHandle;

	/** Shared data for fixture patch editors */
	TSharedPtr<FDMXFixturePatchSharedData> SharedData;

	/** The owning editor */
	TWeakPtr<FDMXEditor> WeakDMXEditor;

	private:
	///////////////////////////////////////////////////
	// Context menu Commands related

	/** Registers commands for this widget */
	void RegisterCommands();

	/** Creates the right click context menu */
	TSharedRef<SWidget> CreateContextMenu(int32 Channel);

	/** Command list for this widget */
	TSharedPtr<FUICommandList> CommandList;
};
