// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDMXReadOnlyFixturePatchList.h"

#include "Algo/Sort.h"
#include "DMXEditorUtils.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/DMXLibrary.h"
#include "TimerManager.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuContext.h"
#include "ToolMenuSection.h"
#include "Widgets/DMXReadOnlyFixturePatchListItem.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/SDMXReadOnlyFixturePatchListRow.h"
#include "Widgets/Views/SListView.h"


#define LOCTEXT_NAMESPACE "SDMXReadOnlyFixturePatchList"

namespace UE::DMX::SDMXReadOnlyFixturePatchListNamespace::Private
{
	class FScopedRestoreSelection
	{
	public:
		FScopedRestoreSelection(const TSharedRef<SDMXReadOnlyFixturePatchList>& InFixturePatchList)
			: FixturePatchList(InFixturePatchList)
		{
			SelectedItemsToRestore = FixturePatchList->GetSelectedItems();
		}

		~FScopedRestoreSelection()
		{
			const TArray<TSharedPtr<FDMXReadOnlyFixturePatchListItem>> NewListItems = FixturePatchList->GetListItems();
			
			TArray<TSharedPtr<FDMXReadOnlyFixturePatchListItem>> NewSelection;
			for (const TSharedPtr<FDMXReadOnlyFixturePatchListItem>& ItemToRestore : SelectedItemsToRestore)
			{
				const TSharedPtr<FDMXReadOnlyFixturePatchListItem>* CorrespondingItemPtr = Algo::FindByPredicate(NewListItems, [ItemToRestore](const TSharedPtr<FDMXReadOnlyFixturePatchListItem> NewItem)
					{
						return NewItem.IsValid() && ItemToRestore.IsValid() && NewItem->GetFixturePatch() == ItemToRestore->GetFixturePatch();
					});
				if (CorrespondingItemPtr)
				{
					NewSelection.Add(*CorrespondingItemPtr);
				}
			}

			FixturePatchList->SelectItems(NewSelection);
		}

	private:
		/** Addressed fixture patch list */
		TSharedRef<SDMXReadOnlyFixturePatchList> FixturePatchList;

		/** Items to restore */
		TArray<TSharedPtr<FDMXReadOnlyFixturePatchListItem>> SelectedItemsToRestore;
	};
}

const FName FDMXReadOnlyFixturePatchListCollumnIDs::EditorColor = "EditorColor";
const FName FDMXReadOnlyFixturePatchListCollumnIDs::FixturePatchName = "Name";
const FName FDMXReadOnlyFixturePatchListCollumnIDs::FixtureID = "FID";
const FName FDMXReadOnlyFixturePatchListCollumnIDs::FixtureType = "Type";
const FName FDMXReadOnlyFixturePatchListCollumnIDs::Mode = "Mode";
const FName FDMXReadOnlyFixturePatchListCollumnIDs::Patch = "Patch";

void SDMXReadOnlyFixturePatchList::Construct(const FArguments& InArgs)
{
	WeakDMXLibrary = InArgs._DMXLibrary;
	OnRowDragDetectedDelegate = InArgs._OnRowDragDetected;

	ChildSlot
		[
			SNew(SVerticalBox)

			// SearchBox section
			+ SVerticalBox::Slot()
			.Padding(8.f, 0.f, 8.f, 8.f)
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.Padding(0.f, 0.f, 4.f, 0.f)
				.AutoWidth()
				[
					SNew(SComboButton)
					.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
					.HasDownArrow(true)
					.OnGetMenuContent(this, &SDMXReadOnlyFixturePatchList::GenerateHeaderRowFilterMenu)
					.ButtonContent()
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Filter"))
					]
				]

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Fill)
				[
					SAssignNew(SearchBox, SSearchBox)
					.OnTextChanged(this, &SDMXReadOnlyFixturePatchList::OnSearchTextChanged)
					.ToolTipText(LOCTEXT("SearchBarTooltip", "Examples:\n\n* PatchName\n* FixtureTypeName\n* SomeMode\n* 1.\n* 1.1\n* Universe 1\n* Uni 1-3\n* Uni 1, 3\n* Uni 1, 4-5'."))
				]
			]

			// ListView section
			+ SVerticalBox::Slot()
			[
				SAssignNew(ListView, SListView<TSharedPtr<FDMXReadOnlyFixturePatchListItem>>)
				.HeaderRow(GenerateHeaderRow())
				.ListItemsSource(&ListItems)
				.OnGenerateRow(this, &SDMXReadOnlyFixturePatchList::OnGenerateRow)
				.OnContextMenuOpening(InArgs._OnContextMenuOpening)
				.OnSelectionChanged(InArgs._OnRowSelectionChanged)
				.OnMouseButtonClick(InArgs._OnRowClicked)
				.OnMouseButtonDoubleClick(InArgs._OnRowDoubleClicked)
			]
		];


	// Handle Entity changes
	UDMXLibrary::GetOnEntitiesAdded().AddSP(this, &SDMXReadOnlyFixturePatchList::OnEntityAddedOrRemoved);
	UDMXLibrary::GetOnEntitiesRemoved().AddSP(this, &SDMXReadOnlyFixturePatchList::OnEntityAddedOrRemoved);
	UDMXEntityFixturePatch::GetOnFixturePatchChanged().AddSP(this, &SDMXReadOnlyFixturePatchList::OnFixturePatchChanged);
	UDMXEntityFixtureType::GetOnFixtureTypeChanged().AddSP(this, &SDMXReadOnlyFixturePatchList::OnFixtureTypeChanged);

	ApplyListDescriptor(InArgs._ListDescriptor);

	ForceRefresh();
}

void SDMXReadOnlyFixturePatchList::RequestRefresh()
{
	if (!ListRefreshTimerHandle.IsValid())
	{
		ListRefreshTimerHandle = GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateSP(this, &SDMXReadOnlyFixturePatchList::ForceRefresh));
	}
}

TArray<UDMXEntityFixturePatch*> SDMXReadOnlyFixturePatchList::GetFixturePatchesInDMXLibrary() const
{
	if (WeakDMXLibrary.IsValid())
	{
		return WeakDMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>();
	}
	
	return TArray<UDMXEntityFixturePatch*>();
}

TArray<UDMXEntityFixturePatch*> SDMXReadOnlyFixturePatchList::GetFixturePatchesInList() const
{
	TArray<UDMXEntityFixturePatch*> FixturePatches;
	Algo::TransformIf(ListItems, FixturePatches,
		[](const TSharedPtr<FDMXReadOnlyFixturePatchListItem>& Item)
		{
			return Item.IsValid() && Item->GetFixturePatch();
		},
		[](const TSharedPtr<FDMXReadOnlyFixturePatchListItem>& Item)
		{
			return Item->GetFixturePatch();
		});

	return FixturePatches;
}

TArray<UDMXEntityFixturePatch*> SDMXReadOnlyFixturePatchList::GetSelectedFixturePatches() const
{
	const TArray<TSharedPtr<FDMXReadOnlyFixturePatchListItem>> SelectedItems = GetSelectedItems();
	TArray<UDMXEntityFixturePatch*> Result;
	Algo::TransformIf(SelectedItems, Result,
		[](const TSharedPtr<FDMXReadOnlyFixturePatchListItem>& Item)
		{
			return Item.IsValid() && Item->GetFixturePatch();
		},
		[](const TSharedPtr<FDMXReadOnlyFixturePatchListItem>& Item)
		{
			return Item->GetFixturePatch();
		});

	return Result;
}

void SDMXReadOnlyFixturePatchList::SetDMXLibrary(UDMXLibrary* InDMXLibrary)
{
	if (WeakDMXLibrary.Get() != InDMXLibrary)
	{
		WeakDMXLibrary = InDMXLibrary;
		RequestRefresh();
	}
}

void SDMXReadOnlyFixturePatchList::SetExcludedFixturePatches(const TArray<UDMXEntityFixturePatch*>& NewExcludedFixturePatches)
{
	ExcludedFixturePatches = NewExcludedFixturePatches;
}

void SDMXReadOnlyFixturePatchList::SelectItems(const TArray<TSharedPtr<FDMXReadOnlyFixturePatchListItem>>& ItemsToSelect, ESelectInfo::Type SelectInfo)
{
	ListView->ClearSelection();

	constexpr bool bSelected = true;
	ListView->SetItemSelection(ItemsToSelect, bSelected, SelectInfo);
}

void SDMXReadOnlyFixturePatchList::SetItemSelection(TSharedPtr<FDMXReadOnlyFixturePatchListItem> SelectedItem, bool bSelected, ESelectInfo::Type SelectInfo)
{
	if (ListView.IsValid())
	{
		ListView->SetItemSelection(SelectedItem, bSelected, SelectInfo);
	}
}

TArray<TSharedPtr<FDMXReadOnlyFixturePatchListItem>> SDMXReadOnlyFixturePatchList::GetSelectedItems() const
{
	return ListView.IsValid() ? ListView->GetSelectedItems() : TArray<TSharedPtr<FDMXReadOnlyFixturePatchListItem>>();
}

FDMXReadOnlyFixturePatchListDescriptor SDMXReadOnlyFixturePatchList::MakeListDescriptor() const
{
	FDMXReadOnlyFixturePatchListDescriptor ListDescriptor;
	ListDescriptor.SortedByColumnID = SortedByColumnID;
	ListDescriptor.ColumnIDToShowStateMap.Append(ColumnIDToShowStateMap);

	return ListDescriptor;
}

void SDMXReadOnlyFixturePatchList::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(ExcludedFixturePatches);
}

FString SDMXReadOnlyFixturePatchList::GetReferencerName() const
{
	return TEXT("SDMXReadOnlyFixturePatchList");
}

void SDMXReadOnlyFixturePatchList::ForceRefresh()
{
	ListRefreshTimerHandle.Invalidate();

	using namespace UE::DMX::SDMXReadOnlyFixturePatchListNamespace::Private;
	const FScopedRestoreSelection ScopedRestoreSelection(StaticCastSharedRef<SDMXReadOnlyFixturePatchList>(AsShared()));

	// Create new list items
	ListItems.Empty();
	if (WeakDMXLibrary.IsValid())
	{
		const TArray<UDMXEntityFixturePatch*> FixturePatchesInLibrary = WeakDMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>();
		Algo::TransformIf(FixturePatchesInLibrary, ListItems,
			[](UDMXEntityFixturePatch* FixturePatch)
			{
				return FixturePatch != nullptr;
			},
			[](UDMXEntityFixturePatch* FixturePatch)
			{
				return MakeShared<FDMXReadOnlyFixturePatchListItem>(FixturePatch);;
			});

		ListItems.RemoveAll([this](const TSharedPtr<FDMXReadOnlyFixturePatchListItem>& Item)
			{
				return ExcludedFixturePatches.Contains(Item->GetFixturePatch());
			});
		ListItems = FilterListItems(SearchBox->GetText());
	}

	// Reset cached rows
	ListRows.Reset(ListItems.Num());

	// Sort
	SortByColumnID(EColumnSortPriority::Max, SortedByColumnID, SortMode);

	ListView->RequestListRefresh();
}

TSharedRef<SHeaderRow> SDMXReadOnlyFixturePatchList::GenerateHeaderRow()
{
	TSharedRef<SHeaderRow> HeaderRow = SNew(SHeaderRow);

	HeaderRow->AddColumn(
		SHeaderRow::FColumn::FArguments()
		.ColumnId(FDMXReadOnlyFixturePatchListCollumnIDs::EditorColor)
		.DefaultLabel(FText())
		.FixedWidth(16.f)
	);
	
	HeaderRow->AddColumn(
		SHeaderRow::FColumn::FArguments()
		.ColumnId(FDMXReadOnlyFixturePatchListCollumnIDs::FixturePatchName)
		.SortMode(this, &SDMXReadOnlyFixturePatchList::GetColumnSortMode, FDMXReadOnlyFixturePatchListCollumnIDs::FixturePatchName)
		.OnSort(this, &SDMXReadOnlyFixturePatchList::SortByColumnID)
		.FillWidth(0.25f)
		.VAlignHeader(VAlign_Center)
		.HeaderContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("FixturePatchNameColumnLabel", "Name"))
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		]
	);

	HeaderRow->AddColumn(
		SHeaderRow::FColumn::FArguments()
		.ColumnId(FDMXReadOnlyFixturePatchListCollumnIDs::FixtureID)
		.SortMode(this, &SDMXReadOnlyFixturePatchList::GetColumnSortMode, FDMXReadOnlyFixturePatchListCollumnIDs::FixtureID)
		.OnSort(this, &SDMXReadOnlyFixturePatchList::SortByColumnID)
		.FillWidth(0.1f)
		.VAlignHeader(VAlign_Center)
		.HeaderContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("FixtureIDColumnLabel", "FID"))
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		]
	);

	HeaderRow->AddColumn(
		SHeaderRow::FColumn::FArguments()
		.ColumnId(FDMXReadOnlyFixturePatchListCollumnIDs::FixtureType)
		.SortMode(this, &SDMXReadOnlyFixturePatchList::GetColumnSortMode, FDMXReadOnlyFixturePatchListCollumnIDs::FixtureType)
		.OnSort(this, &SDMXReadOnlyFixturePatchList::SortByColumnID)
		.FillWidth(0.2f)
		.VAlignHeader(VAlign_Center)
		.HeaderContent()
		[
			SNew(STextBlock)
			.Text((LOCTEXT("FixtureTypeColumnLabel", "Type")))
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		]
	);

	HeaderRow->AddColumn(
		SHeaderRow::FColumn::FArguments()
		.ColumnId(FDMXReadOnlyFixturePatchListCollumnIDs::Mode)
		.SortMode(this, &SDMXReadOnlyFixturePatchList::GetColumnSortMode, FDMXReadOnlyFixturePatchListCollumnIDs::Mode)
		.OnSort(this, &SDMXReadOnlyFixturePatchList::SortByColumnID)
		.FillWidth(0.2f)
		.VAlignHeader(VAlign_Center)
		.HeaderContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ModeColumnLabel", "Mode"))
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		]
	);

	HeaderRow->AddColumn(
		SHeaderRow::FColumn::FArguments()
		.ColumnId(FDMXReadOnlyFixturePatchListCollumnIDs::Patch)
		.SortMode(this, &SDMXReadOnlyFixturePatchList::GetColumnSortMode, FDMXReadOnlyFixturePatchListCollumnIDs::Patch)
		.OnSort(this, &SDMXReadOnlyFixturePatchList::SortByColumnID)
		.FillWidth(0.1f)
		.VAlignHeader(VAlign_Center)
		.HeaderContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("PatchColumnLabel", "Patch"))
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		]
	);

	return HeaderRow;
}

TSharedRef<ITableRow> SDMXReadOnlyFixturePatchList::OnGenerateRow(TSharedPtr<FDMXReadOnlyFixturePatchListItem> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	const TSharedRef<SDMXReadOnlyFixturePatchListRow> NewRow =
		SNew(SDMXReadOnlyFixturePatchListRow, OwnerTable, InItem)
		.OnRowDragDetected(OnRowDragDetectedDelegate);

	return NewRow;
}

FName SDMXReadOnlyFixturePatchList::GetHeaderRowFilterMenuName() const
{
	return "DMXEditor.ReadOnlyFixturePatchList.HeaderRowFilterMenu";
}

TSharedRef<SWidget> SDMXReadOnlyFixturePatchList::GenerateHeaderRowFilterMenu()
{
	const FName MenuName = GetHeaderRowFilterMenuName();

	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		UToolMenu* Menu = ToolMenus->RegisterMenu(MenuName);
		FToolMenuSection& Section = Menu->AddSection("ShowColumnSection", LOCTEXT("ShowColumnSection", "Columns"));
		
		auto AddMenuEntryLambda = [this, &Section](const FName& Name, const FText& Label, const FText& ToolTip, const FName& ColumnID)
		{
			Section.AddMenuEntry
			(
				Name,
				Label,
				ToolTip,
				FSlateIcon(),
				FUIAction
				(
					FExecuteAction::CreateSP(this, &SDMXReadOnlyFixturePatchList::ToggleColumnShowState, ColumnID),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(this, &SDMXReadOnlyFixturePatchList::IsColumnShown, ColumnID)
				),
				EUserInterfaceActionType::ToggleButton
			);
		};

		AddMenuEntryLambda(
			"ShowNameColumn",
			LOCTEXT("FixturePatchNameColumn_Label", "Show Name"),
			LOCTEXT("FixturePatchLNameColumn_Tooltip", "Show/Hide Name Column"),
			FDMXReadOnlyFixturePatchListCollumnIDs::FixturePatchName
		);

		AddMenuEntryLambda(
			"ShowFIDColumn",
			LOCTEXT("FixturePatchFIDColumn_Label", "Show FID"),
			LOCTEXT("FixturePatchFIDColumn_Tooltip", "Show/Hide FID Column"),
			FDMXReadOnlyFixturePatchListCollumnIDs::FixtureID
		);

		AddMenuEntryLambda(
			"ShowTypeColumn",
			LOCTEXT("FixturePatchTypeColumn_Label", "Show Type"),
			LOCTEXT("FixturePatchTypeColumn_Tooltip", "Show/Hide Type Column"),
			FDMXReadOnlyFixturePatchListCollumnIDs::FixtureType
		);

		AddMenuEntryLambda(
			"ShowModeColumn",
			LOCTEXT("FixturePatchModeColumn_Label", "Show Mode"),
			LOCTEXT("FixturePatchModeColumn_Tooltip", "Show/Hide Mode Column"),
			FDMXReadOnlyFixturePatchListCollumnIDs::Mode
		);

		AddMenuEntryLambda(
			"ShowPatchColumn",
			LOCTEXT("FixturePatchyColumn_Label", "Show Patch"),
			LOCTEXT("FixturePatchColumn_Tooltip", "Show/Hide Patch Column"),
			FDMXReadOnlyFixturePatchListCollumnIDs::Patch
		);
	}

	FToolMenuContext Context;
	UToolMenu* Menu = ToolMenus->GenerateMenu(MenuName, Context);

	return ToolMenus->GenerateWidget(Menu);
}

void SDMXReadOnlyFixturePatchList::ApplyListDescriptor(const FDMXReadOnlyFixturePatchListDescriptor& InListDescriptor)
{
	const TSharedPtr<SHeaderRow> HeaderRow = ListView.IsValid() ? ListView->GetHeaderRow() : nullptr;
	if (!HeaderRow.IsValid())
	{
		return;
	}

	ColumnIDToShowStateMap = InListDescriptor.ColumnIDToShowStateMap;
	SortedByColumnID = InListDescriptor.SortedByColumnID;
	SortByColumnID(EColumnSortPriority::None, SortedByColumnID, SortMode);

	for (const SHeaderRow::FColumn& Column : HeaderRow->GetColumns())
	{
		// Use the column show state of the descriptor if possilble. Always show columns not present in the descriptor.
		const bool* bShowColumnPtr = ColumnIDToShowStateMap.Find(Column.ColumnId);
		const bool bShowColumn = bShowColumnPtr ? *bShowColumnPtr : true;

		HeaderRow->SetShowGeneratedColumn(Column.ColumnId, bShowColumn);
	}

	ColumnIDToShowStateMap.Append(InListDescriptor.ColumnIDToShowStateMap);
}

TArray<TSharedPtr<FDMXReadOnlyFixturePatchListItem>> SDMXReadOnlyFixturePatchList::FilterListItems(const FText& SearchText)
{
	// Filter and return in order of precendence
	if (SearchText.IsEmpty())
	{
		return ListItems;
	}

	TArray<TSharedPtr<FDMXReadOnlyFixturePatchListItem>> Items = ListItems;
	const FString SearchString = SearchText.ToString();

	const TArray<int32> Universes = FDMXEditorUtils::ParseUniverses(SearchString);
	if (!Universes.IsEmpty())
	{
		Items.RemoveAll([Universes](const TSharedPtr<FDMXReadOnlyFixturePatchListItem>& FixturePatchRef)
			{
				const UDMXEntityFixturePatch* FixturePatch = FixturePatchRef.IsValid() ? FixturePatchRef->GetFixturePatch() : nullptr;
				if (FixturePatch)
				{
					return !Universes.Contains(FixturePatch->GetUniverseID());
				}

				return true;
			});

		return Items;
	}

	int32 Address;
	if (FDMXEditorUtils::ParseAddress(SearchString, Address))
	{
		Items.RemoveAll([Address](const TSharedPtr<FDMXReadOnlyFixturePatchListItem>& FixturePatchRef)
			{
				const UDMXEntityFixturePatch* FixturePatch = FixturePatchRef.IsValid() ? FixturePatchRef->GetFixturePatch() : nullptr;
				if (FixturePatch)
				{
					return FixturePatch->GetStartingChannel() != Address;
				}

				return true;
			});

		return Items;
	}

	const TArray<int32> FixtureIDs = FDMXEditorUtils::ParseFixtureIDs(SearchString);
	for (int32 FixtureID : FixtureIDs)
	{
		Items.RemoveAll([FixtureID](const TSharedPtr<FDMXReadOnlyFixturePatchListItem>& FixturePatchRef)
			{
				const UDMXEntityFixturePatch* FixturePatch = FixturePatchRef.IsValid() ? FixturePatchRef->GetFixturePatch() : nullptr;
				if (FixturePatch)
				{
					const int32 FID = FixturePatch->GetFixtureID();
					return FID != FixtureID;
				}

				return true;
			});

		if (ListItems.Num() > 0)
		{
			return Items;
		}
	}

	Items.RemoveAll([SearchString](const TSharedPtr<FDMXReadOnlyFixturePatchListItem>& FixturePatchRef)
		{
			const UDMXEntityFixturePatch* FixturePatch = FixturePatchRef.IsValid() ? FixturePatchRef->GetFixturePatch() : nullptr;
			if (FixturePatch)
			{
				return !FixturePatch->Name.Contains(SearchString);
			}

			return true;
		});

	return Items;
}

void SDMXReadOnlyFixturePatchList::SortByColumnID(const EColumnSortPriority::Type SortPriority, const FName& ColumnID, const EColumnSortMode::Type InSortMode)
{
	SortMode = InSortMode;
	SortedByColumnID = ColumnID;

	const bool bAscending = InSortMode == EColumnSortMode::Ascending ? true : false;

	if (ColumnID == FDMXReadOnlyFixturePatchListCollumnIDs::FixturePatchName)
	{
		Algo::StableSort(ListItems, [bAscending](const TSharedPtr<FDMXReadOnlyFixturePatchListItem>& ItemA, const TSharedPtr<FDMXReadOnlyFixturePatchListItem>& ItemB)
			{
				const UDMXEntityFixturePatch* FixturePatchA = ItemA->GetFixturePatch();
				const UDMXEntityFixturePatch* FixturePatchB = ItemB->GetFixturePatch();
				if (!FixturePatchA || !FixturePatchB)
				{
					return false;
				}

				const FString FixturePatchNameA = FixturePatchA->Name;
				const FString FixturePatchNameB = FixturePatchB->Name;

				const bool bIsGreater = FixturePatchNameA >= FixturePatchNameB;
				return bAscending ? !bIsGreater : bIsGreater;
			});
	}
	else if (ColumnID == FDMXReadOnlyFixturePatchListCollumnIDs::FixtureID)
	{
		Algo::StableSort(ListItems, [bAscending](const TSharedPtr<FDMXReadOnlyFixturePatchListItem>& ItemA, const TSharedPtr<FDMXReadOnlyFixturePatchListItem>& ItemB)
			{
				bool bIsGreater = [ItemA, ItemB]()
				{
					const UDMXEntityFixturePatch* FixturePatchA = ItemA->GetFixturePatch();
					const UDMXEntityFixturePatch* FixturePatchB = ItemB->GetFixturePatch();
					if (!FixturePatchA || !FixturePatchB)
					{
						return false;
					}

					const int32 FixtureIDA = FixturePatchA->GetFixtureID();
					const int32 FixtureIDB = FixturePatchB->GetFixtureID();

					return FixtureIDA >= FixtureIDB;
				}();

				return bAscending ? !bIsGreater : bIsGreater;
			});
	}
	else if (ColumnID == FDMXReadOnlyFixturePatchListCollumnIDs::FixtureType)
	{
		Algo::StableSort(ListItems, [bAscending](const TSharedPtr<FDMXReadOnlyFixturePatchListItem>& ItemA, const TSharedPtr<FDMXReadOnlyFixturePatchListItem>& ItemB)
			{
				const UDMXEntityFixturePatch* FixturePatchA = ItemA->GetFixturePatch();
				const UDMXEntityFixturePatch* FixturePatchB = ItemB->GetFixturePatch();
				if (!FixturePatchA || !FixturePatchB)
				{
					return false;
				}

				const FString FixtureTypeA = FixturePatchA->GetFixtureType()->Name;
				const FString FixtureTypeB = FixturePatchB->GetFixtureType()->Name;

				const bool bIsGreater = FixtureTypeA >= FixtureTypeB;
				return bAscending ? !bIsGreater : bIsGreater;
			});
	}
	else if (ColumnID == FDMXReadOnlyFixturePatchListCollumnIDs::Mode)
	{
		Algo::StableSort(ListItems, [bAscending](const TSharedPtr<FDMXReadOnlyFixturePatchListItem>& ItemA, const TSharedPtr<FDMXReadOnlyFixturePatchListItem>& ItemB)
			{
				const UDMXEntityFixturePatch* FixturePatchA = ItemA->GetFixturePatch();
				const UDMXEntityFixturePatch* FixturePatchB = ItemB->GetFixturePatch();
				if (!FixturePatchA || !FixturePatchB)
				{
					return false;
				}

				const bool bIsGreater = FixturePatchA->GetActiveModeIndex() >= FixturePatchB->GetActiveModeIndex();
				return bAscending ? !bIsGreater : bIsGreater;
			});
	}
	else if (ColumnID == FDMXReadOnlyFixturePatchListCollumnIDs::Patch)
	{
		Algo::StableSort(ListItems, [bAscending](const TSharedPtr<FDMXReadOnlyFixturePatchListItem>& ItemA, const TSharedPtr<FDMXReadOnlyFixturePatchListItem>& ItemB)
			{
				const UDMXEntityFixturePatch* FixturePatchA = ItemA->GetFixturePatch();
				const UDMXEntityFixturePatch* FixturePatchB = ItemB->GetFixturePatch();
				if (!FixturePatchA || !FixturePatchB)
				{
					return false;
				}

				const bool bIsUniverseIDGreater = FixturePatchA->GetUniverseID() > FixturePatchB->GetUniverseID();
				const bool bIsSameUniverse = FixturePatchA->GetUniverseID() == FixturePatchB->GetUniverseID();
				const bool bAreAddressesGreater = FixturePatchA->GetStartingChannel() > FixturePatchB->GetStartingChannel();

				const bool bIsGreater = bIsUniverseIDGreater || (bIsSameUniverse && bAreAddressesGreater);
				return bAscending ? !bIsGreater : bIsGreater;
			});
	}

	ListView->RequestListRefresh();
}

void SDMXReadOnlyFixturePatchList::OnSearchTextChanged(const FText& SearchText)
{
	RequestRefresh();
}

void SDMXReadOnlyFixturePatchList::OnEntityAddedOrRemoved(UDMXLibrary* InDMXLibrary, TArray<UDMXEntity*> Entities)
{
	if (InDMXLibrary == WeakDMXLibrary)
	{
		RequestRefresh();
	}
}

void SDMXReadOnlyFixturePatchList::OnFixturePatchChanged(const UDMXEntityFixturePatch* FixturePatch)
{
	// Refresh only if the fixture patch is in the library this editor handles
	if (FixturePatch && FixturePatch->GetParentLibrary() == WeakDMXLibrary)
	{
		RequestRefresh();
	}
}

void SDMXReadOnlyFixturePatchList::OnFixtureTypeChanged(const UDMXEntityFixtureType* FixtureType)
{
	// Refresh only if the fixture type is in the library this editor handles
	if (FixtureType && FixtureType->GetParentLibrary() == WeakDMXLibrary)
	{
		RequestRefresh();
	}
}

void SDMXReadOnlyFixturePatchList::ToggleColumnShowState(const FName ColumnID)
{
	const TSharedPtr<SHeaderRow> HeaderRow = ListView.IsValid() ? ListView->GetHeaderRow() : nullptr;
	if (!HeaderRow.IsValid())
	{
		return;
	}

	const bool bShowState = ColumnIDToShowStateMap[ColumnID];
	ColumnIDToShowStateMap.Add(ColumnID) = !bShowState;

	HeaderRow->SetShowGeneratedColumn(ColumnID, !bShowState);
}

bool SDMXReadOnlyFixturePatchList::IsColumnShown(const FName ColumnID) const
{
	return ColumnIDToShowStateMap[ColumnID];
}

EColumnSortMode::Type SDMXReadOnlyFixturePatchList::GetColumnSortMode(const FName ColumnID) const
{
	if (SortedByColumnID != ColumnID)
	{
		return EColumnSortMode::None;
	}

	return SortMode;
}

#undef LOCTEXT_NAMESPACE
