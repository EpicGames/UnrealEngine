// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDMXReadOnlyFixturePatchListRow.h"

#include "DMXEditor.h"
#include "DMXEditorStyle.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/DMXLibrary.h"
#include "Styling/AppStyle.h"
#include "Widgets/DMXReadOnlyFixturePatchListItem.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"


void SDMXReadOnlyFixturePatchListRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, const TSharedPtr<FDMXReadOnlyFixturePatchListItem>& InItem)
{
	Item = InItem;

	SMultiColumnTableRow<TSharedPtr<FDMXReadOnlyFixturePatchListItem>>::Construct(
		FSuperRowType::FArguments()
		.OnDragDetected(InArgs._OnRowDragDetected)
		.IsEnabled(InArgs._IsEnabled)
		.Style(&FDMXEditorStyle::Get().GetWidgetStyle<FTableRowStyle>("FixturePatchList.Row")),
		InOwnerTable);
}

TSharedRef<SWidget> SDMXReadOnlyFixturePatchListRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == FDMXReadOnlyFixturePatchListCollumnIDs::EditorColor)
	{
		return GenerateEditorColorRow();
	}
	else if (ColumnName == FDMXReadOnlyFixturePatchListCollumnIDs::FixturePatchName)
	{
		return GenerateFixturePatchNameRow();
	}
	else if (ColumnName == FDMXReadOnlyFixturePatchListCollumnIDs::FixtureID)
	{
		return GenerateFixtureIDRow();
	}
	else if (ColumnName == FDMXReadOnlyFixturePatchListCollumnIDs::FixtureType)
	{
		return GenerateFixtureTypeRow();
	}
	else if (ColumnName == FDMXReadOnlyFixturePatchListCollumnIDs::Mode)
	{
		return GenerateModeRow();
	}
	else if (ColumnName == FDMXReadOnlyFixturePatchListCollumnIDs::Patch)
	{
		return GeneratePatchRow();
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SDMXReadOnlyFixturePatchListRow::GenerateEditorColorRow()
{
	return
		SNew(SBorder)
		.HAlign(HAlign_Fill)
		.Padding(5.f, 2.f)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		[
			SNew(SImage)
			.Image(FDMXEditorStyle::Get().GetBrush("DMXEditor.WhiteRoundedPropertyBorder"))
			.ColorAndOpacity(Item.Get(), &FDMXReadOnlyFixturePatchListItem::GetEditorColor)
		];
}

TSharedRef<SWidget> SDMXReadOnlyFixturePatchListRow::GenerateFixturePatchNameRow()
{
	return
		SNew(SBorder)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.Padding(4.f)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			.Text(Item.Get(), &FDMXReadOnlyFixturePatchListItem::GetNameText)
			.ColorAndOpacity(FLinearColor::White)
		];
}

TSharedRef<SWidget> SDMXReadOnlyFixturePatchListRow::GenerateFixtureIDRow()
{
	return
		SNew(SBorder)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.Padding(4.f)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			.Text(Item.Get(), &FDMXReadOnlyFixturePatchListItem::GetFixtureIDText)
			.ColorAndOpacity(FLinearColor::White)
		];
}

TSharedRef<SWidget> SDMXReadOnlyFixturePatchListRow::GenerateFixtureTypeRow()
{
	return
		SNew(SBorder)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.Padding(4.f)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			.Text(Item.Get(), &FDMXReadOnlyFixturePatchListItem::GetFixtureTypeText)
			.ColorAndOpacity(FLinearColor::White)
		];
}

TSharedRef<SWidget> SDMXReadOnlyFixturePatchListRow::GenerateModeRow()
{
	return
		SNew(SBorder)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.Padding(4.f)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		[
			SNew(SInlineEditableTextBlock)
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			.Text(Item.Get(), &FDMXReadOnlyFixturePatchListItem::GetModeText)
			.ColorAndOpacity(FLinearColor::White)
		];
}

TSharedRef<SWidget> SDMXReadOnlyFixturePatchListRow::GeneratePatchRow()
{
	return
		SNew(SBorder)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.Padding(4.f)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			.Text(Item.Get(), &FDMXReadOnlyFixturePatchListItem::GetUniverseChannelText)
			.ColorAndOpacity(FLinearColor::White)
		];
}
