// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXFixtureTypeFunctionsEditorFunctionItem.h"

#include "DMXAttribute.h"
#include "DMXAttributeToDefaultPhyiscalProperties.h"
#include "DMXEditor.h"
#include "DMXRuntimeUtils.h"
#include "Library/DMXEntityFixtureType.h"
#include "ScopedTransaction.h"


#define LOCTEXT_NAMESPACE "DMXFixtureTypeFunctionsEditorFunctionItem"

FDMXFixtureTypeFunctionsEditorFunctionItem::FDMXFixtureTypeFunctionsEditorFunctionItem(const TSharedRef<FDMXEditor>& InDMXEditor, TWeakObjectPtr<UDMXEntityFixtureType> InFixtureType, int32 InModeIndex, int32 InFunctionIndex)
	: FDMXFixtureTypeFunctionsEditorItemBase(InDMXEditor, InFixtureType, InModeIndex)
	, ModeIndex(InModeIndex)
	, FunctionIndex(InFunctionIndex)
{
	ensureMsgf(FixtureType.IsValid() && FixtureType->Modes.IsValidIndex(ModeIndex) && FixtureType->Modes[ModeIndex].Functions.IsValidIndex(FunctionIndex), TEXT("FDMXFixtureTypeFunctionsEditorFunctionItem is constructed with an invalid Function."));
}

FText FDMXFixtureTypeFunctionsEditorFunctionItem::GetDisplayName() const
{
	return GetFunctionName();
}

int32 FDMXFixtureTypeFunctionsEditorFunctionItem::GetFunctionIndex() const
{
	if (ensureMsgf(FixtureType.IsValid() && FixtureType->Modes.IsValidIndex(ModeIndex) && FixtureType->Modes[ModeIndex].Functions.IsValidIndex(FunctionIndex), TEXT("Invalid Fixture Type, Mode or Function in FDMXFixtureTypeFunctionsEditorFunctionItem.")))
	{
		return FunctionIndex;
	}

	return INDEX_NONE;
}

bool FDMXFixtureTypeFunctionsEditorFunctionItem::HasValidAttribute() const
{
	// Can be invalid when the name is gettered while the modes or functions are changing
	if (FixtureType.IsValid() && FixtureType->Modes.IsValidIndex(ModeIndex) && FixtureType->Modes[ModeIndex].Functions.IsValidIndex(FunctionIndex))
	{
		const FDMXFixtureFunction& Function = FixtureType->Modes[ModeIndex].Functions[FunctionIndex];
		return !Function.Attribute.IsValid();
	}

	return false;
}

FText FDMXFixtureTypeFunctionsEditorFunctionItem::GetFunctionName() const
{
	// Can be invalid when the name is gettered while the modes or functions are changing
	if (FixtureType.IsValid() && FixtureType->Modes.IsValidIndex(ModeIndex) && FixtureType->Modes[ModeIndex].Functions.IsValidIndex(FunctionIndex))
	{
		const FDMXFixtureFunction& Function = FixtureType->Modes[ModeIndex].Functions[FunctionIndex];
		return FText::FromString(Function.FunctionName);
	}

	return FText(LOCTEXT("InvalidFunctionWarning", "Function is no longer valid"));
}

int32 FDMXFixtureTypeFunctionsEditorFunctionItem::GetStartingChannel() const
{
	// Can be invalid when the name is gettered while the modes or functions are changing
	if (FixtureType.IsValid() && FixtureType->Modes.IsValidIndex(ModeIndex) && FixtureType->Modes[ModeIndex].Functions.IsValidIndex(FunctionIndex))
	{
		const FDMXFixtureFunction& Function = FixtureType->Modes[ModeIndex].Functions[FunctionIndex];
		return Function.Channel;
	}

	return -1;
}

void FDMXFixtureTypeFunctionsEditorFunctionItem::SetStartingChannel(int32 NewStartingChannel)
{
	if (ensureMsgf(NewStartingChannel > 0 && NewStartingChannel <= DMX_MAX_ADDRESS, TEXT("Invalid Starting Channel Provided when setting a Function's Starting Channel.")))
	{
		if (ensureMsgf(FixtureType.IsValid() && FixtureType->Modes.IsValidIndex(ModeIndex) && FixtureType->Modes[ModeIndex].Functions.IsValidIndex(FunctionIndex), TEXT("Invalid Fixture Type, Mode or Function in FDMXFixtureTypeFunctionsEditorFunctionItem.")))
		{
			const FScopedTransaction ReorderFunctionTransaction(LOCTEXT("SetChannelTransaction", "Set Fixture Function Starting Channel"));
			FixtureType->PreEditChange(FDMXFixtureFunction::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FDMXFixtureFunction, Channel)));

			int32 ResultingStartingChannel;
			FixtureType->SetFunctionStartingChannel(ModeIndex, FunctionIndex, NewStartingChannel, ResultingStartingChannel);
			
			FixtureType->PostEditChange();
		}
	}
}

int32 FDMXFixtureTypeFunctionsEditorFunctionItem::GetNumChannels() const
{
	if (FixtureType.IsValid() && FixtureType->Modes.IsValidIndex(ModeIndex) && FixtureType->Modes[ModeIndex].Functions.IsValidIndex(FunctionIndex))
	{
		const FDMXFixtureFunction& Function = FixtureType->Modes[ModeIndex].Functions[FunctionIndex];
		return Function.GetNumChannels();
	}

	return -1;
}

FDMXAttributeName FDMXFixtureTypeFunctionsEditorFunctionItem::GetAttributeName() const
{
	if (FixtureType.IsValid() && FixtureType->Modes.IsValidIndex(ModeIndex) && FixtureType->Modes[ModeIndex].Functions.IsValidIndex(FunctionIndex))
	{
		const FDMXFixtureFunction& Function = FixtureType->Modes[ModeIndex].Functions[FunctionIndex];
		return Function.Attribute;
	}

	return FDMXAttributeName(NAME_None);
}

void FDMXFixtureTypeFunctionsEditorFunctionItem::SetAttributeName(const FDMXAttributeName& AttributeName) const
{
	using namespace UE::DMX;

	if (ensureMsgf(FixtureType.IsValid() && FixtureType->Modes.IsValidIndex(ModeIndex) && FixtureType->Modes[ModeIndex].Functions.IsValidIndex(FunctionIndex), TEXT("Invalid Fixture Type, Mode or Function in FDMXFixtureTypeFunctionsEditorFunctionItem.")))
	{
		const FScopedTransaction SetAttributeNamelTransaction(LOCTEXT("SetAttributeNamelTransaction", "Set Fixture Function Attribute Name"));
		FixtureType->PreEditChange(FDMXFixtureFunction::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FDMXFixtureFunction, Attribute)));

		FDMXFixtureFunction& Function = FixtureType->Modes[ModeIndex].Functions[FunctionIndex];
		Function.Attribute = AttributeName;

		FDMXAttributeToDefaultPhyiscalProperties::ResetToDefaultPhysicalProperties(Function);

		FixtureType->PostEditChange();
	}
}

bool FDMXFixtureTypeFunctionsEditorFunctionItem::IsValidFunctionName(const FText& InFunctionName, FText& OutInvalidReason)
{
	if (ensureMsgf(FixtureType.IsValid() && FixtureType->Modes.IsValidIndex(ModeIndex) && FixtureType->Modes[ModeIndex].Functions.IsValidIndex(FunctionIndex), TEXT("Invalid Fixture Type, Mode or Function in FDMXFixtureTypeFunctionsEditorFunctionItem.")))
	{
		if (FText::TrimPrecedingAndTrailing(InFunctionName).IsEmpty())
		{
			OutInvalidReason = LOCTEXT("EmptyFunctionNameError", "The Function Name cannot be blank.");
			return false;
		}

		const FDMXFixtureMode& MyMode = FixtureType->Modes[ModeIndex];
		const FDMXFixtureFunction& MyMutableFunction = FixtureType->Modes[ModeIndex].Functions[FunctionIndex];
		const FString OldName = MyMutableFunction.FunctionName;
		const FString& NewName = InFunctionName.ToString();

		if (OldName == NewName)
		{
			return true;
		}

		const bool bUniqueName = [NewName, this]()
		{
			int32 NumFunctionWithSameName = 0;
			for (const FDMXFixtureFunction& Function : FixtureType->Modes[ModeIndex].Functions)
			{
				if (Function.FunctionName == NewName)
				{
					NumFunctionWithSameName++;
				}
			}

			return NumFunctionWithSameName < 2;
		}();

		if (bUniqueName)
		{
			return true;
		}
		else
		{
			OutInvalidReason = LOCTEXT("DuplicateFunctionNameError", "A Function with this name already exists.");
			return false;
		}
	}

	OutInvalidReason = LOCTEXT("InvalidFunctionError", "Invalid Fixture Type or Mode or Function no longer exists in Fixture Type");
	return false;
}

void FDMXFixtureTypeFunctionsEditorFunctionItem::SetFunctionName(const FText& DesiredFunctionName, FText& OutUniqueFunctionName)
{
	if (FixtureType.IsValid())
	{
		const FScopedTransaction SetFunctionNameTransaction(LOCTEXT("SetFunctionNameTransaction", "Rename Fixture Function"));
		FixtureType->PreEditChange(FDMXFixtureFunction::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FDMXFixtureFunction, FunctionName)));

		FString UniqueFunctionNameString;
		FixtureType->SetFunctionName(ModeIndex, FunctionIndex, DesiredFunctionName.ToString(), UniqueFunctionNameString);

		OutUniqueFunctionName = FText::FromString(UniqueFunctionNameString);

		FixtureType->PostEditChange();
	}
}

void FDMXFixtureTypeFunctionsEditorFunctionItem::DeleteFunction()
{
	if (ensureMsgf(FixtureType.IsValid() && FixtureType->Modes.IsValidIndex(ModeIndex) && FixtureType->Modes[ModeIndex].Functions.IsValidIndex(FunctionIndex), TEXT("Invalid Fixture Type, Mode or Function in FDMXFixtureTypeFunctionsEditorFunctionItem.")))
	{
		const FScopedTransaction DeleteFunctionTransaction(LOCTEXT("DeleteFunctionTransaction", "Delete Fixture Function"));
		FixtureType->PreEditChange(FDMXFixtureFunction::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FDMXFixtureMode, Functions)));

		FixtureType->Modes[ModeIndex].Functions.RemoveAt(FunctionIndex);

		FixtureType->PostEditChange();
	}
}

TSharedPtr<FDMXFixtureTypeSharedData> FDMXFixtureTypeFunctionsEditorFunctionItem::GetFixtureTypeSharedData() const
{
	if (TSharedPtr<FDMXEditor> DMXEditor = WeakDMXEditor.Pin())
	{
		return DMXEditor->GetFixtureTypeSharedData();
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
