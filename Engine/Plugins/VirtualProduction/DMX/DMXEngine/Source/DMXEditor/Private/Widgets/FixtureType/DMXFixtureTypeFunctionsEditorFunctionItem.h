// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/FixtureType/DMXFixtureTypeFunctionsEditorItemBase.h"

enum class EDMXFixtureSignalFormat : uint8;
struct FDMXAttributeName;


/** A Function as an item in a list */
class FDMXFixtureTypeFunctionsEditorFunctionItem
	: public FDMXFixtureTypeFunctionsEditorItemBase
{
public:
	/** Constructor for Normal Functions */
	FDMXFixtureTypeFunctionsEditorFunctionItem(const TSharedRef<FDMXEditor>& InDMXEditor, TWeakObjectPtr<UDMXEntityFixtureType> InFixtureType, int32 InModeIndex, int32 InFunctionIndex);

	//~ Begin FDMXFixtureTypeFunctionsEditorFunctionItemBase interface
	virtual EItemType GetType() const { return EItemType::Function; }
	virtual FText GetDisplayName() const;
	//~ End FDMXFixtureTypeFunctionsEditorFunctionItemBase interface

	/** Returns the Index of the Function in the Functions array */
	int32 GetFunctionIndex() const;

	/** Returns true if the function has a valid Attribute */
	bool HasValidAttribute() const;

	/** Returns the Function Name as Text */
	FText GetFunctionName() const;

	/** Returns the Starting Channel of the Function  */
	int32 GetStartingChannel() const;

	/** Sets the Starting Channel of the Function  */
	void SetStartingChannel(int32 NewStartingChannel);

	/** Returns the Number of Channels of the Function */
	int32 GetNumChannels() const;

	/** Returns the Attribut Name */
	FDMXAttributeName GetAttributeName() const;

	/** Sets the Attribut Name */
	void SetAttributeName(const FDMXAttributeName& AttributeName) const;

	/** Returns if the Name is valid for the Function. If the Function Name is not valid, returns a non-empty OutInvalidReason */
	bool IsValidFunctionName(const FText& InFunctionName, FText& OutInvalidReason);

	/** Sets a Function Name, makes it unique. Returns the OutUniqueFunctionName as output argument */
	void SetFunctionName(const FText& DesiredFunctionName, FText& OutUniqueFunctionName);

	/** Deletes the function this item stands for */
	void DeleteFunction();

	/** Returns the Fixture Type Shared Data for the editor that makes use of this Item */
	TSharedPtr<FDMXFixtureTypeSharedData> GetFixtureTypeSharedData() const;

private:
	/** The index of the Mode the Function resides in, in the Fixture Type's Modes array */
	int32 ModeIndex = INDEX_NONE;

	/** The index of the Function in the Mode */
	int32 FunctionIndex = INDEX_NONE;
};
