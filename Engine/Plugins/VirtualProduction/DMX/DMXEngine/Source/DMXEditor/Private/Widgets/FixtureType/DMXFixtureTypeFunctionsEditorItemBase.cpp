// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXFixtureTypeFunctionsEditorItemBase.h"

#include "DMXEditor.h"
#include "Library/DMXEntityFixtureType.h"


FDMXFixtureTypeFunctionsEditorItemBase::FDMXFixtureTypeFunctionsEditorItemBase(const TSharedRef<FDMXEditor>& InDMXEditor, TWeakObjectPtr<UDMXEntityFixtureType> InFixtureType, int32 InModeIndex)
	: FixtureType(InFixtureType)
	, ModeIndex(InModeIndex)
	, SharedData(InDMXEditor->GetFixtureTypeSharedData())
	, WeakDMXEditor(InDMXEditor)
{
	ensureMsgf(SharedData.IsValid(), TEXT("FixtureTypeSharedData not valid when trying to create a FDMXFixtureTypeFunctionsEditorItemBase"));
}

int32 FDMXFixtureTypeFunctionsEditorItemBase::GetModeIndex() const
{
	if (ensureMsgf(FixtureType.IsValid() && FixtureType->Modes.IsValidIndex(ModeIndex), TEXT("Invalid Fixture Type, Mode or Function in FDMXFixtureTypeFunctionsEditorFunctionItem.")))
	{
		return ModeIndex;
	}

	return INDEX_NONE;
}
