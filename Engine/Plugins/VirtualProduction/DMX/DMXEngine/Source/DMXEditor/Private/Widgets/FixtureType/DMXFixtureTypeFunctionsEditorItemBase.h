// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"

class FDMXEditor;
class FDMXFixtureTypeSharedData;
class UDMXEntityFixtureType;


/** An item in a Functions list */
class FDMXFixtureTypeFunctionsEditorItemBase
	: public TSharedFromThis<FDMXFixtureTypeFunctionsEditorItemBase>
{
public:
	/** Constructor */
	FDMXFixtureTypeFunctionsEditorItemBase(const TSharedRef<FDMXEditor>& InDMXEditor, TWeakObjectPtr<UDMXEntityFixtureType> InFixtureType, int32 InModeIndex);

	/** Destructor */
	virtual ~FDMXFixtureTypeFunctionsEditorItemBase() {}

	enum class EItemType : uint8
	{
		Function,
		Matrix
	}; 

	/** Returns the type of the item */
	virtual EItemType GetType() const = 0;

	/** Returns a display name for the icon, for example to use it with the warning and error status */
	virtual FText GetDisplayName() const = 0;

	/** Returns the DMX Editor this item is used in */
	TSharedPtr<FDMXEditor> GetDMXEditor() const { return WeakDMXEditor.Pin(); }

	/** Returns the Fixture Type of the Item */
	TWeakObjectPtr<UDMXEntityFixtureType> GetFixtureType() const { return FixtureType; }

	/** Returns the Index of the Mode in the Modes array */
	int32 GetModeIndex() const;

	/** The warning status of the item. If an empty text is set it means there is no warning. */
	FText WarningStatus;

	/** The error status of the item. If an empty text is set it means there is no error. */
	FText ErrorStatus;

protected:
	/** The Fixture Type which owns the Function */
	TWeakObjectPtr<UDMXEntityFixtureType> FixtureType;

	/** The index of the Mode the Function resides in, in the Fixture Type's Modes array */
	int32 ModeIndex = INDEX_NONE;

	/** Fixture type shared data */
	TSharedPtr<FDMXFixtureTypeSharedData> SharedData;

	/** The DMX Editor that owns this widget */
	TWeakPtr<FDMXEditor> WeakDMXEditor;
};
