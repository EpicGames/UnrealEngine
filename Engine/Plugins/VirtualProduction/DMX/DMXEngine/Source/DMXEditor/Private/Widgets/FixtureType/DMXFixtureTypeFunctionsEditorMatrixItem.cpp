// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXFixtureTypeFunctionsEditorMatrixItem.h"

#include "Library/DMXEntityFixtureType.h"

#include "ScopedTransaction.h"


#define LOCTEXT_NAMESPACE "DMXFixtureTypeFunctionsEditorMatrixItem"

FDMXFixtureTypeFunctionsEditorMatrixItem::FDMXFixtureTypeFunctionsEditorMatrixItem(const TSharedRef<FDMXEditor>& InDMXEditor, TWeakObjectPtr<UDMXEntityFixtureType> InFixtureType, int32 InModeIndex)
	: FDMXFixtureTypeFunctionsEditorItemBase(InDMXEditor, InFixtureType, InModeIndex)
{
	ensureMsgf(FixtureType->Modes.IsValidIndex(InModeIndex) && FixtureType->Modes[InModeIndex].bFixtureMatrixEnabled, TEXT("Trying to construct DMXFixtureTypeFunctionsEditorMatrixItem, but mode is not valid or fixture matrix is not enabled."));
}

FText FDMXFixtureTypeFunctionsEditorMatrixItem::GetDisplayName() const
{
	static const FText DisplayName = LOCTEXT("DisplayName", "Matrix");
	return DisplayName;
}

int32 FDMXFixtureTypeFunctionsEditorMatrixItem::GetStartingChannel() const
{
	if (FixtureType.IsValid())
	{
		const FDMXFixtureMatrix& Matrix = FixtureType->Modes[ModeIndex].FixtureMatrixConfig;
		return Matrix.FirstCellChannel;
	}

	return 1;
}

void FDMXFixtureTypeFunctionsEditorMatrixItem::SetStartingChannel(int32 NewStartingChannel)
{
	if (ensureMsgf(NewStartingChannel > 0 && NewStartingChannel < DMX_MAX_ADDRESS, TEXT("Invalid Starting Channel Provided when setting a Function's Starting Channel.")))
	{
		if (ensureMsgf(FixtureType.IsValid(), TEXT("Invalid Fixture Type in FDMXFixtureTypeFunctionsEditorFunctionItem.")))
		{
			const FScopedTransaction ReorderFunctionTransaction(LOCTEXT("SetChannelTransaction", "Set Fixture Type Matrix Starting Channel"));
			FixtureType->PreEditChange(FDMXFixtureMatrix::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FDMXFixtureMatrix, FirstCellChannel)));

			FDMXFixtureMatrix& Matrix = FixtureType->Modes[ModeIndex].FixtureMatrixConfig;
			Matrix.FirstCellChannel = NewStartingChannel;

			FixtureType->PostEditChange();
		}
	}
}

int32 FDMXFixtureTypeFunctionsEditorMatrixItem::GetNumChannels() const
{
	if (FixtureType.IsValid())
	{
		const FDMXFixtureMatrix& Matrix = FixtureType->Modes[ModeIndex].FixtureMatrixConfig;
		return Matrix.GetNumChannels();
	}

	return 0;
}

const FDMXFixtureMatrix& FDMXFixtureTypeFunctionsEditorMatrixItem::GetFixtureMatrix() const
{
	if (FixtureType.IsValid() && FixtureType->Modes.IsValidIndex(ModeIndex) && FixtureType->Modes[ModeIndex].bFixtureMatrixEnabled)
	{
		return FixtureType->Modes[ModeIndex].FixtureMatrixConfig;
	}

	static const FDMXFixtureMatrix DummyMatrix = FDMXFixtureMatrix();
	return DummyMatrix;
}

#undef LOCTEXT_NAMESPACE
