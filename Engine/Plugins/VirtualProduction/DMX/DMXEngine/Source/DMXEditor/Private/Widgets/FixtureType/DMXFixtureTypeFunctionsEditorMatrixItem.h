// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/FixtureType/DMXFixtureTypeFunctionsEditorItemBase.h"

struct FDMXAttributeName;
struct FDMXFixtureMatrix;


/** The Matrix as an item in a list of Functions */
class FDMXFixtureTypeFunctionsEditorMatrixItem
	: public FDMXFixtureTypeFunctionsEditorItemBase
{
public:
	/** Constructor for Normal Functions */
	FDMXFixtureTypeFunctionsEditorMatrixItem(const TSharedRef<FDMXEditor>& InDMXEditor, TWeakObjectPtr<UDMXEntityFixtureType> InFixtureType, int32 InModeIndex);

	//~ Begin FDMXFixtureTypeFunctionsEditorFunctionItemBase interface
	virtual EItemType GetType() const { return EItemType::Matrix; }
	virtual FText GetDisplayName() const;
	//~ End FDMXFixtureTypeFunctionsEditorFunctionItemBase interface

	/** Returns the Starting Channel of the Matrix */
	int32 GetStartingChannel() const;

	/** Sets the Starting Channel of the Matrix */
	void SetStartingChannel(int32 NewStartingChannel);

	/** Returns the FixtureMatrix */
	const FDMXFixtureMatrix& GetFixtureMatrix() const;

	/** Returns the Number of Channels of the Matrix */
	int32 GetNumChannels() const;
};
