// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXFixtureTypeMatrixFunctionsEditorItem.h"

#include "DMXEditor.h"
#include "DMXFixtureTypeSharedData.h"
#include "Library/DMXEntityFixtureType.h"

#include "ScopedTransaction.h"


FDMXFixtureTypeMatrixFunctionsEditorItem::FDMXFixtureTypeMatrixFunctionsEditorItem(const TSharedRef<FDMXEditor>& InDMXEditor, TWeakObjectPtr<UDMXEntityFixtureType> InFixtureType, int32 InModeIndex, int32 InCellAttributeIndex)
	: ModeIndex(InModeIndex)
	, CellAttributeIndex(InCellAttributeIndex)
	, FixtureType(InFixtureType)
	, SharedData(InDMXEditor->GetFixtureTypeSharedData())
	, WeakDMXEditor(InDMXEditor)
{
	ensureMsgf(FixtureType.IsValid() && FixtureType->Modes.IsValidIndex(ModeIndex) && FixtureType->Modes[ModeIndex].FixtureMatrixConfig.CellAttributes.IsValidIndex(CellAttributeIndex), TEXT("Invalid Cell Attributes for FDMXFixtureTypeMatrixFunctionsEditorItem."));
}

FDMXAttributeName FDMXFixtureTypeMatrixFunctionsEditorItem::GetCellAttributeName() const
{
	if (FixtureType.IsValid() && FixtureType->Modes.IsValidIndex(ModeIndex) && FixtureType->Modes[ModeIndex].FixtureMatrixConfig.CellAttributes.IsValidIndex(CellAttributeIndex))
	{
		const FDMXFixtureCellAttribute& CellAttribute = FixtureType->Modes[ModeIndex].FixtureMatrixConfig.CellAttributes[CellAttributeIndex];

		return CellAttribute.Attribute;
	}

	return FDMXAttributeName();
}

void FDMXFixtureTypeMatrixFunctionsEditorItem::SetCellAttributeName(const FDMXAttributeName& CellAtributeName)
{
	if (FixtureType.IsValid() && FixtureType->Modes.IsValidIndex(ModeIndex) && FixtureType->Modes[ModeIndex].FixtureMatrixConfig.CellAttributes.IsValidIndex(CellAttributeIndex))
	{
		const FScopedTransaction SetCellAttributeTransaction(NSLOCTEXT("DMXFixtureTypeMatrixFunctionsEditorItem", "SetCellAttributeTransaction", "Set Fixture Type Matrix Starting Channel"));
		FixtureType->PreEditChange(nullptr);

		FDMXFixtureCellAttribute& MutableCellAttribute = FixtureType->Modes[ModeIndex].FixtureMatrixConfig.CellAttributes[CellAttributeIndex];

		MutableCellAttribute.Attribute = CellAtributeName;

		FixtureType->PostEditChange();
	}
}

void FDMXFixtureTypeMatrixFunctionsEditorItem::RemoveFromFixtureType()
{
	if (FixtureType.IsValid() && FixtureType->Modes.IsValidIndex(ModeIndex) && FixtureType->Modes[ModeIndex].FixtureMatrixConfig.CellAttributes.IsValidIndex(CellAttributeIndex))
	{
		const FScopedTransaction SetCellAttributeTransaction(NSLOCTEXT("DMXFixtureTypeMatrixFunctionsEditorItem", "RemoveCellAttributeTransaction", "Remove Cell Attribute"));
		FixtureType->PreEditChange(nullptr);

		FixtureType->RemoveCellAttribute(ModeIndex, CellAttributeIndex);

		FixtureType->PostEditChange();
	}
}
