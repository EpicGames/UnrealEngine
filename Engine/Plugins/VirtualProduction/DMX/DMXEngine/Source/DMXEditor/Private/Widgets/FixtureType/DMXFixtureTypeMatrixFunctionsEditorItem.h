// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"

struct FDMXAttributeName;
class FDMXEditor;
class FDMXFixtureTypeCellAttributesEditorItem;
class FDMXFixtureTypeSharedData;
class UDMXEntityFixtureType;


/** A Cell Attribute item in a list */
class FDMXFixtureTypeMatrixFunctionsEditorItem
	: public TSharedFromThis<FDMXFixtureTypeMatrixFunctionsEditorItem>
{
public:
	/** Constructor */
	FDMXFixtureTypeMatrixFunctionsEditorItem(const TSharedRef<FDMXEditor>& InDMXEditor, TWeakObjectPtr<UDMXEntityFixtureType> InFixtureType, int32 InModeIndex, int32 InCellAttributeIndex);

	/** Returns the Attribut Name */
	FDMXAttributeName GetCellAttributeName() const;

	/** Sets the Attribut Name */
	void SetCellAttributeName(const FDMXAttributeName& CellAtributeName);

	/** Removes the Cell Attribute from the Fixture type */
	void RemoveFromFixtureType();

	/** The channel number displayed for the row */
	FText ChannelNumberText;

	/** The warning status of the row. If an empty text is set it means there is no warning. */
	FText WarningStatus;

	/** The error status of the row. If an empty text is set it means there is no error. */
	FText ErrorStatus;

private:
	/** The index of the mode in the Fixture Type's Modes array */
	int32 ModeIndex = INDEX_NONE;

	/** The index of the mode in the Fixture Type's Modes array */
	int32 CellAttributeIndex = INDEX_NONE;

	/** The Fixture Type which owns the Mode */
	TWeakObjectPtr<UDMXEntityFixtureType> FixtureType;

	/** Fixture type shared data */
	TSharedPtr<FDMXFixtureTypeSharedData> SharedData;

	/** The DMX Editor that owns this item */
	TWeakPtr<FDMXEditor> WeakDMXEditor;
};
