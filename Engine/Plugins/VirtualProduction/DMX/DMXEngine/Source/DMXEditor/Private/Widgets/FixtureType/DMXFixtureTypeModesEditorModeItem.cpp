// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXFixtureTypeModesEditorModeItem.h"

#include "DMXEditor.h"
#include "DMXFixtureTypeSharedData.h"
#include "DMXRuntimeUtils.h"
#include "Library/DMXEntityFixtureType.h"
#include "ScopedTransaction.h"
#include "Internationalization/Regex.h"


#define LOCTEXT_NAMESPACE "DMXFixtureTypeModesEditorModeItem"

FDMXFixtureTypeModesEditorModeItem::FDMXFixtureTypeModesEditorModeItem(const TSharedRef<FDMXEditor>& InDMXEditor, TWeakObjectPtr<UDMXEntityFixtureType> InFixtureType, int32 InModeIndex)
	: ModeIndex(InModeIndex)
	, FixtureType(InFixtureType)
	, SharedData(InDMXEditor->GetFixtureTypeSharedData())
	, WeakDMXEditor(InDMXEditor)
{
	if (!ensureMsgf(FixtureType.IsValid() && FixtureType->Modes.IsValidIndex(ModeIndex), TEXT("Invalid Fixture Type or Mode in FDMXFixtureTypeModesEditorModeItem.")))
	{
		return;
	}

	if (!FixtureType->GDTFSource.IsNull())
	{
		const FRegexPattern ModeNumberRegexPattern(TEXT("mode (\\d+)"), ERegexPatternFlags::CaseInsensitive);
		FRegexMatcher ModeNumberRegexMatcher(ModeNumberRegexPattern, FixtureType->Modes[ModeIndex].ModeName);
		if (ModeNumberRegexMatcher.FindNext())
		{
			LexTryParseString(GDTFModeNumber, *ModeNumberRegexMatcher.GetCaptureGroup(1));
		}
	}
}

int32 FDMXFixtureTypeModesEditorModeItem::GetModeIndex() const
{
	if (FixtureType.IsValid() && FixtureType->Modes.IsValidIndex(ModeIndex))
	{
		return ModeIndex;
	}

	return INDEX_NONE;
}

FText FDMXFixtureTypeModesEditorModeItem::GetModeName() const
{
	// Can be invalid when the name is gettered while the modes are changing
	if (FixtureType.IsValid() && FixtureType->Modes.IsValidIndex(ModeIndex))
	{
		return FText::FromString(FixtureType->Modes[ModeIndex].ModeName);
	}

	return FText(LOCTEXT("InvalidModeWarning", "Mode is no longer valid"));
}

bool FDMXFixtureTypeModesEditorModeItem::IsValidModeName(const FText& InModeName, FText& OutInvalidReason)
{
	if (FixtureType.IsValid() && FixtureType->Modes.IsValidIndex(ModeIndex))
	{
		if (FText::TrimPrecedingAndTrailing(InModeName).IsEmpty())
		{
			OutInvalidReason = LOCTEXT("EmptyModeNameError", "The Mode name cannot be blank.");
			return false;
		}

		const FDMXFixtureMode& MyMode = FixtureType->Modes[ModeIndex];
		const FString OldName = MyMode.ModeName;
		const FString& NewName = InModeName.ToString();

		if (OldName == NewName)
		{
			return true;
		}

		const bool bUniqueName = [NewName, this]()
		{
			int32 NumModesWithSameName = 0;
			for (const FDMXFixtureMode& Mode : FixtureType->Modes)
			{
				if (Mode.ModeName == NewName)
				{
					NumModesWithSameName++;
				}
			}

			return NumModesWithSameName < 2;
		}();

		if (bUniqueName)
		{
			return true;
		}
		else
		{
			OutInvalidReason = LOCTEXT("DuplicateModeNameError", "A Mode with this name already exists.");
			return false;
		}
	}

	OutInvalidReason = LOCTEXT("InvalidModeError", "Invalid Fixture Type or Mode no longer exists in Fixture Type");
	return false;
}

void FDMXFixtureTypeModesEditorModeItem::SetModeName(const FText& DesiredModeName, FText& OutUniqueModeName)
{
	if (FixtureType.IsValid())
	{
		const FScopedTransaction SetModeNameTransaction(LOCTEXT("SetModeNameTransaction", "Rename Fixture Mode"));
		FixtureType->PreEditChange(FDMXFixtureMode::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FDMXFixtureMode, ModeName)));

		FString UniqueModeNameString;
		FixtureType->SetModeName(ModeIndex, DesiredModeName.ToString(), UniqueModeNameString);

		OutUniqueModeName = FText::FromString(UniqueModeNameString);

		FixtureType->PostEditChange();
	}
}

#undef LOCTEXT_NAMESPACE
