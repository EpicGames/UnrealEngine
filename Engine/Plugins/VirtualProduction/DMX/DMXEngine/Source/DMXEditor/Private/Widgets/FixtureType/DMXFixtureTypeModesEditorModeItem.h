// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"

class FDMXEditor;
class FDMXFixtureTypeSharedData;
class UDMXEntityFixtureType;


/** A Mode as an item in a list */
class FDMXFixtureTypeModesEditorModeItem
	: public TSharedFromThis<FDMXFixtureTypeModesEditorModeItem>
{
public:
	/** Constructor */
	FDMXFixtureTypeModesEditorModeItem(const TSharedRef<FDMXEditor>& InDMXEditor, TWeakObjectPtr<UDMXEntityFixtureType> InFixtureType, int32 InModeIndex);

	/** Returns the Index of the Mode in the Modes array */
	int32 GetModeIndex() const;

	/** Returns the Mode Name as Text */
	FText GetModeName() const;

	/** 
	 * GDTF modes typically start with "Mode" and a number, e.g. "Mode 3", the latest revision is the last entry.
	 * This function returns the number of the mode or INDEX_NONE if the mode is not enumerated.
	 */
	int32 GetGDTFModeNumber() const { return GDTFModeNumber; }

	/** Returns if the Name is valid for the Mode. If the Mode Name is not valid, returns a non-empty OutInvalidReason */
	bool IsValidModeName(const FText& InModeName, FText& OutInvalidReason);

	/** Sets a Mode Name, makes it unique. Returns the OutUniqueModeName as output argument */
	void SetModeName(const FText& DesiredModeName, FText& OutUniqueModeName);

private:
	/** The index of the mode in the Fixture Type's Modes array */
	int32 ModeIndex = INDEX_NONE;

	/** The number of the mode or INDEX_NONE if the mode is not enumerated. See GetGDTFModeNumber. */
	int32 GDTFModeNumber = INDEX_NONE;

	/** The Fixture Type which owns the Mode */
	TWeakObjectPtr<UDMXEntityFixtureType> FixtureType;

	/** Fixture type shared data */
	TSharedPtr<FDMXFixtureTypeSharedData> SharedData;

	/** The DMX Editor that owns this widget */
	TWeakPtr<FDMXEditor> WeakDMXEditor;
};
