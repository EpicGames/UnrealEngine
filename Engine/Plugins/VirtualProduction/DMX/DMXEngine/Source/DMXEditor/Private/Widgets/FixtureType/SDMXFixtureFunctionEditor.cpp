// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXFixtureFunctionEditor.h"

#include "DMXAttributeToDefaultPhyiscalProperties.h"
#include "DMXConversions.h"
#include "DMXEditor.h"
#include "DMXFixtureTypeSharedData.h"
#include "DMXPhysicalUnitToDefaultValueRange.h"
#include "IStructureDetailsView.h"
#include "Library/DMXEntityFixtureType.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDMXFixtureFunctionEditor"

namespace UE::DMX::SDMXFixtureFunctionEditor::Private
{
	FDMXFixtureFunction* GetFixtureFunction(UDMXEntityFixtureType& FixtureType, int32 ModeIndex, int32 FunctionIndex)
	{
		if (!ensureMsgf(FixtureType.Modes.IsValidIndex(ModeIndex), TEXT("Trying to clamp Function Default Value, but Mode Index is not valid.")))
		{
			return nullptr;
		}

		FDMXFixtureMode& Mode = FixtureType.Modes[ModeIndex];
		if (!ensureMsgf(Mode.Functions.IsValidIndex(FunctionIndex), TEXT("Trying to clamp Function Default Value, but Function Index is not valid.")))
		{
			return nullptr;
		}

		return &Mode.Functions[FunctionIndex];
	}

	/**
	 * Clamps the Default Value of the Function by its Data Type
	 *
	 * @param ModeIndex						The Index of the Mode in which the Functions reside
	 * @param FunctionIndex					The Index of the Function for which the Default Value is clamped
	 */
	void ClampFunctionDefautValueByDataType(UDMXEntityFixtureType& FixtureType, int32 ModeIndex, int32 FunctionIndex)
	{
		if (FDMXFixtureFunction* FunctionPtr = GetFixtureFunction(FixtureType, ModeIndex, FunctionIndex))
		{
			const uint32 SafeDefaultValue = FMath::Min(static_cast<int64>(TNumericLimits<uint32>::Max()), FunctionPtr->DefaultValue);
			const uint32 ClampedDefaultValue = FDMXConversions::ClampValueBySignalFormat(SafeDefaultValue, FunctionPtr->DataType);

			FunctionPtr->DefaultValue = ClampedDefaultValue;
			FunctionPtr->UpdatePhysicalDefaultValue();
		}
	}

	/**
	 * Clamps the Physical Default Value of the Function by its Physical Unit^.
	 *
	 * @param ModeIndex						The Index of the Mode in which the Functions reside
	 * @param FunctionIndex					The Index of the Function for which the Physical Default Value is clamped
	 */
	void ClampFunctionPhysicalDefautValueByPhysicalUnit(UDMXEntityFixtureType& FixtureType, int32 ModeIndex, int32 FunctionIndex)
	{
		if (FDMXFixtureFunction* FunctionPtr = GetFixtureFunction(FixtureType, ModeIndex, FunctionIndex))
		{
			const double PhysicalMin = FMath::Min(FunctionPtr->GetPhysicalFrom(), FunctionPtr->GetPhysicalTo());
			const double PhysicalMax = FMath::Max(FunctionPtr->GetPhysicalFrom(), FunctionPtr->GetPhysicalTo());

			const double ClampedPhysicalDefaultValue = FMath::Clamp(FunctionPtr->GetPhysicalDefaultValue(), PhysicalMin, PhysicalMax);
			FunctionPtr->SetPhysicalDefaultValue(ClampedPhysicalDefaultValue);
		}
	}

	/**
	 * Updates the Physical Default Value of the Function from its Physical Unit.
	 *
	 * @param ModeIndex						The Index of the Mode in which the Functions reside
	 * @param FunctionIndex					The Index of the Function for which the Physical Default Value is clamped
	 */
	void UpdateFunctionDefaultValueFromPhysicalUnit(UDMXEntityFixtureType& FixtureType, int32 ModeIndex, int32 FunctionIndex)
	{
		if (FDMXFixtureFunction* FunctionPtr = GetFixtureFunction(FixtureType, ModeIndex, FunctionIndex))
		{
			const TPair<double, double>& PhysicalUnitValueRange = FDMXPhysicalUnitToDefaultValueRange::GetValueRange(FunctionPtr->GetPhysicalUnit());

			FunctionPtr->SetPhysicalValueRange(PhysicalUnitValueRange.Key, PhysicalUnitValueRange.Value);
		}
	}

	void UpdateFunctionPhysicalUnitFromAttribute(UDMXEntityFixtureType& FixtureType, int32 ModeIndex, int32 FunctionIndex)
	{
		if (FDMXFixtureFunction* FunctionPtr = GetFixtureFunction(FixtureType, ModeIndex, FunctionIndex))
		{
			FDMXAttributeToDefaultPhyiscalProperties::ResetToDefaultPhysicalProperties(*FunctionPtr);
		}
	}
}

void SDMXFixtureFunctionEditor::Construct(const FArguments& InArgs, const TSharedRef<FDMXEditor>& InDMXEditor)
{
	WeakDMXEditor = InDMXEditor;
	FixtureTypeSharedData = InDMXEditor->GetFixtureTypeSharedData();

	UDMXEntityFixtureType::GetOnFixtureTypeChanged().AddSP(this, &SDMXFixtureFunctionEditor::OnFixtureTypePropertiesChanged);
	FixtureTypeSharedData->OnFixtureTypesSelected.AddSP(this, &SDMXFixtureFunctionEditor::Refresh);
	FixtureTypeSharedData->OnModesSelected.AddSP(this, &SDMXFixtureFunctionEditor::Refresh);
	FixtureTypeSharedData->OnFunctionsSelected.AddSP(this, &SDMXFixtureFunctionEditor::Refresh);

	// Create a Struct Details View. This is not the most convenient type to work with as a property type customization for the FDMXFixtureFunction struct cannot be used.
	// Reason is soley significant performance gains, it's much faster than the easier approach with a UDMXEntityFixtureType customization as it was used up to 4.27.
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = true;
	DetailsViewArgs.bHideSelectionTip = false;
	DetailsViewArgs.bSearchInitialKeyFocus = true;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bShowModifiedPropertiesOption = false;
	DetailsViewArgs.bShowObjectLabel = false;
	DetailsViewArgs.bForceHiddenPropertyVisibility = false;
	DetailsViewArgs.bShowScrollBar = false;
	DetailsViewArgs.NotifyHook = this;

	FStructureDetailsViewArgs StructureDetailsViewArgs;

	using namespace UE::DMX;

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	StructDetailsView = PropertyModule.CreateStructureDetailView(DetailsViewArgs, StructureDetailsViewArgs, nullptr);
	StructDetailsView->GetDetailsView()->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateSP(this, &SDMXFixtureFunctionEditor::IsPropertyVisible));

	StructDetailsViewWidget = StructDetailsView->GetWidget();

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			StructDetailsViewWidget.ToSharedRef()
		]

		+ SVerticalBox::Slot()
		[
			SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SAssignNew(InfoTextBlock, STextBlock)
				.Visibility(EVisibility::Collapsed)
			]
		]
	];

	Refresh();
}

void SDMXFixtureFunctionEditor::NotifyPreChange(FProperty* PropertyAboutToChange)
{
	UDMXEntityFixtureType* FixtureType = WeakFixtureType.Get();
	if (PropertyAboutToChange && FixtureType)
	{
		const FName PropertyName = PropertyAboutToChange->GetFName();

		if (PropertyName == GET_MEMBER_NAME_CHECKED(FDMXFixtureFunction, FunctionName))
		{
			Transaction = MakeUnique<FScopedTransaction>(LOCTEXT("SetFunctionNameTransaction", "Set Fixture Function Name"));
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(FDMXFixtureFunction, Channel))
		{
			Transaction = MakeUnique<FScopedTransaction>(LOCTEXT("SetChannelTransaction", "Set Fixture Function Starting Channel"));
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(FDMXFixtureFunction, DataType))
		{
			Transaction = MakeUnique<FScopedTransaction>(LOCTEXT("SetDataTypeTransaction", "Set Data Type of Function"));
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(FDMXFixtureFunction, DefaultValue))
		{
			Transaction = MakeUnique<FScopedTransaction>(LOCTEXT("SetDefaultValueTransaction", "Set Default Value of Function"));
		}
		else if (PropertyName == FDMXFixtureFunction::GetPhysicalDefaultValuePropertyName())
		{
			Transaction = MakeUnique<FScopedTransaction>(LOCTEXT("SetPhysicalValueTransaction", "Set Physical Default Value of Function"));
		}
		else if (PropertyName == FDMXFixtureFunction::GetPhysicalFromPropertyName() ||
			PropertyName == FDMXFixtureFunction::GetPhysicalToPropertyName())
		{
			Transaction = MakeUnique<FScopedTransaction>(LOCTEXT("SetPhysicalRangeTransaction", "Set Physical Range of Function"));
		}
		else if (PropertyName == FDMXFixtureFunction::GetPhysicalUnitPropertyName())
		{
			Transaction = MakeUnique<FScopedTransaction>(LOCTEXT("SetPhysicalUnitTransaction", "Set Physical Unit of Function"));
		}

		FixtureType->Modify();
		FixtureType->PreEditChange(PropertyAboutToChange);
	}
}

void SDMXFixtureFunctionEditor::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	using namespace UE::DMX::SDMXFixtureFunctionEditor::Private;

	UDMXEntityFixtureType* FixtureType = WeakFixtureType.Get();
	FDMXFixtureFunction* FunctionBeingEditedPtr = GetFunctionBeingEdited();

	if (PropertyThatChanged && FixtureType && FunctionBeingEditedPtr)
	{
		const FName PropertyName = PropertyChangedEvent.GetPropertyName();

		if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
		{
			if (PropertyName == GET_MEMBER_NAME_CHECKED(FDMXFixtureFunction, FunctionName))
			{
				// Make a unique function name
				FString OutUniqueFunctionName;
				FixtureType->SetFunctionName(ModeIndex, FunctionIndex, FunctionBeingEditedPtr->FunctionName, OutUniqueFunctionName);
			}
			else if (PropertyName == GET_MEMBER_NAME_CHECKED(FDMXFixtureFunction, Channel))
			{
				int32 ResultingStartingChannel;
				FixtureType->SetFunctionStartingChannel(ModeIndex, FunctionIndex, FunctionBeingEditedPtr->Channel, ResultingStartingChannel);
			}
			else if (PropertyName == GET_MEMBER_NAME_CHECKED(FDMXFixtureFunction, DataType))
			{
				FixtureType->AlignFunctionChannels(ModeIndex);
				UpdateFunctionDefaultValueFromPhysicalUnit(*FixtureType, ModeIndex, FunctionIndex);
			}
			else if (PropertyName == GET_MEMBER_NAME_CHECKED(FDMXFixtureFunction, DefaultValue))
			{
				ClampFunctionDefautValueByDataType(*FixtureType, ModeIndex, FunctionIndex);
			}
			else if (PropertyName == FDMXFixtureFunction::GetPhysicalDefaultValuePropertyName() || 
					PropertyName == FDMXFixtureFunction::GetPhysicalFromPropertyName() ||
					PropertyName == FDMXFixtureFunction::GetPhysicalToPropertyName())
			{
				ClampFunctionPhysicalDefautValueByPhysicalUnit(*FixtureType, ModeIndex, FunctionIndex);
			}
			else if (PropertyName == FDMXFixtureFunction::GetPhysicalUnitPropertyName())
			{
				UpdateFunctionDefaultValueFromPhysicalUnit(*FixtureType, ModeIndex, FunctionIndex);
			}

			FPropertyChangedEvent ObjectPropertyChangedEvent(PropertyChangedEvent);
			FixtureType->PostEditChangeProperty(ObjectPropertyChangedEvent);
		}
	}

	Transaction.Reset();
}

void SDMXFixtureFunctionEditor::Refresh()
{
	const TArray<TWeakObjectPtr<UDMXEntityFixtureType>>& SelectedFixtureTypes = FixtureTypeSharedData->GetSelectedFixtureTypes();
	const TArray<int32>& SelectedModeIndices = FixtureTypeSharedData->GetSelectedModeIndices();
	const TArray<int32>& SelectedFunctionIndices = FixtureTypeSharedData->GetSelectedFunctionIndices();

	bool bValidSelection = false;
	if (SelectedFixtureTypes.Num() == 1 && SelectedModeIndices.Num() == 1 && SelectedFunctionIndices.Num() == 1)
	{
		if (UDMXEntityFixtureType* FixtureType = SelectedFixtureTypes[0].Get())
		{
			if (FixtureType->Modes.IsValidIndex(SelectedModeIndices[0]))
			{
				const FDMXFixtureMode& Mode = FixtureType->Modes[SelectedModeIndices[0]];
				if (Mode.Functions.IsValidIndex(SelectedFunctionIndices[0]))
				{
					SetFunction(FixtureType, SelectedModeIndices[0], SelectedFunctionIndices[0]);
					bValidSelection = true;
				}
			}
		}
	}

	if (!bValidSelection)
	{
		const FText ErrorText = [SelectedFunctionIndices]()
		{
			if (SelectedFunctionIndices.Num() > 1)
			{
				return LOCTEXT("MultiEditingNotSupportedWarning", "Multi editing Functions is not supported");
			}

			return LOCTEXT("NoFunctionSelectedWarning", "No Function selected");
		}();

		InfoTextBlock->SetText(ErrorText);
		InfoTextBlock->SetVisibility(EVisibility::Visible);
		StructDetailsViewWidget->SetVisibility(EVisibility::Collapsed);
	}
}

void SDMXFixtureFunctionEditor::SetFunction(UDMXEntityFixtureType* InFixtureType, int32 InModeIndex, int32 InFunctionIndex)
{
	bool bSuccess = false;
	if (ensureMsgf(InFixtureType && InFixtureType->Modes.IsValidIndex(InModeIndex), TEXT("Invalid Fixture Type or Mode Index when setting the Mode in the Function editor.")))
	{
		WeakFixtureType = InFixtureType;
		ModeIndex = InModeIndex;
		FDMXFixtureMode& Mode = InFixtureType->Modes[ModeIndex];

		if (ensureMsgf(Mode.Functions.IsValidIndex(InFunctionIndex), TEXT("Invalid Function Index specified when setting the Function in the Function editor.")))
		{
			FunctionIndex = InFunctionIndex;
			FDMXFixtureFunction& Function = Mode.Functions[FunctionIndex];
			const TSharedRef<FStructOnScope> FunctionStructOnScope = MakeShared<FStructOnScope>(FDMXFixtureFunction::StaticStruct(), (uint8*)&Function);

			StructDetailsView->SetStructureData(FunctionStructOnScope);

			StructDetailsViewWidget->SetVisibility(EVisibility::Visible);
			InfoTextBlock->SetVisibility(EVisibility::Collapsed);
			bSuccess = true;
		}
	}
	
	if (!bSuccess)
	{
		InfoTextBlock->SetText(LOCTEXT("CannotCreateDetailViewForFunctionWarning", "Cannot create Detail View for Function. Fixture Type, Mode or Function no longer exist."));
		InfoTextBlock->SetVisibility(EVisibility::Visible);
		StructDetailsViewWidget->SetVisibility(EVisibility::Collapsed);
	}
}

void SDMXFixtureFunctionEditor::OnFixtureTypePropertiesChanged(const UDMXEntityFixtureType* FixtureType)
{
	if (!Transaction.IsValid() && FixtureType == WeakFixtureType.Get())
	{
		Refresh();
	}
}

bool SDMXFixtureFunctionEditor::IsPropertyVisible(const FPropertyAndParent& PropertyAndParent) const
{
	// Hide all mode properties
	if (PropertyAndParent.Property.GetFName() == GET_MEMBER_NAME_CHECKED(FDMXFixtureMode, bAutoChannelSpan) ||
		PropertyAndParent.Property.GetFName() == GET_MEMBER_NAME_CHECKED(FDMXFixtureMode, bFixtureMatrixEnabled) ||
		PropertyAndParent.Property.GetFName() == GET_MEMBER_NAME_CHECKED(FDMXFixtureMode, ChannelSpan) ||
		PropertyAndParent.Property.GetFName() == GET_MEMBER_NAME_CHECKED(FDMXFixtureMode, FixtureMatrixConfig) ||
		PropertyAndParent.Property.GetFName() == GET_MEMBER_NAME_CHECKED(FDMXFixtureMode, Functions) ||
		PropertyAndParent.Property.GetFName() == GET_MEMBER_NAME_CHECKED(FDMXFixtureMode, ModeName))
	{
		return false;
	}

	return true;
}

FDMXFixtureMode* SDMXFixtureFunctionEditor::GetModeBeingEdited() const
{
	if (UDMXEntityFixtureType* FixtureType = WeakFixtureType.Get())
	{
		if (FixtureType->Modes.IsValidIndex(ModeIndex))
		{
			return &FixtureType->Modes[ModeIndex];
		}
	}

	return nullptr;
}

FDMXFixtureFunction* SDMXFixtureFunctionEditor::GetFunctionBeingEdited() const
{
	FDMXFixtureMode* EditedModePtr = GetModeBeingEdited();
	if (EditedModePtr && EditedModePtr->Functions.IsValidIndex(FunctionIndex))
	{
		return &EditedModePtr->Functions[FunctionIndex];
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
