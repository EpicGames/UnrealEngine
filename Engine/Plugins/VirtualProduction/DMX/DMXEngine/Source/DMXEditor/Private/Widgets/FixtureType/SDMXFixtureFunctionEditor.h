// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotifyHook.h"
#include "ScopedTransaction.h"
#include "Widgets/SCompoundWidget.h"

class FDMXEditor;
struct FDMXFixtureFunction;
struct FDMXFixtureMode;
class FDMXFixtureTypeSharedData;
struct FPropertyAndParent;
class IStructureDetailsView;
class STextBlock;
class SWidget;
class UDMXEntityFixtureType;

/** Editor for a single Function struct inside a UDMXEntityFixtureType. */
class SDMXFixtureFunctionEditor
	: public SCompoundWidget
	, public FNotifyHook
{
public:
	SLATE_BEGIN_ARGS(SDMXFixtureFunctionEditor)
	{}

	SLATE_END_ARGS()

	/** Constructs this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<FDMXEditor>& InDMXEditor);

protected:
	//~ Begin FNotifyHook Interface
	virtual void NotifyPreChange(FProperty* PropertyAboutToChange) override;
	virtual void NotifyPostChange(const FPropertyChangedEvent & PropertyChangedEvent, FProperty * PropertyThatChanged) override;
	//~ End FNotifyHook Interface

private:
	/** Refereshes the Mode displayed */
	void Refresh();

	/** Sets the Mode that is being edited */
	void SetFunction(UDMXEntityFixtureType* InFixtureType, int32 InModeIndex, int32 InFunctionIndex);

	/** Called when the properties of the fixture type changed */
	void OnFixtureTypePropertiesChanged(const UDMXEntityFixtureType* FixtureType);

	/** Returns true if the properties are visible */
	bool IsPropertyVisible(const FPropertyAndParent& PropertyAndParent) const;
	
	/** Returns the Mode currently being edited, or nullptr if the current selection cannot be resolved. */
	FDMXFixtureMode* GetModeBeingEdited() const;

	/** Returns the Function currently being edited */
	FDMXFixtureFunction* GetFunctionBeingEdited() const;

	/** Fixture type of the Function currently being edited */
	TWeakObjectPtr<UDMXEntityFixtureType> WeakFixtureType;

	/** Mode Index currently being edited */
	int32 ModeIndex;

	/** Function Index currently being edited */
	int32 FunctionIndex;

	/** The current transaction or nullptr if there's no transaction ongoing */
	TUniquePtr<FScopedTransaction> Transaction;
	
	/** Details view for the Function Struct being edited */
	TSharedPtr<IStructureDetailsView> StructDetailsView;

	/** Widget of the Details view  */
	TSharedPtr<SWidget> StructDetailsViewWidget;

	/** Text block that displays info in case the struct details view cannot be shown */
	TSharedPtr<STextBlock> InfoTextBlock;

	/** Shared Data for */
	TSharedPtr<FDMXFixtureTypeSharedData> FixtureTypeSharedData;

	/** The DMX Editor that owns this widget */
	TWeakPtr<FDMXEditor> WeakDMXEditor;
};
