// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXFixtureModeEditor.h"

#include "DMXEditor.h"
#include "DMXFixtureTypeSharedData.h"
#include "Library/DMXEntityFixtureType.h"

#include "IStructureDetailsView.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "SDMXFixtureModeEditor"

void SDMXFixtureModeEditor::Construct(const FArguments& InArgs, const TSharedRef<FDMXEditor>& InDMXEditor)
{
	WeakDMXEditor = InDMXEditor;
	FixtureTypeSharedData = InDMXEditor->GetFixtureTypeSharedData();

	UDMXEntityFixtureType::GetOnFixtureTypeChanged().AddSP(this, &SDMXFixtureModeEditor::OnFixtureTypePropertiesChanged);
	FixtureTypeSharedData->OnFixtureTypesSelected.AddSP(this, &SDMXFixtureModeEditor::RequestRefresh);
	FixtureTypeSharedData->OnFixtureTypesSelected.AddSP(this, &SDMXFixtureModeEditor::RequestRefresh);
	FixtureTypeSharedData->OnModesSelected.AddSP(this, &SDMXFixtureModeEditor::RequestRefresh);

	// Create a Struct Details View. This is not the most convenient type to work with as a property type customization for the FDMXFixtureMode struct cannot be used.
	// Reason is soley significant performance gains, it's much faster than the easier approach with a UDMXEntityFixtureType customization as it was used up to 4.27.
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = true;
	DetailsViewArgs.bHideSelectionTip = false;
	DetailsViewArgs.bSearchInitialKeyFocus = true;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bShowModifiedPropertiesOption = false;
	DetailsViewArgs.bShowObjectLabel = false;
	DetailsViewArgs.bForceHiddenPropertyVisibility = false;
	DetailsViewArgs.bShowScrollBar = true;
	DetailsViewArgs.NotifyHook = this;

	FStructureDetailsViewArgs StructureDetailsViewArgs;

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	StructDetailsView = PropertyModule.CreateStructureDetailView(DetailsViewArgs, StructureDetailsViewArgs, nullptr);
	StructDetailsView->GetDetailsView()->SetCustomFilterLabel(LOCTEXT("SearchModePropertiesLabel", "Search Mode Properties"));
	StructDetailsView->GetDetailsView()->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateSP(this, &SDMXFixtureModeEditor::IsPropertyVisible));

	StructDetailsViewWidget = StructDetailsView->GetWidget();

	ChildSlot
	[
		SNew(SOverlay)

		+ SOverlay::Slot()
		[
			StructDetailsViewWidget.ToSharedRef()
		]

		+ SOverlay::Slot()
		[
			SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SAssignNew(InfoTextBlock, STextBlock)
				.Visibility(EVisibility::Collapsed)
			]
		]
	];

	RefreshInternal();
}

void SDMXFixtureModeEditor::RequestRefresh()
{
	if (!RefreshHandle.IsValid())
	{
		RefreshHandle = GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateSP(this, &SDMXFixtureModeEditor::RefreshInternal));
	}
}

void SDMXFixtureModeEditor::NotifyPreChange(FProperty* PropertyAboutToChange)
{
	UDMXEntityFixtureType* FixtureType = WeakFixtureType.Get();
	FDMXFixtureMode* ModeBeingEditedPtr = GetModeBeingEdited();
	if (PropertyAboutToChange && FixtureType && ModeBeingEditedPtr)
	{
		FixtureType->PreEditChange(PropertyAboutToChange);

		const FName PropertyName = PropertyAboutToChange->GetFName();

		if (PropertyName == GET_MEMBER_NAME_CHECKED(FDMXFixtureMode, ModeName))
		{
			Transaction = MakeUnique<FScopedTransaction>(LOCTEXT("RenameModeTransaction", "Mode Name"));
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(FDMXFixtureMode, bFixtureMatrixEnabled))
		{
			const FText FixtureMatrixEnabledTransactionText = FText::Format(
				LOCTEXT("AddFunctionTransaction", "Matrix of Mode {0}"),
				ModeBeingEditedPtr->bFixtureMatrixEnabled == false ? LOCTEXT("FixtureMatrixEnabled", "Enabled") : LOCTEXT("FixtureMatrixDisabled", "Disabled")
			);
			Transaction = MakeUnique<FScopedTransaction>(FixtureMatrixEnabledTransactionText);
			
			FixtureType->SetFixtureMatrixEnabled(ModeIndex, !ModeBeingEditedPtr->bFixtureMatrixEnabled);
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(FDMXFixtureMode, bAutoChannelSpan))
		{
			const FText SetAutoChannelSpanTransactionText = FText::Format(
				LOCTEXT("SetAutoChannelSpanTransaction", "Auto Channel Span of Mode {0}"),
				ModeBeingEditedPtr->bAutoChannelSpan == false ? LOCTEXT("AutoChannelSpanEnabled", "Enabled") : LOCTEXT("AutoChannelSpanDisabled", "Disabled")
			);
			Transaction = MakeUnique<FScopedTransaction>(SetAutoChannelSpanTransactionText);
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(FDMXFixtureMode, ChannelSpan))
		{
			Transaction = MakeUnique<FScopedTransaction>(LOCTEXT("SetChannelSpanTransaction", "Channel Span of Mode"));
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(FDMXFixtureCellAttribute, DataType))
		{
			Transaction = MakeUnique<FScopedTransaction>(LOCTEXT("SetCellAttributeDataType", "CellAttributeDatatType"));
		}
	}
}

void SDMXFixtureModeEditor::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	UDMXEntityFixtureType* FixtureType = WeakFixtureType.Get();
	FDMXFixtureMode* ModeBeingEditedPtr = GetModeBeingEdited();
	if (FixtureType && ModeBeingEditedPtr)
	{
		const FName PropertyName = PropertyChangedEvent.GetPropertyName();

		if (PropertyName == GET_MEMBER_NAME_CHECKED(FDMXFixtureMode, ModeName))
		{
			// Make a unique Mode Name
			FString OutUniqueModeName;
			FixtureType->SetModeName(ModeIndex, ModeBeingEditedPtr->ModeName, OutUniqueModeName);
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(FDMXFixtureMatrix, XCells))
		{
			constexpr bool bSelectMatrix = true;
			FixtureTypeSharedData->SetFunctionAndMatrixSelection(TArray<int32>(), bSelectMatrix);
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(FDMXFixtureMatrix, YCells))
		{
			constexpr bool bSelectMatrix = true;
			FixtureTypeSharedData->SetFunctionAndMatrixSelection(TArray<int32>(), bSelectMatrix);
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(FDMXFixtureMode, bFixtureMatrixEnabled))
		{
			FixtureTypeSharedData->SetFunctionAndMatrixSelection(TArray<int32>(), ModeBeingEditedPtr->bFixtureMatrixEnabled);
			RequestRefresh();
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(FDMXFixtureMode, bAutoChannelSpan))
		{
			if (ModeBeingEditedPtr->bAutoChannelSpan)
			{
				FixtureType->UpdateChannelSpan(ModeIndex);
			}
			RequestRefresh();
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(FDMXFixtureCellAttribute, DataType))
		{
			// Disable the matrix and enable it again so surrounding functions align
			FixtureType->SetFixtureMatrixEnabled(ModeIndex, false);
			FixtureType->SetFixtureMatrixEnabled(ModeIndex, true);

			FixtureType->UpdateChannelSpan(ModeIndex);
			RequestRefresh();
		}
		FPropertyChangedEvent ObjectPropertyChangedEvent(PropertyChangedEvent);
		FixtureType->PostEditChangeProperty(ObjectPropertyChangedEvent);
	}

	Transaction.Reset();
}

void SDMXFixtureModeEditor::RefreshInternal()
{
	RefreshHandle.Invalidate();

	const TArray<TWeakObjectPtr<UDMXEntityFixtureType>>& SelectedFixtureTypes = FixtureTypeSharedData->GetSelectedFixtureTypes();
	const TArray<int32>& SelectedModeIndices = FixtureTypeSharedData->GetSelectedModeIndices();

	bool bValidSelection = false;
	if (SelectedFixtureTypes.Num() == 1 && SelectedModeIndices.Num() == 1)
	{
		if (UDMXEntityFixtureType* FixtureType = SelectedFixtureTypes[0].Get())
		{
			if (FixtureType->Modes.IsValidIndex(SelectedModeIndices[0]))
			{
				SetMode(FixtureType, SelectedModeIndices[0]);
				bValidSelection = true;
			}
		}
	}

	if (!bValidSelection)
	{
		const FText ErrorText = [SelectedModeIndices]()
		{
			if (SelectedModeIndices.Num() > 1)
			{
				return LOCTEXT("MultiEditingNotSupportedWarning", "Multi editing Modes is not supported");
			}

			return LOCTEXT("NoModeSelectedWarning", "No Mode selected");
		}();

		InfoTextBlock->SetText(ErrorText);
		InfoTextBlock->SetVisibility(EVisibility::Visible);
		StructDetailsViewWidget->SetVisibility(EVisibility::Collapsed);
	}
}

void SDMXFixtureModeEditor::SetMode(UDMXEntityFixtureType* InFixtureType, int32 InModeIndex)
{
	if (ensureMsgf(InFixtureType && InFixtureType->Modes.IsValidIndex(InModeIndex), TEXT("Invalid Fixture Type or Mode when setting the Mode in the Mode editor.")))
	{
		WeakFixtureType = InFixtureType;
		ModeIndex = InModeIndex;
		FDMXFixtureMode& Mode = InFixtureType->Modes[ModeIndex];
		const TSharedRef<FStructOnScope> ModeStructOnScope = MakeShared<FStructOnScope>(FDMXFixtureMode::StaticStruct(), (uint8*)&Mode);

		StructDetailsView->SetStructureData(ModeStructOnScope);

		StructDetailsViewWidget->SetVisibility(EVisibility::Visible);
		InfoTextBlock->SetVisibility(EVisibility::Collapsed);
	}
	else
	{
		InfoTextBlock->SetText(LOCTEXT("CannotCreateDetailViewForModeWarning", "Cannot create Detail View for Mode. Fixture Type or Mode no longer exist."));
		InfoTextBlock->SetVisibility(EVisibility::Visible);
		StructDetailsViewWidget->SetVisibility(EVisibility::Collapsed);
	}
}

void SDMXFixtureModeEditor::OnFixtureTypePropertiesChanged(const UDMXEntityFixtureType* FixtureType)
{
	if (!Transaction.IsValid() && FixtureType == WeakFixtureType.Get())
	{
		RequestRefresh();
	}
}

bool SDMXFixtureModeEditor::IsPropertyVisible(const FPropertyAndParent& PropertyAndParent) const
{
	if (PropertyAndParent.Property.GetFName() == GET_MEMBER_NAME_CHECKED(FDMXFixtureMode, Functions))
	{
		return false;
	}
	if (PropertyAndParent.Property.GetFName() == GET_MEMBER_NAME_CHECKED(FDMXFixtureMode, FixtureMatrixConfig))
	{
		if (FDMXFixtureMode* ModeBeingEditedPtr = GetModeBeingEdited())
		{
			return ModeBeingEditedPtr->bFixtureMatrixEnabled;
		}
	}

	return true;
}

FDMXFixtureMode* SDMXFixtureModeEditor::GetModeBeingEdited() const
{
	if (UDMXEntityFixtureType* FixtureType = WeakFixtureType.Get())
	{
		if (FixtureType->Modes.IsValidIndex(ModeIndex))
		{
			return &FixtureType->Modes[ModeIndex];
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
