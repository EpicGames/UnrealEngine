// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Misc/NotifyHook.h"

class FDMXEditor;
struct FDMXFixtureMode;
class FDMXFixtureTypeSharedData;
class UDMXEntityFixtureType;

struct FPropertyAndParent;
class FScopedTransaction;
class IStructureDetailsView;
class SBorder;
class STextBlock;


/** Editor for a single Mode struct inside a UDMXEntityFixtureType. */
class SDMXFixtureModeEditor
	: public SCompoundWidget
	, public FNotifyHook
{
public:
	SLATE_BEGIN_ARGS(SDMXFixtureModeEditor)
	{}

	SLATE_END_ARGS()

	/** Constructs this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<FDMXEditor>& InDMXEditor);

	/** Requests to refresh the widget on the next tick */
	void RequestRefresh();

protected:
	//~ Begin FNotifyHook Interface
	virtual void NotifyPreChange(FProperty* PropertyAboutToChange) override;
	virtual void NotifyPostChange(const FPropertyChangedEvent & PropertyChangedEvent, FProperty * PropertyThatChanged) override;
	//~ End FNotifyHook Interface

private:
	/** Refereshes the Mode displayed instantly */
	void RefreshInternal();

	/** Sets the Mode that is being edited */
	void SetMode(UDMXEntityFixtureType* InFixtureType, int32 InModeIndex);

	/** Called when the properties of the fixture type changed */
	void OnFixtureTypePropertiesChanged(const UDMXEntityFixtureType* FixtureType);

	/** Returns true if the properties are visible */
	bool IsPropertyVisible(const FPropertyAndParent& PropertyAndParent) const;

	/** Returns the Mode currently being edited */
	FDMXFixtureMode* GetModeBeingEdited() const;

	/** Fixture type of the Mode currently being edited */
	TWeakObjectPtr<UDMXEntityFixtureType> WeakFixtureType;

	/** Mode Index currently being edited */
	int32 ModeIndex;

	/** The current transaction or nullptr if there's no transaction ongoing */
	TUniquePtr<FScopedTransaction> Transaction;

	/** Details view for the Mode Struct being edited */
	TSharedPtr<IStructureDetailsView> StructDetailsView;

	/** Widget of the Details view */
	TSharedPtr<SWidget> StructDetailsViewWidget;

	/** Text block that displays info in case the struct details view cannot be shown */
	TSharedPtr<STextBlock> InfoTextBlock;

	/** Timer handle set while refresh is requested but not carried out yet */
	FTimerHandle RefreshHandle;

	/** Shared Data for */
	TSharedPtr<FDMXFixtureTypeSharedData> FixtureTypeSharedData;

	/** The DMX Editor that owns this widget */
	TWeakPtr<FDMXEditor> WeakDMXEditor;
};
