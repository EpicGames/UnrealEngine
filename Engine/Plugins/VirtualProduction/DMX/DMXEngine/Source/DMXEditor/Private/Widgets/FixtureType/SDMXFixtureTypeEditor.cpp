// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXFixtureTypeEditor.h"

#include "DMXEditor.h"
#include "DMXEditorUtils.h"
#include "DMXFixtureTypeSharedData.h"
#include "IDetailsView.h"
#include "PropertyEditorModule.h"
#include "SDMXFixtureFunctionEditor.h"
#include "SDMXFixtureModeEditor.h"
#include "SDMXFixtureTypeFunctionsEditor.h"
#include "SDMXFixtureTypeModesEditor.h"
#include "SDMXFixtureTypeTree.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/DMXImport.h"
#include "Library/DMXLibrary.h"
#include "Library/DMXEntityFixtureType.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SSplitter.h"


void SDMXFixtureTypeEditor::Construct(const FArguments& InArgs, const TSharedRef<FDMXEditor>& InDMXEditor)
{
	SDMXEntityEditor::Construct(SDMXEntityEditor::FArguments());

	WeakDMXEditor = InDMXEditor;
	FixtureTypeSharedData = InDMXEditor->GetFixtureTypeSharedData();

	SetCanTick(false);
	bCanSupportFocus = false;

	FixtureTypeDetailsView = GenerateFixtureTypeDetailsView();

	ChildSlot
	.VAlign(VAlign_Fill)
	.HAlign(HAlign_Fill)
	[
		SNew(SSplitter)
		.Orientation(EOrientation::Orient_Horizontal)
		.ResizeMode(ESplitterResizeMode::FixedPosition)

		// 1st Collumn
		+ SSplitter::Slot()
		[
			SNew(SSplitter)
			.Orientation(EOrientation::Orient_Vertical)
			.ResizeMode(ESplitterResizeMode::FixedPosition)

			// Top Row
			+ SSplitter::Slot()
			[
				SAssignNew(FixtureTypeTree, SDMXFixtureTypeTree)
				.DMXEditor(InDMXEditor)
			]

			// Bottom Row
			+ SSplitter::Slot()
			[
				FixtureTypeDetailsView.ToSharedRef()
			]
		]
			
		// 2nd Collumn
		+ SSplitter::Slot()
		[
			SNew(SSplitter)
			.Orientation(EOrientation::Orient_Vertical)
			.ResizeMode(ESplitterResizeMode::FixedPosition)
				
			// Top Row
			+ SSplitter::Slot()			
			[
				SNew(SDMXFixtureTypeModesEditor, InDMXEditor)
			]

			// Bottom Row
			+ SSplitter::Slot()
			[
				SNew(SDMXFixtureModeEditor, InDMXEditor)
			]
		]

		// 3rd Collumn
		+ SSplitter::Slot()
		[
			SNew(SSplitter)
			.Orientation(EOrientation::Orient_Vertical)
			.ResizeMode(ESplitterResizeMode::FixedPosition)

			// Top Row
			+ SSplitter::Slot()
			[
				SNew(SDMXFixtureTypeFunctionsEditor, InDMXEditor)
			]

			// Bottom Row
			+ SSplitter::Slot()
			[
				SNew(SDMXFixtureFunctionEditor, InDMXEditor)
			]
		]
	];	

	// Adopt the selection
	OnFixtureTypesSelected();

	// Bind to selection changes
	FixtureTypeSharedData->OnFixtureTypesSelected.AddSP(this, &SDMXFixtureTypeEditor::OnFixtureTypesSelected);
}

void SDMXFixtureTypeEditor::RequestRenameOnNewEntity(const UDMXEntity* InEntity, ESelectInfo::Type SelectionType)
{
	check(FixtureTypeTree.IsValid());

	FixtureTypeTree->UpdateTree();
	FixtureTypeTree->SelectItemByEntity(InEntity, SelectionType);
}

void SDMXFixtureTypeEditor::SelectEntity(UDMXEntity* InEntity, ESelectInfo::Type InSelectionType /*= ESelectInfo::Type::Direct*/)
{
	check(FixtureTypeTree.IsValid());

	FixtureTypeTree->SelectItemByEntity(InEntity, InSelectionType);
}

void SDMXFixtureTypeEditor::SelectEntities(const TArray<UDMXEntity*>& InEntities, ESelectInfo::Type InSelectionType /*= ESelectInfo::Type::Direct*/)
{
	check(FixtureTypeTree.IsValid());

	FixtureTypeTree->SelectItemsByEntities(InEntities, InSelectionType);
}

TArray<UDMXEntity*> SDMXFixtureTypeEditor::GetSelectedEntities() const
{
	check(FixtureTypeTree.IsValid());

	return FixtureTypeTree->GetSelectedEntities();
}

void SDMXFixtureTypeEditor::OnFixtureTypesSelected()
{
	TArray<TWeakObjectPtr<UDMXEntityFixtureType>> SelectedFixtureTypes = FixtureTypeSharedData->GetSelectedFixtureTypes();
	TArray<UObject*> SelectedObjects;
	for (TWeakObjectPtr<UDMXEntityFixtureType> WeakSelectedFixtureType : SelectedFixtureTypes)
	{
		if (UDMXEntity* SelectedObject = WeakSelectedFixtureType.Get())
		{
			SelectedObjects.Add(SelectedObject);
		}
	}
	FixtureTypeDetailsView->SetObjects(SelectedObjects);
}

TSharedRef<IDetailsView> SDMXFixtureTypeEditor::GenerateFixtureTypeDetailsView() const
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = true;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;

	return PropertyEditorModule.CreateDetailView(DetailsViewArgs);
}
