// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SDMXEntityEditor.h"

#include "CoreMinimal.h"

class FDMXEditor;
class FDMXFixtureTypeSharedData;
class SDMXFixtureModeEditor;
class SDMXFixtureTypeTree;

struct FPropertyChangedEvent;
class IDetailsView;

/** The whole editor for Fixture Types */
class SDMXFixtureTypeEditor
	: public SDMXEntityEditor
{
public:
	SLATE_BEGIN_ARGS(SDMXFixtureTypeEditor)
	{}

	SLATE_END_ARGS()

public:
	/** Constructs this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<FDMXEditor>& InDMXEditor);

	/** Begin SDMXEntityEditorTab interface */
	void RequestRenameOnNewEntity(const UDMXEntity* InEntity, ESelectInfo::Type SelectionType);
	void SelectEntity(UDMXEntity* InEntity, ESelectInfo::Type InSelectionType = ESelectInfo::Type::Direct);
	void SelectEntities(const TArray<UDMXEntity*>& InEntities, ESelectInfo::Type SelectionType = ESelectInfo::Type::Direct);
	TArray<UDMXEntity*> GetSelectedEntities() const;
	/** ~End SDMXEntityEditorTab interface */

private:
	/** Called when fixture types were selected */
	void OnFixtureTypesSelected();

	/** Generates a details view for the Fixture Type Details */
	TSharedRef<IDetailsView> GenerateFixtureTypeDetailsView() const;

	/** Tree of Fixture Types */
	TSharedPtr<SDMXFixtureTypeTree> FixtureTypeTree;

	/** Details View for the selected Fixture Type */
	TSharedPtr<IDetailsView> FixtureTypeDetailsView;

	/** Shared Data for Fixture Types */
	TSharedPtr<FDMXFixtureTypeSharedData> FixtureTypeSharedData;

	/** Pointer back to the DMXEditor tool that owns us */
	TWeakPtr<FDMXEditor> WeakDMXEditor;
}; 
