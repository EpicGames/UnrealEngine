// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXFixtureTypeFunctionsEditor.h"

#include "DMXEditor.h"
#include "DMXEditorSettings.h"
#include "DMXFixtureTypeSharedData.h"
#include "Library/DMXEntityFixtureType.h"
#include "Widgets/FixtureType/DMXFixtureTypeFunctionsEditorFunctionItem.h"
#include "Widgets/FixtureType/DMXFixtureTypeFunctionsEditorMatrixItem.h"
#include "Widgets/FixtureType/SDMXFixtureTypeFunctionsEditorCategoryRow.h"
#include "Widgets/FixtureType/SDMXFixtureTypeFunctionsEditorFunctionRow.h"
#include "Widgets/FixtureType/SDMXFixtureTypeFunctionsEditorMatrixRow.h"

#include "Algo/Copy.h"
#include "Algo/Find.h"
#include "Styling/AppStyle.h"
#include "ScopedTransaction.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/SListView.h"


#define LOCTEXT_NAMESPACE "SDMXFixtureTypeFunctionsEditor"

const FName SDMXFixtureTypeFunctionsEditor::FCollumnID::Status = "Status";
const FName SDMXFixtureTypeFunctionsEditor::FCollumnID::Channel = "Channel";
const FName SDMXFixtureTypeFunctionsEditor::FCollumnID::Name = "Name";
const FName SDMXFixtureTypeFunctionsEditor::FCollumnID::Attribute = "Attribute";
const FName SDMXFixtureTypeFunctionsEditor::FCollumnID::DeleteAttribute = "DeleteAttribute";

SDMXFixtureTypeFunctionsEditor::~SDMXFixtureTypeFunctionsEditor()
{
	SaveHeaderRowSettings();
}

void SDMXFixtureTypeFunctionsEditor::Construct(const FArguments& InArgs, const TSharedRef<FDMXEditor>& InDMXEditor)
{
	WeakDMXEditor = InDMXEditor;
	FixtureTypeSharedData = InDMXEditor->GetFixtureTypeSharedData();

	if (ensureMsgf(FixtureTypeSharedData.IsValid(), TEXT("Trying to display modes while Fixture Type Shared Data is not valid")))
	{
		// Handle Shared Data selection changes
		FixtureTypeSharedData->OnFixtureTypesSelected.AddSP(this, &SDMXFixtureTypeFunctionsEditor::RebuildList);
		FixtureTypeSharedData->OnModesSelected.AddSP(this, &SDMXFixtureTypeFunctionsEditor::RebuildList);
		FixtureTypeSharedData->OnFunctionsSelected.AddSP(this, &SDMXFixtureTypeFunctionsEditor::OnSharedDataFunctionOrMatrixSelectionChanged);
		FixtureTypeSharedData->OnMatrixSelectionChanged.AddSP(this, &SDMXFixtureTypeFunctionsEditor::OnSharedDataFunctionOrMatrixSelectionChanged);
	}

	UDMXEntityFixtureType::GetOnFixtureTypeChanged().AddSP(this, &SDMXFixtureTypeFunctionsEditor::OnFixtureTypeChanged);

	static const FTableViewStyle TableViewStyle = FAppStyle::Get().GetWidgetStyle<FTableViewStyle>("TreeView");

	ChildSlot
	[
		SNew(SVerticalBox)
		
		// Header
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		.AutoHeight()
		[
			SNew(SDMXFixtureTypeFunctionsEditorCategoryRow, InDMXEditor)
			.OnSearchTextChanged(this, &SDMXFixtureTypeFunctionsEditor::OnSearchTextChanged)
		]

		// Functions 
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.FillHeight(1.f)
		[
			SAssignNew(ListContentBorder, SBorder)
			.Padding(2.0f)
			.BorderImage(&TableViewStyle.BackgroundBrush)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
		]
	];

	RegisterCommands();
	RebuildList();
}

 FReply SDMXFixtureTypeFunctionsEditor::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) 
{
	 if (CommandList->ProcessCommandBindings(InKeyEvent))
	 {
		 return FReply::Handled();
	 }

	 return FReply::Unhandled();
}

void SDMXFixtureTypeFunctionsEditor::RebuildList()
{
	SaveHeaderRowSettings();

	FunctionRows.Reset();
	ListContentBorder->ClearContent();

	if (TSharedPtr<FDMXEditor> DMXEditor = WeakDMXEditor.Pin())
	{
		if (ensureMsgf(FixtureTypeSharedData.IsValid(), TEXT("Trying to access Fixture Type Shared Data after it was destroyed.")))
		{
			const TArray<TWeakObjectPtr<UDMXEntityFixtureType>> SelectedFixtureTypes = FixtureTypeSharedData->GetSelectedFixtureTypes();
			const TArray<int32> SelectedModeIndices = FixtureTypeSharedData->GetSelectedModeIndices();

			// Only single Fixture Type editing is supported
			const UDMXEntityFixtureType* FixtureType = SelectedFixtureTypes.Num() == 1 ? SelectedFixtureTypes[0].Get() : nullptr;

			if (FixtureType && SelectedModeIndices.Num() == 1 && FixtureType->Modes.IsValidIndex(SelectedModeIndices[0]))
			{
				// Rebuild the List Source
				RebuildListSource();

				// Create the list
				ListContentBorder->SetContent
				(
					SAssignNew(ListView, SListView<TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>>)
					.HeaderRow(GenerateHeaderRow())
					.ListItemsSource(&ListSource)
					.OnGenerateRow(this, &SDMXFixtureTypeFunctionsEditor::OnGenerateRow)
					.OnSelectionChanged(this, &SDMXFixtureTypeFunctionsEditor::OnListSelectionChanged)
					.OnContextMenuOpening(this, &SDMXFixtureTypeFunctionsEditor::OnListContextMenuOpening)
					.ReturnFocusToSelection(false)
				);

				// Select what's selected in Shared Data
				const TArray<int32>& SelectedFunctionIndices = FixtureTypeSharedData->GetSelectedFunctionIndices();
				TArray<TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>> SelectedItems;
				for (const int32 SelectedFunctionIndex : SelectedFunctionIndices)
				{
					const TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>* SelectedItemPtr = Algo::FindByPredicate(ListSource, [SelectedFunctionIndex](const TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>& Item)
						{
							if (Item->GetType() == FDMXFixtureTypeFunctionsEditorItemBase::EItemType::Function)
							{
								TSharedPtr<FDMXFixtureTypeFunctionsEditorFunctionItem> FunctionItem = StaticCastSharedPtr<FDMXFixtureTypeFunctionsEditorFunctionItem>(Item);
								return FunctionItem->GetFunctionIndex() == SelectedFunctionIndex;
							}

							return false;
						});

					if (SelectedItemPtr)
					{
						SelectedItems.Add(*SelectedItemPtr);
					}
				}

				if (FixtureTypeSharedData->IsFixtureMatrixSelected())
				{
					const TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>* SelectedMatrixPtr = Algo::FindByPredicate(ListSource, [](const TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>& Item)
						{
							return Item->GetType() == FDMXFixtureTypeFunctionsEditorItemBase::EItemType::Matrix;
						});

					if (SelectedMatrixPtr)
					{
						SelectedItems.Add(*SelectedMatrixPtr);
					}
				}

				if (SelectedItems.Num() > 0)
				{
					ListView->SetItemSelection(SelectedItems, true, ESelectInfo::Direct);
					ListView->RequestScrollIntoView(SelectedItems[0]);
				}
				else if(ListSource.Num() > 0)
				{
					// Since there was no selection, make an initial selection, as if the user clicked it
					ListView->SetSelection(ListSource[0], ESelectInfo::OnMouseClick);
				}
			}
			else if (!FixtureType)
			{
				// Show a warning when no fixture type is selected
				ListContentBorder->SetContent
				(
					SNew(SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("NoFixtureTypeSelectedWarning", "No Fixture Type selected"))
					]
				);
			}
			else if (SelectedModeIndices.Num() != 1)
			{
				// Show a warning when multiple fixture types are selected
				ListContentBorder->SetContent
				(
					SNew(SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("MultiEditingModesNotSupportedWarning", "Multi-Editing Modes is not supported."))
					]
				);
			}
		}
	}
}

void SDMXFixtureTypeFunctionsEditor::RebuildListSource()
{
	ListSource.Reset();

	if (TSharedPtr<FDMXEditor> DMXEditor = WeakDMXEditor.Pin())
	{
		if (ensureMsgf(FixtureTypeSharedData.IsValid(), TEXT("Trying to access Fixture Type Shared Data after it was destroyed.")))
		{
			const TArray<TWeakObjectPtr<UDMXEntityFixtureType>> SelectedFixtureTypes = FixtureTypeSharedData->GetSelectedFixtureTypes();
			const TArray<int32> SelectedModeIndices = FixtureTypeSharedData->GetSelectedModeIndices();

			// Only single Fixture Type and single Mode editing is supported
			UDMXEntityFixtureType* FixtureType = SelectedFixtureTypes.Num() == 1 ? SelectedFixtureTypes[0].Get() : nullptr;
			if (FixtureType && SelectedModeIndices.Num() == 1 && FixtureType->Modes.IsValidIndex(SelectedModeIndices[0]))
			{
				// Rebuild the list source
				const int32 ModeIndex = SelectedModeIndices[0];
				const FDMXFixtureMode& Mode = FixtureType->Modes[ModeIndex];
				const FDMXFixtureMatrix& Matrix = Mode.FixtureMatrixConfig;
				TSharedPtr<FDMXFixtureTypeFunctionsEditorMatrixItem> MatrixItem;
				
				const FString SearchString = SearchText.ToString();

				for (int32 FunctionIndex = 0; FunctionIndex < Mode.Functions.Num(); FunctionIndex++)
				{					
					// If the Matrix is enabled, add it once at its Channel 
					if (Mode.bFixtureMatrixEnabled && !MatrixItem.IsValid() && Matrix.FirstCellChannel <= Mode.Functions[FunctionIndex].Channel)
					{
						if (SearchString.IsEmpty() || SearchString.StartsWith(TEXT("m"), ESearchCase::IgnoreCase))
						{
							MatrixItem = MakeShared<FDMXFixtureTypeFunctionsEditorMatrixItem>(DMXEditor.ToSharedRef(), FixtureType, ModeIndex);
							ListSource.Add(MatrixItem);
						}
					}

					// Add the Function
					const FDMXFixtureFunction& Function = Mode.Functions[FunctionIndex];
					if (SearchString.IsEmpty() || Function.FunctionName.Contains(SearchString, ESearchCase::IgnoreCase))
					{
						TSharedPtr<FDMXFixtureTypeFunctionsEditorFunctionItem> Item = MakeShared<FDMXFixtureTypeFunctionsEditorFunctionItem>(DMXEditor.ToSharedRef(), FixtureType, ModeIndex, FunctionIndex);
						ListSource.Add(Item);
					}
				}

				// If the matrix was not added, it resides after all the functions, so add it here
				if (Mode.bFixtureMatrixEnabled && !MatrixItem.IsValid())
				{
					if (SearchString.IsEmpty() || SearchString.StartsWith(TEXT("m"), ESearchCase::IgnoreCase))
					{
						MatrixItem = MakeShared<FDMXFixtureTypeFunctionsEditorMatrixItem>(DMXEditor.ToSharedRef(), FixtureType, ModeIndex);
						ListSource.Add(MatrixItem);
					}
				}
			}
		}
	}

	UpdateItemsStatus();
}

void SDMXFixtureTypeFunctionsEditor::UpdateItemsStatus()
{
	// Test conflicting channels
	TMap<int32, TSet<TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>>> OccupiedChannelsToItemsMap;
	for (const TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>& Item : ListSource)
	{
		TSet<TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>> ConflictingItems;

		if (Item->GetType() == FDMXFixtureTypeFunctionsEditorItemBase::EItemType::Matrix)
		{
			TSharedPtr<FDMXFixtureTypeFunctionsEditorMatrixItem> MatrixItem = StaticCastSharedPtr<FDMXFixtureTypeFunctionsEditorMatrixItem>(Item);
			
			MatrixItem->WarningStatus = FText::GetEmpty();

			const int32 StartingChannel = MatrixItem->GetStartingChannel();
			for (int32 ChannelIndex = 0; ChannelIndex < MatrixItem->GetNumChannels(); ChannelIndex++)
			{
				OccupiedChannelsToItemsMap.FindOrAdd(StartingChannel + ChannelIndex).Add(MatrixItem);
			}
		}
		else
		{
			TSharedPtr<FDMXFixtureTypeFunctionsEditorFunctionItem> FunctionItem = StaticCastSharedPtr<FDMXFixtureTypeFunctionsEditorFunctionItem>(Item);

			const int32 StartingChannel = FunctionItem->GetStartingChannel();
			for (int32 ChannelIndex = 0; ChannelIndex < FunctionItem->GetNumChannels(); ChannelIndex++)
			{
				OccupiedChannelsToItemsMap.FindOrAdd(StartingChannel + ChannelIndex).Add(FunctionItem);
			}
		}
	}
	for (const TTuple<int32, TSet<TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>>>& OccupiedChannelsToItemsPair : OccupiedChannelsToItemsMap)
	{
		if (OccupiedChannelsToItemsPair.Value.Num() > 1)
		{
			// Create the warning status
			FText NewWarningStatus = LOCTEXT("ConflictingChannelsWarning", "Conflicting Channels in:");

			for (const TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>& ConflictingItem : OccupiedChannelsToItemsPair.Value)
			{
				NewWarningStatus = FText::Format(LOCTEXT("ConflictingChannelsWarningAppend", "{0} '{1}'"), NewWarningStatus, ConflictingItem->GetDisplayName());
			}

			// Set the same status for all conflicting 
			for (const TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>& ConflictingItem : OccupiedChannelsToItemsPair.Value)
			{
				ConflictingItem->WarningStatus = NewWarningStatus;
			}
		}
	}

	// Test if the same attributes are used more than once
	TMap<FDMXAttributeName, TSet<TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>>> OccupiedAttributeToItemsMap;
	for (const TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>& Item : ListSource)
	{
		if (Item->GetType() == FDMXFixtureTypeFunctionsEditorItemBase::EItemType::Function)
		{
			TSharedPtr<FDMXFixtureTypeFunctionsEditorFunctionItem> FunctionItem = StaticCastSharedPtr<FDMXFixtureTypeFunctionsEditorFunctionItem>(Item);
			const FDMXAttributeName& Attribute = FunctionItem->GetAttributeName();

			OccupiedAttributeToItemsMap.FindOrAdd(Attribute).Add(FunctionItem);
		}
	}
	for (const TTuple<FDMXAttributeName, TSet<TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>>>& OccupiedAttribueToItemsPair : OccupiedAttributeToItemsMap)
	{
		if (OccupiedAttribueToItemsPair.Value.Num() > 1 && OccupiedAttribueToItemsPair.Key.Name != NAME_None)
		{
			// Create the warning status
			FText NewWarningStatus = FText::Format(LOCTEXT("ConflictingAttributesWarning", "Ambiguous Attribute '{0}' used in:"), FText::FromName(OccupiedAttribueToItemsPair.Key.Name));

			for (const TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>& ConflictingItem : OccupiedAttribueToItemsPair.Value)
			{
				NewWarningStatus = FText::Format(LOCTEXT("ConflictingAttributesWarningAppend", "{0} '{1}'"), NewWarningStatus, ConflictingItem->GetDisplayName());
			}

			// Set the same status for all conflicting items
			for (const TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>& ConflictingItem : OccupiedAttribueToItemsPair.Value)
			{
				ConflictingItem->WarningStatus = NewWarningStatus;
			}
		}
	}
}

void SDMXFixtureTypeFunctionsEditor::OnFixtureTypeChanged(const UDMXEntityFixtureType* ChangedFixtureType)
{
	if (ChangedFixtureType)
	{
		const bool bDisplayedFixtureTypeChanged = FixtureTypeSharedData->GetSelectedFixtureTypes().ContainsByPredicate([ChangedFixtureType](TWeakObjectPtr<UDMXEntityFixtureType> FixtureType)
			{
				return FixtureType.Get() == ChangedFixtureType;
			});

		if (bDisplayedFixtureTypeChanged)
		{
			RebuildList();
		}
	}
}

TSharedRef<SHeaderRow> SDMXFixtureTypeFunctionsEditor::GenerateHeaderRow()
{
	const float StatusColumnWidth = FMath::Max(FAppStyle::Get().GetBrush("Icons.Warning")->GetImageSize().X + 6.f, FAppStyle::Get().GetBrush("Icons.Error")->GetImageSize().X + 6.f);
	const float DeleteAttributeColumnWidth = FAppStyle::GetBrush("Icons.Delete")->GetImageSize().X + 8.f;

	HeaderRow = SNew(SHeaderRow);
	SHeaderRow::FColumn::FArguments ColumnArgs;

	HeaderRow->AddColumn(
		SHeaderRow::FColumn::FArguments()
		.ColumnId(SDMXFixtureTypeFunctionsEditor::FCollumnID::Status)
		.DefaultLabel(FText())
		.FixedWidth(StatusColumnWidth)
	);

	HeaderRow->AddColumn(
		SHeaderRow::FColumn::FArguments()
		.ColumnId(SDMXFixtureTypeFunctionsEditor::FCollumnID::Channel)
		.DefaultLabel(LOCTEXT("ChannelColumnLabel", "Ch."))
		.FixedWidth(56.f)
	);

	HeaderRow->AddColumn(
		SHeaderRow::FColumn::FArguments()
		.ColumnId(SDMXFixtureTypeFunctionsEditor::FCollumnID::Name)
		.DefaultLabel(LOCTEXT("NameColumnLabel", "Name"))
		.FillWidth(0.5f)
	);

	HeaderRow->AddColumn(
		SHeaderRow::FColumn::FArguments()
		.ColumnId(SDMXFixtureTypeFunctionsEditor::FCollumnID::Attribute)
		.DefaultLabel(LOCTEXT("AttributeColumnLabel", "Attribute"))
		.FillWidth(0.5f)
	);

	HeaderRow->AddColumn(
		SHeaderRow::FColumn::FArguments()
		.ColumnId(SDMXFixtureTypeFunctionsEditor::FCollumnID::DeleteAttribute)
		.DefaultLabel(FText())
		.FixedWidth(DeleteAttributeColumnWidth)
	);

	// Restore user settings
	if (const UDMXEditorSettings* EditorSettings = GetDefault<UDMXEditorSettings>())
	{
		const float NameColumnWidth = EditorSettings->FixtureTypeFunctionsEditorSettings.NameColumnWidth;
		if (NameColumnWidth > 10.f)
		{
			HeaderRow->SetColumnWidth("Name", NameColumnWidth);
		}

		const float AttributeColumnWidth = EditorSettings->FixtureTypeFunctionsEditorSettings.AttributeColumnWidth;
		if (AttributeColumnWidth > 10.f)
		{
			HeaderRow->SetColumnWidth(SDMXFixtureTypeFunctionsEditor::FCollumnID::Attribute, AttributeColumnWidth);
		}
	}

	return HeaderRow.ToSharedRef();
}

void SDMXFixtureTypeFunctionsEditor::SaveHeaderRowSettings()
{
	UDMXEditorSettings* EditorSettings = GetMutableDefault<UDMXEditorSettings>();
	if (HeaderRow.IsValid() && EditorSettings)
	{
		for (const SHeaderRow::FColumn& Column : HeaderRow->GetColumns())
		{
			if (Column.ColumnId == SDMXFixtureTypeFunctionsEditor::FCollumnID::Name)
			{
				EditorSettings->FixtureTypeFunctionsEditorSettings.NameColumnWidth = Column.Width.Get();

			}
			else if (Column.ColumnId == SDMXFixtureTypeFunctionsEditor::FCollumnID::Attribute)
			{
				EditorSettings->FixtureTypeFunctionsEditorSettings.AttributeColumnWidth = Column.Width.Get();
			}
		}

		EditorSettings->SaveConfig();
	}
}

TSharedRef<ITableRow> SDMXFixtureTypeFunctionsEditor::OnGenerateRow(TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	if (InItem->GetType() == FDMXFixtureTypeFunctionsEditorItemBase::EItemType::Matrix)
	{
		TSharedPtr<FDMXFixtureTypeFunctionsEditorMatrixItem> MatrixItem = StaticCastSharedPtr<FDMXFixtureTypeFunctionsEditorMatrixItem>(InItem);
		TSharedRef<SDMXFixtureTypeFunctionsEditorMatrixRow> MatrixRow = SNew(SDMXFixtureTypeFunctionsEditorMatrixRow, OwnerTable, MatrixItem.ToSharedRef());

		return MatrixRow;
	}
	else
	{
		TSharedPtr<FDMXFixtureTypeFunctionsEditorFunctionItem> FunctionItem = StaticCastSharedPtr<FDMXFixtureTypeFunctionsEditorFunctionItem>(InItem);
		TSharedRef<SDMXFixtureTypeFunctionsEditorFunctionRow> FunctionRow = 
			SNew(SDMXFixtureTypeFunctionsEditorFunctionRow, OwnerTable, FunctionItem.ToSharedRef())
			.IsSelected(this, &SDMXFixtureTypeFunctionsEditor::IsItemSelectedExclusively, InItem);

		FunctionRows.Add(FunctionRow);

		return FunctionRow;
	}
}

bool SDMXFixtureTypeFunctionsEditor::IsItemSelectedExclusively(TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase> InItem) const
{
	return
		ListView->GetNumItemsSelected() == 1 && 
		ListView->IsItemSelected(InItem);
}

void SDMXFixtureTypeFunctionsEditor::OnListSelectionChanged(TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase> NewlySelectedItem, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo != ESelectInfo::Direct)
	{
		const TArray<int32> SelectedFunctionIndicies = GetSelectionAsFunctionIndices();

		if (SelectedFunctionIndicies.IsEmpty())
		{
			// Never clear selection
			TArray<TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>> ItemsToSelect;
			Algo::CopyIf(ListSource, ItemsToSelect, [&SelectedFunctionIndicies](const TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>& Item)
				{
					if (Item->GetType() == FDMXFixtureTypeFunctionsEditorItemBase::EItemType::Function)
					{
						const TSharedPtr<FDMXFixtureTypeFunctionsEditorFunctionItem> FunctionItem = StaticCastSharedPtr<FDMXFixtureTypeFunctionsEditorFunctionItem>(Item);
						return SelectedFunctionIndicies.Contains(FunctionItem->GetFunctionIndex());
					}

					return false;
				});

			if (ItemsToSelect.Num() > 0)
			{
				ListView->SetItemSelection(ItemsToSelect, true);
			}
		}
		else
		{
			const bool bMatrixSelected = IsMatrixSelected();
			FixtureTypeSharedData->SetFunctionAndMatrixSelection(SelectedFunctionIndicies, bMatrixSelected);
		}
	}
	else
	{
		ListView->RequestScrollIntoView(NewlySelectedItem);
	}
}

void SDMXFixtureTypeFunctionsEditor::OnSharedDataFunctionOrMatrixSelectionChanged()
{
	const TArray<int32> SelectedFunctionIndices = FixtureTypeSharedData->GetSelectedFunctionIndices();
	TArray<TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>> NewSelection;
	for (int32 SelectedFunctionIndex : SelectedFunctionIndices)
	{
		const TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>* SelectedItemPtr = Algo::FindByPredicate(ListSource, [SelectedFunctionIndex](const TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>& Item)
			{
				if (Item->GetType() == FDMXFixtureTypeFunctionsEditorItemBase::EItemType::Function)
				{
					const TSharedPtr<FDMXFixtureTypeFunctionsEditorFunctionItem> FunctionItem = StaticCastSharedPtr<FDMXFixtureTypeFunctionsEditorFunctionItem>(Item);
					return FunctionItem->GetFunctionIndex() == SelectedFunctionIndex;
				}
				return false;
			});

		if (SelectedItemPtr)
		{
			NewSelection.Add(*SelectedItemPtr);
		}
	}

	if (FixtureTypeSharedData->IsFixtureMatrixSelected())
	{
		const TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>* SelectedItemPtr = Algo::FindByPredicate(ListSource, [](const TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>& Item)
			{
				return Item->GetType() == FDMXFixtureTypeFunctionsEditorItemBase::EItemType::Matrix;
			});

		if (SelectedItemPtr)
		{
			NewSelection.Add(*SelectedItemPtr);
		}
	}

	ListView->ClearSelection();
	ListView->SetItemSelection(NewSelection, true, ESelectInfo::Direct);
}

void SDMXFixtureTypeFunctionsEditor::OnSearchTextChanged(const FText& InSearchText)
{
	SearchText = InSearchText;

	RebuildList();
}

TArray<int32> SDMXFixtureTypeFunctionsEditor::GetSelectionAsFunctionIndices() const
{
	TArray<TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>> SelectedItems;
	ListView->GetSelectedItems(SelectedItems);

	TArray<int32> SelectedFunctionIndices;
	for (const TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>& Item : SelectedItems)
	{
		if (Item->GetType() == FDMXFixtureTypeFunctionsEditorItemBase::EItemType::Function)
		{
			TSharedPtr<FDMXFixtureTypeFunctionsEditorFunctionItem> FunctionItem = StaticCastSharedPtr<FDMXFixtureTypeFunctionsEditorFunctionItem>(Item);
			if (FunctionItem->GetFunctionIndex() != INDEX_NONE)
			{
				SelectedFunctionIndices.Add(FunctionItem->GetFunctionIndex());
			}
		}
	}

	return SelectedFunctionIndices;
}

bool SDMXFixtureTypeFunctionsEditor::IsMatrixSelected() const
{
	TArray<TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>> SelectedItems;
	ListView->GetSelectedItems(SelectedItems);

	const bool bSelectedItemsContainMatrix = SelectedItems.ContainsByPredicate([](const TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>& Item)
	{
		return Item->GetType() == FDMXFixtureTypeFunctionsEditorItemBase::EItemType::Matrix;
	});

	return bSelectedItemsContainMatrix;
}

TSharedPtr<SWidget> SDMXFixtureTypeFunctionsEditor::OnListContextMenuOpening()
{
	const bool bCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bCloseWindowAfterMenuSelection, CommandList);

	MenuBuilder.BeginSection("BasicOperations");
	{
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SDMXFixtureTypeFunctionsEditor::RegisterCommands()
{
	// Listen to common editor shortcuts for copy/paste etc
	if (!CommandList.IsValid())
	{
		CommandList = MakeShared<FUICommandList>();

		CommandList->MapAction(FGenericCommands::Get().Cut,
			FUIAction(FExecuteAction::CreateSP(this, &SDMXFixtureTypeFunctionsEditor::OnCutSelectedItems),
				FCanExecuteAction::CreateSP(this, &SDMXFixtureTypeFunctionsEditor::CanCutItems))
			);

		CommandList->MapAction(FGenericCommands::Get().Copy,
			FUIAction(FExecuteAction::CreateSP(this, &SDMXFixtureTypeFunctionsEditor::OnCopySelectedItems),
				FCanExecuteAction::CreateSP(this, &SDMXFixtureTypeFunctionsEditor::CanCopyItems))
			);

		CommandList->MapAction(FGenericCommands::Get().Paste,
			FUIAction(FExecuteAction::CreateSP(this, &SDMXFixtureTypeFunctionsEditor::OnPasteItems),
				FCanExecuteAction::CreateSP(this, &SDMXFixtureTypeFunctionsEditor::CanPasteItems))
			);

		CommandList->MapAction(FGenericCommands::Get().Duplicate,
			FUIAction(FExecuteAction::CreateSP(this, &SDMXFixtureTypeFunctionsEditor::OnDuplicateItems),
				FCanExecuteAction::CreateSP(this, &SDMXFixtureTypeFunctionsEditor::CanDuplicateItems))
			);

		CommandList->MapAction(FGenericCommands::Get().Delete,
			FUIAction(FExecuteAction::CreateSP(this, &SDMXFixtureTypeFunctionsEditor::OnDeleteItems),
				FCanExecuteAction::CreateSP(this, &SDMXFixtureTypeFunctionsEditor::CanDeleteItems))
			);

		CommandList->MapAction(FGenericCommands::Get().Rename,
			FUIAction(FExecuteAction::CreateSP(this, &SDMXFixtureTypeFunctionsEditor::OnRenameItem),
				FCanExecuteAction::CreateSP(this, &SDMXFixtureTypeFunctionsEditor::CanRenameItem))
			);
	}
}

bool SDMXFixtureTypeFunctionsEditor::CanCutItems() const
{
	TArray<TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>> SelectedItems;
	ListView->GetSelectedItems(SelectedItems);
	if (SelectedItems.Num() == 1 && SelectedItems[0]->GetType() == FDMXFixtureTypeFunctionsEditorItemBase::EItemType::Matrix)
	{
		return false;
	}

	const int32 NumSelectedItems = ListView->GetNumItemsSelected();
	return NumSelectedItems == 1;
}

void SDMXFixtureTypeFunctionsEditor::OnCutSelectedItems()
{
	const int32 NumSelectedItems = ListView->GetNumItemsSelected();
	const FScopedTransaction Transaction(LOCTEXT("CutFunctionTransaction", "Cut Function"));

	OnCopySelectedItems();
	OnDeleteItems();
}

bool SDMXFixtureTypeFunctionsEditor::CanCopyItems() const
{
	TArray<TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>> SelectedItems;
	ListView->GetSelectedItems(SelectedItems);
	if (SelectedItems.Num() == 1 && SelectedItems[0]->GetType() == FDMXFixtureTypeFunctionsEditorItemBase::EItemType::Matrix)
	{
		return false;
	}

	return FixtureTypeSharedData->CanCopyFunctionsToClipboard();
}

void SDMXFixtureTypeFunctionsEditor::OnCopySelectedItems()
{
	FixtureTypeSharedData->CopyFunctionsToClipboard();
}

bool SDMXFixtureTypeFunctionsEditor::CanPasteItems() const
{
	return FixtureTypeSharedData->CanPasteFunctionsFromClipboard();
}

void SDMXFixtureTypeFunctionsEditor::OnPasteItems()
{
	TArray<int32> NewlyAddedFunctionIndices;
	FixtureTypeSharedData->PasteFunctionsFromClipboard(NewlyAddedFunctionIndices);

	// Select the newly added Functions
	constexpr bool bSelectMatrix = false;
	FixtureTypeSharedData->SetFunctionAndMatrixSelection(NewlyAddedFunctionIndices, bSelectMatrix);
}

bool SDMXFixtureTypeFunctionsEditor::CanDuplicateItems() const
{
	const int32 NumSelectedItems = ListView->GetNumItemsSelected();
	return NumSelectedItems > 0;
}

void SDMXFixtureTypeFunctionsEditor::OnDuplicateItems()
{
	const TArray<TWeakObjectPtr<UDMXEntityFixtureType>>& SelectedFixtureTypes = FixtureTypeSharedData->GetSelectedFixtureTypes();
	if (ensureMsgf(SelectedFixtureTypes.Num() == 1, TEXT("Trying to Duplicate Functions, but many or no Fixture Type is selected. This should not be possible.")))
	{
		if (UDMXEntityFixtureType* FixtureType = SelectedFixtureTypes[0].Get())
		{
			const TArray<int32>& SelectedModeIndices = FixtureTypeSharedData->GetSelectedModeIndices();
			if (ensureMsgf(SelectedModeIndices.Num() == 1, TEXT("Trying to Duplicate Functions, but many or no Mode is selected. This should not be possible.")))
			{
				const FText TransactionText = FText::Format(LOCTEXT("DuplicateFunctionTransaction", "Duplicate Fixture {0}|plural(one=Function, other=Functions)"), GetSelectionAsFunctionIndices().Num());
				const FScopedTransaction DuplicateFunctionTransaction(TransactionText);
				FixtureType->PreEditChange(FDMXFixtureMode::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FDMXFixtureMode, Functions)));

				TArray<int32> NewlyAddedFunctionIndices;
				FixtureType->DuplicateFunctions(SelectedModeIndices[0], GetSelectionAsFunctionIndices(), NewlyAddedFunctionIndices);

				FixtureType->PostEditChange();

				// Select the newly added Functions
				constexpr bool bSelectMatrix = false;
				FixtureTypeSharedData->SetFunctionAndMatrixSelection(NewlyAddedFunctionIndices, bSelectMatrix);
			}
		}
	}
}

bool SDMXFixtureTypeFunctionsEditor::CanDeleteItems() const
{
	const int32 NumSelectedItems = ListView->GetNumItemsSelected();
	return NumSelectedItems > 0;
}

void SDMXFixtureTypeFunctionsEditor::OnDeleteItems()
{
	const TArray<TWeakObjectPtr<UDMXEntityFixtureType>>& SelectedFixtureTypes = FixtureTypeSharedData->GetSelectedFixtureTypes();
	if (ensureMsgf(SelectedFixtureTypes.Num() == 1, TEXT("Trying to Delete Functions, but many or no Fixture Type is selected. This should not be possible.")))
	{
		if (UDMXEntityFixtureType* FixtureType = SelectedFixtureTypes[0].Get())
		{
			const TArray<int32>& SelectedModeIndices = FixtureTypeSharedData->GetSelectedModeIndices();
			if (ensureMsgf(SelectedModeIndices.Num() == 1 && FixtureType->Modes.IsValidIndex(SelectedModeIndices[0]), TEXT("Trying to Delete Functions, but many or no Mode is selected. This should not be possible.")))
			{
				FDMXFixtureMode& Mode = FixtureType->Modes[SelectedModeIndices[0]];

				const TArray<int32>& SelectedFunctionIndices = GetSelectionAsFunctionIndices();
				const bool bMatrixSelected = IsMatrixSelected();

				if (SelectedFunctionIndices.Num() > 0 || bMatrixSelected)
				{
					// Create a nice transaction text
					const FText TransactionText = [SelectedFunctionIndices, bMatrixSelected]()
					{
						if (SelectedFunctionIndices.Num() > 0 && bMatrixSelected)
						{
							return FText::Format(LOCTEXT("DeleteFunctionAndMatrixTransaction", "Delete Fixture {0}|plural(one=Function, other=Functions) and Matrix"), SelectedFunctionIndices.Num());
						}
						else if (SelectedFunctionIndices.Num() > 0)
						{
							return FText::Format(LOCTEXT("DeleteFunctionTransaction", "Delete Fixture {0}|plural(one=Function, other=Functions)"), SelectedFunctionIndices.Num());
						}
						else if (bMatrixSelected)
						{
							return LOCTEXT("DeleteMatrixTransaction", "Delete Fixture Matrix");
						}
						return FText::GetEmpty();
					}();

					// Delete Functions and/or Matrix transacted
					if (!TransactionText.IsEmpty())
					{
						const FScopedTransaction DeleteFunctionTransaction(TransactionText);
						FixtureType->PreEditChange(FDMXFixtureMode::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FDMXFixtureMode, Functions)));

						if (SelectedFunctionIndices.Num() > 0)
						{
							FixtureType->RemoveFunctions(SelectedModeIndices[0], GetSelectionAsFunctionIndices());
						}

						if (bMatrixSelected)
						{
							Mode.bFixtureMatrixEnabled = false;
						}

						FixtureType->PostEditChange();
					}
				}
			}
		}
	}
}

bool SDMXFixtureTypeFunctionsEditor::CanRenameItem() const
{
	TArray<TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>> SelectedItems;
	ListView->GetSelectedItems(SelectedItems);
	if (SelectedItems.Num() != 1 || SelectedItems[0]->GetType() == FDMXFixtureTypeFunctionsEditorItemBase::EItemType::Matrix)
	{
		return false;
	}

	return true;
}

void SDMXFixtureTypeFunctionsEditor::OnRenameItem()
{
	TArray<TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>> SelectedFunctionItems;
	ListView->GetSelectedItems(SelectedFunctionItems);
	
	if (SelectedFunctionItems.Num() == 1)
	{
		const TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>& SelectedItem = SelectedFunctionItems[0];
		const TSharedPtr<SDMXFixtureTypeFunctionsEditorFunctionRow>* SelectedRowPtr = Algo::FindByPredicate(FunctionRows, [SelectedItem](const TSharedPtr<SDMXFixtureTypeFunctionsEditorFunctionRow>& FunctionRow)
			{
				return FunctionRow->GetFunctionItem() == SelectedItem;
			});

		if (SelectedRowPtr)
		{
			(*SelectedRowPtr)->EnterFunctionNameEditingMode();
		}
	}
}

#undef LOCTEXT_NAMESPACE
