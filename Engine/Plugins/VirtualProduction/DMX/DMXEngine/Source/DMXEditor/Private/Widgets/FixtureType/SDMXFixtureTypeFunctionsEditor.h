// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"

class FDMXEditor;
class FDMXFixtureTypeSharedData;
class FDMXFixtureTypeFunctionsEditorItemBase;
class SDMXFixtureTypeFunctionsEditorFunctionRow;
class SDMXFixtureTypeFunctionsEditorMatrixRow;
class UDMXEntityFixtureType;
class UDMXLibrary;

class FUICommandList;
class ITableRow;
class SBorder;
class SHeaderRow;
template <typename ItemType> class SListView;
class STableViewBase;
class SVerticalBox;


using FDMXFixtureTypeFunctionsEditorFunctionListType = SListView<TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>>;

/** Editor for the Functions Array of the Modes in a Fixture Type */
class SDMXFixtureTypeFunctionsEditor
	: public SCompoundWidget
{
public:
	/** Collumn IDs in the Functions Editor */
	struct FCollumnID
	{
		static const FName Status;
		static const FName Channel;
		static const FName Name;
		static const FName Attribute;
		static const FName DeleteAttribute;
	};

	SLATE_BEGIN_ARGS(SDMXFixtureTypeFunctionsEditor)
	{}
		
	SLATE_END_ARGS()

	/** Destructor */
	virtual ~SDMXFixtureTypeFunctionsEditor();

	/** Constructs this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<FDMXEditor>& InDMXEditor);

protected:
	//~ Begin SWidget Interface
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget Interface

private:
	/** Rebuilds the Functions List */
	void RebuildList();

	/** Rebuilds the List Source */
	void RebuildListSource();

	/** Updates the Status of each Item */
	void UpdateItemsStatus();

	/** Called when a fixture type changed */
	void OnFixtureTypeChanged(const UDMXEntityFixtureType* ChangedFixtureType);

	/** Generates a heaer row for the Function List */
	TSharedRef<SHeaderRow> GenerateHeaderRow();

	/** Saves header row settings in project settings */
	void SaveHeaderRowSettings();

	/** Generates a row in the List of Functions */
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Returns true if the item is selected */
	bool IsItemSelectedExclusively(TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase> InItem) const;

	/** Called when the selelction in the list changed */
	void OnListSelectionChanged(TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase> NewlySelectedItem, ESelectInfo::Type SelectInfo);

	/** Called when the selected Function or Matrix in Fixture Type Shared Data Changed */
	void OnSharedDataFunctionOrMatrixSelectionChanged();

	/** Called when the Search Text changed */
	void OnSearchTextChanged(const FText& SearchText);

	/** Returns the selected Function Indices in the Mode's Function array */
	TArray<int32> GetSelectionAsFunctionIndices() const;

	/** Returns true if the Matrix is selected */
	bool IsMatrixSelected() const;

	/** Called when the context menu of the list is opening */
	TSharedPtr<SWidget> OnListContextMenuOpening();

	/** Border that holds the list content, or widgets showing reasons why no list can be drawn */
	TSharedPtr<SBorder> ListContentBorder;

	/** The the list of Functions */
	TSharedPtr<FDMXFixtureTypeFunctionsEditorFunctionListType> ListView;

	/** Source array for the Functions List */
	TArray<TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>> ListSource;

	/** The Header Row of the List */
	TSharedPtr<SHeaderRow> HeaderRow;

	/** Rows of Mode widgets in the List */
	TArray<TSharedPtr<SDMXFixtureTypeFunctionsEditorFunctionRow>> FunctionRows;

	/** Search Text entered in the Category Row */
	FText SearchText;

	/** Shared data for Fixture Types */
	TSharedPtr<FDMXFixtureTypeSharedData> FixtureTypeSharedData;

	/** The Fixture Type Editor that owns this widget */
	TWeakPtr<FDMXEditor> WeakDMXEditor;
	

private:
	///////////////////////////////////////////////////
	// Context menu Commands related

	/** Registers the commands */
	void RegisterCommands();

	bool CanCutItems() const;
	void OnCutSelectedItems();
	bool CanCopyItems() const;
	void OnCopySelectedItems();
	bool CanPasteItems() const;
	void OnPasteItems();
	bool CanDuplicateItems() const;
	void OnDuplicateItems();
	bool CanDeleteItems() const;
	void OnDeleteItems();
	bool CanRenameItem() const;
	void OnRenameItem();

	TSharedPtr<FUICommandList> CommandList;
};
