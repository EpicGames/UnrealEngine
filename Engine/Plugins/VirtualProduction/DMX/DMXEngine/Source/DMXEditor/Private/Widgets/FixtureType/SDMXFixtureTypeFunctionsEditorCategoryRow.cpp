// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXFixtureTypeFunctionsEditorCategoryRow.h"

#include "DMXEditor.h"
#include "DMXEditorStyle.h"
#include "DMXFixtureTypeSharedData.h"
#include "Commands/DMXEditorCommands.h"
#include "Library/DMXEntityFixtureType.h"

#include "Styling/AppStyle.h"
#include "ScopedTransaction.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "SDMXFixtureTypeFunctionsEditorCategoryRow"

void SDMXFixtureTypeFunctionsEditorCategoryRow::Construct(const FArguments& InArgs, const TSharedRef<FDMXEditor>& InDMXEditor)
{
	WeakDMXEditor = InDMXEditor;
	SharedData = InDMXEditor->GetFixtureTypeSharedData();

	ChildSlot
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Top)
	[
		SNew(SBorder)
		.Padding(3.0f)
		.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
		.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f))
		[
			SNew(SHorizontalBox)
				
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			[
				// Search box
				SNew(SSearchBox)
				.HintText(LOCTEXT("SearchModesHint", "Search"))
				.OnTextChanged(InArgs._OnSearchTextChanged)
			]
					
			// 'Add Function' Button
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			.AutoWidth()
			.Padding(10.f, 2.f, 2.f, 2.f)
			[
				SNew(SButton)
				.IsEnabled(this, &SDMXFixtureTypeFunctionsEditorCategoryRow::GetIsAddFunctionButtonEnabled)
				.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
				.ForegroundColor(FLinearColor::White)
				.ToolTipText(this, &SDMXFixtureTypeFunctionsEditorCategoryRow::GetAddFunctionButtonTooltipText)
				.ContentPadding(FMargin(5.0f, 1.0f))
				.OnClicked(this, &SDMXFixtureTypeFunctionsEditorCategoryRow::OnAddFunctionButtonClicked)
				.Content()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(0.f, 1.f))
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Plus"))
					]
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(FMargin(2.f, 0.f, 2.f, 0.f))
					[
						SNew(STextBlock)
						.Text(this, &SDMXFixtureTypeFunctionsEditorCategoryRow::GetAddFunctionButtonText)
					]
				]
			]
		]
	];
}

bool SDMXFixtureTypeFunctionsEditorCategoryRow::GetIsAddFunctionButtonEnabled() const
{
	const TArray<TWeakObjectPtr<UDMXEntityFixtureType>>& SelectedFixtureTypes = SharedData->GetSelectedFixtureTypes();
	const TArray<int32>& SelectedModeIndices = SharedData->GetSelectedModeIndices();

	return SelectedFixtureTypes.Num() == 1 && SelectedModeIndices.Num() == 1;
}

FText SDMXFixtureTypeFunctionsEditorCategoryRow::GetAddFunctionButtonTooltipText() const
{
	if (GetIsAddFunctionButtonEnabled())
	{
		return FDMXEditorCommands::Get().AddNewFixtureTypeMode->GetDescription();
	}

	return LOCTEXT("AddFunctionButtonDisabledTooltip", "Please select a single Mode to add a Function to it.");
}

FReply SDMXFixtureTypeFunctionsEditorCategoryRow::OnAddFunctionButtonClicked() const
{
	const TArray<TWeakObjectPtr<UDMXEntityFixtureType>>& SelectedFixtureTypes = SharedData->GetSelectedFixtureTypes();
	if (ensureMsgf(SelectedFixtureTypes.Num() == 1, TEXT("Trying to Add Function, but many or no Fixture Type is selected. This should not be possible.")))
	{
		if (UDMXEntityFixtureType* FixtureType = SelectedFixtureTypes[0].Get())
		{
			const TArray<int32>& SelectedModeIndices = SharedData->GetSelectedModeIndices();
			if (ensureMsgf(SelectedModeIndices.Num() == 1, TEXT("Trying to Add Function, but many or no Mode is selected. This should not be possible.")))
			{
				const FScopedTransaction AddFunctionTransaction(LOCTEXT("AddFunctionTransaction", "Add Fixture Function"));
				FixtureType->PreEditChange(FDMXFixtureMode::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FDMXFixtureMode, Functions)));

				const int32 NewFunctionIndex = FixtureType->AddFunction(SelectedModeIndices[0]);

				FixtureType->PostEditChange();

				if (NewFunctionIndex != INDEX_NONE)
				{
					constexpr bool bMatrixSelected = false;
					SharedData->SetFunctionAndMatrixSelection(TArray<int32>({ NewFunctionIndex }), bMatrixSelected);
				}

				return FReply::Handled();
			}
		}
	}

	return FReply::Unhandled();
}

FText SDMXFixtureTypeFunctionsEditorCategoryRow::GetAddFunctionButtonText() const
{
	if (GetIsAddFunctionButtonEnabled())
	{
		return FDMXEditorCommands::Get().AddNewModeFunction->GetLabel();
	}

	return LOCTEXT("AddFunctionButtonDisabled", "Please select a single Mode to add a Function to it.");
}

#undef LOCTEXT_NAMESPACE
