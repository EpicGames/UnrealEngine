// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/SCompoundWidget.h"

class FDMXEditor;
class FDMXFixtureTypeSharedData;


/** The Category Row for the Functions Editor. Note, not a TableRow, but just the topmost widget. */
class SDMXFixtureTypeFunctionsEditorCategoryRow
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMXFixtureTypeFunctionsEditorCategoryRow)
	{}

		/** Called when the search text changed */
		SLATE_EVENT(FOnTextChanged, OnSearchTextChanged)

	SLATE_END_ARGS()

	/** Constructs this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<FDMXEditor>& InDMXEditor);
		
private:
	/** Called to determine wether the 'Add New Mode' button is enabled */
	bool GetIsAddFunctionButtonEnabled() const;

	/** Called to get the tooltip text of the 'Add New Mode' button */
	FText GetAddFunctionButtonTooltipText() const;

	/** Called when the 'Add New Mode' button was clicked */
	FReply OnAddFunctionButtonClicked() const;

	/** Called to get the text of the 'Add New Mode' button */
	FText GetAddFunctionButtonText() const;

	/** Fixture type shared data */
	TSharedPtr<FDMXFixtureTypeSharedData> SharedData;

	/** The DMX Editor that owns this widget */
	TWeakPtr<FDMXEditor> WeakDMXEditor;
};
