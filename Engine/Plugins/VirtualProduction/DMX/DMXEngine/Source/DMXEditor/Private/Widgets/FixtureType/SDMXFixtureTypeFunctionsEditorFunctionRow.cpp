// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXFixtureTypeFunctionsEditorFunctionRow.h"

#include "DMXAttribute.h"
#include "DMXEditor.h"
#include "DMXEditorStyle.h"
#include "DMXFixtureTypeSharedData.h"
#include "DragDrop/DMXFixtureFunctionDragDropOp.h"
#include "DragDrop/DMXFixtureMatrixDragDropOp.h"
#include "Library/DMXEntityFixtureType.h"
#include "ScopedTransaction.h"
#include "Widgets/SNameListPicker.h"
#include "Widgets/FixtureType/DMXFixtureTypeFunctionsEditorFunctionItem.h"
#include "Widgets/FixtureType/DMXFixtureTypeFunctionsEditorMatrixItem.h"
#include "Widgets/FixtureType/SDMXFixtureTypeFunctionsEditor.h"
#include "Widgets/FixtureType/SDMXFixtureTypeFunctionsEditorMatrixRow.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"


#define LOCTEXT_NAMESPACE "SDMXFixtureTypeFunctionsEditorFunctionRow"

void SDMXFixtureTypeFunctionsEditorFunctionRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, const TSharedRef<FDMXFixtureTypeFunctionsEditorFunctionItem>& InFunctionItem)
{
	FunctionItem = InFunctionItem;
	IsSelected = InArgs._IsSelected;

	SMultiColumnTableRow<TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>>::Construct(
		FSuperRowType::FArguments()
		.OnDrop(this, &SDMXFixtureTypeFunctionsEditorFunctionRow::OnRowDrop)
		.OnDragEnter(this, &SDMXFixtureTypeFunctionsEditorFunctionRow::OnRowDragEnter)
		.OnDragLeave(this, &SDMXFixtureTypeFunctionsEditorFunctionRow::OnRowDragLeave),
		OwnerTable
	);
}

void SDMXFixtureTypeFunctionsEditorFunctionRow::EnterFunctionNameEditingMode()
{
	FunctionNameEditableTextBlock->EnterEditingMode();
}

FReply SDMXFixtureTypeFunctionsEditorFunctionRow::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const bool bLeftMouseButton = MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton);
	const TSharedPtr<ITypedTableView<TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>>> OwnerTable = OwnerTablePtr.Pin();
	
	if (bLeftMouseButton && OwnerTable.IsValid())
	{
		if(OwnerTable->Private_GetNumSelectedItems() == 1)
		{
			TSharedRef<FDMXFixtureFunctionDragDropOp> DragDropOp = MakeShared<FDMXFixtureFunctionDragDropOp>(SharedThis(this));

			return FReply::Handled().BeginDragDrop(DragDropOp);
		}
	}

	return FReply::Unhandled();
}

TSharedRef<SWidget> SDMXFixtureTypeFunctionsEditorFunctionRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == SDMXFixtureTypeFunctionsEditor::FCollumnID::Status)
	{
		return
			SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image_Lambda([this]()
					{
						if (!FunctionItem->ErrorStatus.IsEmpty())
						{
							return FAppStyle::GetBrush("Icons.Error");
						}

						if (!FunctionItem->WarningStatus.IsEmpty())
						{
							return FAppStyle::GetBrush("Icons.Warning");
						}

						static const FSlateBrush EmptyBrush = FSlateNoResource();
						return &EmptyBrush;
					})
				.ToolTipText_Lambda([this]()
					{
						if (!FunctionItem->ErrorStatus.IsEmpty())
						{
							return FunctionItem->ErrorStatus;
						}
						else if (!FunctionItem->WarningStatus.IsEmpty())
						{
							return FunctionItem->WarningStatus;
						}

						return FText::GetEmpty();
					})
			];
	}
	else if (ColumnName == SDMXFixtureTypeFunctionsEditor::FCollumnID::Channel)
	{
		return
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.Padding(4.f)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			[
				SAssignNew(StartingChannelEditableTextBlock, SInlineEditableTextBlock)
				.Text_Lambda([this]()
				{
					const int32 StartingChannel = FunctionItem->GetStartingChannel();
					return FText::AsNumber(StartingChannel);
				})
				.Font(FCoreStyle::GetDefaultFontStyle("Regular", 10))
				.IsReadOnly(false)
				.OnVerifyTextChanged(this, &SDMXFixtureTypeFunctionsEditorFunctionRow::OnVerifyStartingChannelChanged)
				.OnTextCommitted(this, &SDMXFixtureTypeFunctionsEditorFunctionRow::OnStartingChannelCommitted)
				.IsSelected(IsSelected)
			];
	}
	else if (ColumnName == SDMXFixtureTypeFunctionsEditor::FCollumnID::Name)
	{
		return
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.Padding(4.f)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			[
				SAssignNew(FunctionNameEditableTextBlock, SInlineEditableTextBlock)
				.Text_Lambda([this]()
					{
						const FText FunctionName = FunctionItem->GetFunctionName();
						if (FunctionItem->HasValidAttribute())
						{
							return FunctionName;
						}
						else if (!FunctionName.IsEmpty())
						{
							return FText::Format(LOCTEXT("InvalidAttributeFunctionName", "<{0}>"), FunctionItem->GetFunctionName());
						}
						else
						{
							return LOCTEXT("InvalidAttributeFunctionName2", "<Unnamed>");
						}
					})
				.IsEnabled_Lambda([this]()
					{
						return FunctionItem->HasValidAttribute();
					})
				.ToolTipText_Lambda([this]
					{
						return LOCTEXT("InvalidAttributeFunctionNameTooltip", "Empty Channel - No Attribute Set");
					})
				.Font(FCoreStyle::GetDefaultFontStyle("Regular", 10))
				.IsReadOnly(false)
				.OnVerifyTextChanged(this, &SDMXFixtureTypeFunctionsEditorFunctionRow::OnVerifyFunctionNameChanged)
				.OnTextCommitted(this, &SDMXFixtureTypeFunctionsEditorFunctionRow::OnFunctionNameCommitted)
				.IsSelected(IsSelected)
			];
	}
	else if (ColumnName == SDMXFixtureTypeFunctionsEditor::FCollumnID::Attribute)
	{
		return
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.Padding(4.f)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			[
				SNew(SNameListPicker)
				.OptionsSource(MakeAttributeLambda(&FDMXAttributeName::GetPredefinedValues))
				.Value(this, &SDMXFixtureTypeFunctionsEditorFunctionRow::GetAttributeName)
				.bDisplayWarningIcon(true)
				.OnValueChanged(this, &SDMXFixtureTypeFunctionsEditorFunctionRow::OnUserChangedAttributeName)
			];
	}
	else if (ColumnName == SDMXFixtureTypeFunctionsEditor::FCollumnID::DeleteAttribute)
	{
		return
			SNew(SButton)
			.ContentPadding(2.0f)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.OnClicked(this, &SDMXFixtureTypeFunctionsEditorFunctionRow::OnDeleteAttributeClicked)
			.ToolTipText(LOCTEXT("RemoveCellAttributeTooltip", "Removes the Cell Attribute"))
			.ForegroundColor(FSlateColor::UseForeground())
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Content()
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.Delete"))
				.ColorAndOpacity(FLinearColor(0.6f, 0.6f, 0.6f))
			];
	}

	return SNullWidget::NullWidget;
}

FReply SDMXFixtureTypeFunctionsEditorFunctionRow::OnRowDrop(const FDragDropEvent& DragDropEvent)
{
	if (const TSharedPtr<FDMXFixtureFunctionDragDropOp> FixtureFunctionDragDropOp = DragDropEvent.GetOperationAs<FDMXFixtureFunctionDragDropOp>())
	{
		if (const TSharedPtr<SDMXFixtureTypeFunctionsEditorFunctionRow> DroppedRow = FixtureFunctionDragDropOp->Row.Pin())
		{
			if (const TSharedPtr<FDMXFixtureTypeFunctionsEditorFunctionItem> DroppedItem = DroppedRow->GetFunctionItem())
			{
				if (const TSharedPtr<FDMXFixtureTypeSharedData>& SharedData = DroppedItem->GetFixtureTypeSharedData())
				{
					if (UDMXEntityFixtureType* ParentFixtureType = DroppedItem->GetFixtureType().Get())
					{
						// Only allow drag drop within the same editor
						if (FunctionItem->GetDMXEditor() == DroppedItem->GetDMXEditor())
						{
							const int32 FunctionToReorderIndex = DroppedRow->GetFunctionItem()->GetFunctionIndex();
							const int32 InsertAtIndex = FunctionItem->GetFunctionIndex();

							const FScopedTransaction ReorderFunctionTransaction(LOCTEXT("ReorderFunctionTransaction", "Reorder Fixture Function"));
							ParentFixtureType->PreEditChange(FDMXFixtureFunction::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FDMXFixtureFunction, Channel)));

							ParentFixtureType->ReorderFunction(FunctionItem->GetModeIndex(), FunctionToReorderIndex, InsertAtIndex);

							ParentFixtureType->PostEditChange();

							// Select the Function, unselect the Matrix
							constexpr bool bMatrixSelected = false;
							SharedData->SetFunctionAndMatrixSelection(TArray<int32>({ InsertAtIndex }), bMatrixSelected);

							return FReply::Handled();
						}
					}
				}
			}
		}
	}
	else if (const TSharedPtr<FDMXFixtureMatrixDragDropOp> FixtureMatrixDragDropOp = DragDropEvent.GetOperationAs<FDMXFixtureMatrixDragDropOp>())
	{
		if (const TSharedPtr<SDMXFixtureTypeFunctionsEditorMatrixRow> DroppedRow = FixtureMatrixDragDropOp->Row.Pin())
		{
			if (const TSharedPtr<FDMXFixtureTypeFunctionsEditorMatrixItem> DroppedItem = DroppedRow->GetMatrixItem())
			{
				if (const TSharedPtr<FDMXFixtureTypeSharedData>& SharedData = FunctionItem->GetFixtureTypeSharedData())
				{
					if (UDMXEntityFixtureType* ParentFixtureType = DroppedItem->GetFixtureType().Get())
					{
						// Only allow drag drop within the same editor
						if (FunctionItem->GetDMXEditor() == DroppedItem->GetDMXEditor())
						{
							const FScopedTransaction ReorderMatrixTransaction(LOCTEXT("ReorderMatrixTransaction", "Reorder Fixture Matrix"));
							ParentFixtureType->PreEditChange(FDMXFixtureFunction::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FDMXFixtureMatrix, FirstCellChannel)));

							ParentFixtureType->ReorderMatrix(FunctionItem->GetModeIndex(), FunctionItem->GetFunctionIndex());

							ParentFixtureType->PostEditChange();

							// Clear the selected functions and select the matrix
							constexpr bool bMatrixSelected = true;
							SharedData->SetFunctionAndMatrixSelection(TArray<int32>(), bMatrixSelected);
						}
					}
				}
			}
		}
	}

	return FReply::Unhandled();
}

void SDMXFixtureTypeFunctionsEditorFunctionRow::OnRowDragEnter(const FDragDropEvent& DragDropEvent)
{
	const TSharedPtr<FDMXFixtureFunctionDragDropOp> FixtureFunctionDragDropOp = DragDropEvent.GetOperationAs<FDMXFixtureFunctionDragDropOp>();
	const TSharedPtr<FDMXFixtureMatrixDragDropOp> FixtureMatrixDragDropOp = DragDropEvent.GetOperationAs<FDMXFixtureMatrixDragDropOp>();
	if (FixtureFunctionDragDropOp.IsValid() || FixtureMatrixDragDropOp.IsValid())
	{
		bIsDragDropTarget = true;
	}
}

void SDMXFixtureTypeFunctionsEditorFunctionRow::OnRowDragLeave(const FDragDropEvent& DragDropEvent)
{
	bIsDragDropTarget = false;
}

bool SDMXFixtureTypeFunctionsEditorFunctionRow::OnVerifyStartingChannelChanged(const FText& InNewText, FText& OutErrorMessage)
{
	FString StringValue = InNewText.ToString();
	int32 Value;
	if (LexTryParseString<int32>(Value, *StringValue))
	{
		if (Value > 0 && Value <= DMX_MAX_ADDRESS)
		{
			return true;
		}
	}

	OutErrorMessage = LOCTEXT("InvalidStartingChannelError", "Channel must be set to a value between 1 and 512");

	return false;
}

void SDMXFixtureTypeFunctionsEditorFunctionRow::OnStartingChannelCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	FString StringValue = InNewText.ToString();
	int32 StartingChannel;
	if (LexTryParseString<int32>(StartingChannel, *StringValue))
	{
		StartingChannel = FMath::Clamp(StartingChannel, 1, DMX_MAX_ADDRESS);
		FunctionItem->SetStartingChannel(StartingChannel);
	}
	else
	{
		StartingChannel = FunctionItem->GetStartingChannel();
	}

	StartingChannelEditableTextBlock->SetText(FText::AsNumber(StartingChannel));
}

bool SDMXFixtureTypeFunctionsEditorFunctionRow::OnVerifyFunctionNameChanged(const FText& InNewText, FText& OutErrorMessage)
{
	FText InvalidReason;
	const bool bValidFunctionName = FunctionItem->IsValidFunctionName(InNewText, InvalidReason);

	if (bValidFunctionName)
	{
		return true;
	}
	else
	{
		OutErrorMessage = InvalidReason;
		return false;
	}
}

void SDMXFixtureTypeFunctionsEditorFunctionRow::OnFunctionNameCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	FText UniqueFunctionName;
	FunctionItem->SetFunctionName(InNewText, UniqueFunctionName);

	FunctionNameEditableTextBlock->SetText(UniqueFunctionName);
}

FName SDMXFixtureTypeFunctionsEditorFunctionRow::GetAttributeName() const
{
	return FunctionItem->GetAttributeName().Name;
}

void SDMXFixtureTypeFunctionsEditorFunctionRow::OnUserChangedAttributeName(FName NewValue)
{
	FDMXAttributeName NewAttributeName;
	NewAttributeName.SetFromName(NewValue);

	FunctionItem->SetAttributeName(NewAttributeName);

	// Since edit mode can be entered without selecting the item, select it when a value is set
	if (TSharedPtr<FDMXEditor> DMXEditor = FunctionItem->GetDMXEditor())
	{
		if (TSharedPtr<FDMXFixtureTypeSharedData> SharedData = DMXEditor->GetFixtureTypeSharedData())
		{
			constexpr bool bMatrixSelected = false;
			SharedData->SetFunctionAndMatrixSelection({ FunctionItem->GetFunctionIndex() }, bMatrixSelected);
		}
	}
}

FReply SDMXFixtureTypeFunctionsEditorFunctionRow::OnDeleteAttributeClicked()
{
	FunctionItem->DeleteFunction();

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
