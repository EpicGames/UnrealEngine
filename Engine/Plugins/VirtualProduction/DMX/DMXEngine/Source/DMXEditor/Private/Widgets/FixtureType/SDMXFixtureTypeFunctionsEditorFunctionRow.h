// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

struct FDMXAttributeName;
class FDMXEditor;
class FDMXFixtureTypeFunctionsEditorFunctionItem;
class FDMXFixtureTypeFunctionsEditorItemBase;
class FDMXFixtureTypeSharedData;
class UDMXEntityFixtureType;

class SInlineEditableTextBlock;
class SPopupErrorText;


/** Function as a row in a list */
class SDMXFixtureTypeFunctionsEditorFunctionRow
	: public SMultiColumnTableRow<TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>>
{
public:
	SLATE_BEGIN_ARGS(SDMXFixtureTypeFunctionsEditorFunctionRow)
	{}

		/** Callback to check if the row is selected (should be hooked up if a parent widget is handling selection or focus) */
		SLATE_EVENT(FIsSelected, IsSelected)

	SLATE_END_ARGS()

	/** Constructs this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, const TSharedRef<FDMXFixtureTypeFunctionsEditorFunctionItem>& InFunctionItem);

	/** Enters editing the Function name in its text box */
	void EnterFunctionNameEditingMode();

	/** Returns the Function Item */
	const TSharedPtr<FDMXFixtureTypeFunctionsEditorFunctionItem> GetFunctionItem() const { return FunctionItem; }

	/** Returns true if the row is being dragged */
	bool IsBeginDragged() const { return bIsBeingDragged; }

	/** Sets if the row is being dragged in a drag drop op */
	void SetIsBeingDragged(bool bDragged) { bIsBeingDragged = bDragged; }

protected:
	//~ Begin SWidget interface
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	//~ End SWidget interface

	//~ Begin SMultiColumnTableRow interface
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
	//~ End SMultiColumnTableRow interface

private:
	/** Called when a dragged row is dropped onto this row */
	FReply OnRowDrop(const FDragDropEvent& DragDropEvent);

	/** Called when a dragged row enters this row */
	void OnRowDragEnter(const FDragDropEvent& DragDropEvent);

	/** Called when a dragged row leaves this row */
	void OnRowDragLeave(const FDragDropEvent& DragDropEvent);

	/** Called to verify the Starting Channel when the User is interactively changing it */
	bool OnVerifyStartingChannelChanged(const FText& InNewText, FText& OutErrorMessage);

	/** Called when the Starting Channel was comitted */
	void OnStartingChannelCommitted(const FText& InNewText, ETextCommit::Type InTextCommit);

	/** Called to verify the Function Name when the User is interactively changing it */
	bool OnVerifyFunctionNameChanged(const FText& InNewText, FText& OutErrorMessage);

	/** Called when Function Name was comitted */
	void OnFunctionNameCommitted(const FText& InNewText, ETextCommit::Type InTextCommit);

	/** Gets the Attribute Name of the Function */
	FName GetAttributeName() const;

	/** Called when the Attribute Name changed */
	void OnUserChangedAttributeName(FName NewValue);

	/** Called when the delete attribute button was clicked */
	FReply OnDeleteAttributeClicked();

	/** True while this is being dragged in a drag drop op */
	bool bIsBeingDragged = false;

	/** True while this is hoverd in a drag drop op */
	bool bIsDragDropTarget = false;

	/** The mode name edit widget */
	TSharedPtr<SInlineEditableTextBlock> StartingChannelEditableTextBlock;

	/** The mode name edit widget */
	TSharedPtr<SInlineEditableTextBlock> FunctionNameEditableTextBlock;

	/** The Function Item this row displays */
	TSharedPtr<FDMXFixtureTypeFunctionsEditorFunctionItem> FunctionItem;

	// Slate arguments
	FIsSelected IsSelected;
};
