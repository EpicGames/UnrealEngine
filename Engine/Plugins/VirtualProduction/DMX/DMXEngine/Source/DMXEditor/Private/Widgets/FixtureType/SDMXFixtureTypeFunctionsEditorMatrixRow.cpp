// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXFixtureTypeFunctionsEditorMatrixRow.h"

#include "DMXEditor.h"
#include "DMXFixtureTypeSharedData.h"
#include "DragDrop/DMXFixtureFunctionDragDropOp.h"
#include "DragDrop/DMXFixtureMatrixDragDropOp.h"
#include "Library/DMXEntityFixtureType.h"
#include "Widgets/FixtureType/DMXFixtureTypeFunctionsEditorFunctionItem.h"
#include "Widgets/FixtureType/DMXFixtureTypeFunctionsEditorMatrixItem.h"
#include "Widgets/FixtureType/SDMXFixtureTypeFunctionsEditor.h"
#include "Widgets/FixtureType/SDMXFixtureTypeFunctionsEditorFunctionRow.h"
#include "Widgets/FixtureType/SDMXFixtureTypeMatrixFunctionsEditor.h"

#include "ScopedTransaction.h"
#include "Styling/CoreStyle.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"


#define LOCTEXT_NAMESPACE "SDMXFixtureTypeFunctionsEditorHeaderRow"

void SDMXFixtureTypeFunctionsEditorMatrixRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, const TSharedRef<FDMXFixtureTypeFunctionsEditorMatrixItem>& InMatrixItem)
{
	MatrixItem = InMatrixItem;
	IsSelected = InArgs._IsSelected;

	SMultiColumnTableRow<TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>>::Construct(
		FSuperRowType::FArguments()
		.OnDrop(this, &SDMXFixtureTypeFunctionsEditorMatrixRow::OnRowDrop)
		.OnDragEnter(this, &SDMXFixtureTypeFunctionsEditorMatrixRow::OnRowDragEnter)
		.OnDragLeave(this, &SDMXFixtureTypeFunctionsEditorMatrixRow::OnRowDragLeave),
		OwnerTable
	);
}

FReply SDMXFixtureTypeFunctionsEditorMatrixRow::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const bool bLeftMouseButton = MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton);
	const TSharedPtr<ITypedTableView<TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>>> OwnerTable = OwnerTablePtr.Pin();

	if (bLeftMouseButton && OwnerTable.IsValid())
	{
		if (OwnerTable->Private_GetNumSelectedItems() == 1)
		{
			TSharedRef<FDMXFixtureMatrixDragDropOp> DragDropOp = MakeShared<FDMXFixtureMatrixDragDropOp>(SharedThis(this));

			return FReply::Handled().BeginDragDrop(DragDropOp);
		}
	}

	return FReply::Unhandled();
}

TSharedRef<SWidget> SDMXFixtureTypeFunctionsEditorMatrixRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (TSharedPtr<FDMXEditor> DMXEditor = MatrixItem->GetDMXEditor())
	{
		TWeakObjectPtr<UDMXEntityFixtureType> FixtureType = MatrixItem->GetFixtureType();
		const int32 ModeIndex = MatrixItem->GetModeIndex();

		if (ColumnName == SDMXFixtureTypeFunctionsEditor::FCollumnID::Status)
		{
			return
				SNew(SBorder)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				[
					SNew(SImage)
					.Image_Lambda([this]()
						{
							if (!MatrixItem->ErrorStatus.IsEmpty())
							{
								return FAppStyle::GetBrush("Icons.Error");
							}

							if (!MatrixItem->WarningStatus.IsEmpty())
							{
								return FAppStyle::GetBrush("Icons.Warning");
							}
							static const FSlateBrush EmptyBrush = FSlateNoResource();
							return &EmptyBrush;
						})
					.ToolTipText_Lambda([this]()
						{
							if (!MatrixItem->ErrorStatus.IsEmpty())
							{
								return MatrixItem->ErrorStatus;
							}
							else if (!MatrixItem->WarningStatus.IsEmpty())
							{
								return MatrixItem->WarningStatus;
							}
							return FText::GetEmpty();
						})
				];
		}
		else if (ColumnName == SDMXFixtureTypeFunctionsEditor::FCollumnID::Channel)
		{
			return
				SNew(SBorder)
				.Padding(4.f)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				[
					SAssignNew(StartingChannelEditableTextBlock, SInlineEditableTextBlock)
					.Text_Lambda([this]()
						{
							const int32 StartingChannel = MatrixItem->GetStartingChannel();
							return FText::AsNumber(StartingChannel);
						})
					.Font(FCoreStyle::GetDefaultFontStyle("Regular", 10))
					.IsReadOnly(false)
					.OnVerifyTextChanged(this, &SDMXFixtureTypeFunctionsEditorMatrixRow::OnVerifyStartingChannelChanged)
					.OnTextCommitted(this, &SDMXFixtureTypeFunctionsEditorMatrixRow::OnStartingChannelCommitted)
					.IsSelected(IsSelected)
				];
		}
		else if (ColumnName == SDMXFixtureTypeFunctionsEditor::FCollumnID::Name)
		{
			return
				SNew(SBorder)
				.Padding(4.f)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Top)
				[
					SNew(SVerticalBox)

					// Title

					+ SVerticalBox::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Top)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("MatrixColumnTitle", "Fixture Matrix"))
						.Font(FCoreStyle::GetDefaultFontStyle("Bold", 10))
						.Margin(FMargin(2.f, 15.f, 2.f, 5.f))
					]

					// Matrix (e.g. 4x4)

					+ SVerticalBox::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Top)
					.Padding(FMargin(8.f, 0.f, 0.f, 0.f))
					[
						SNew(STextBlock)
						.Text_Lambda([this]()
							{
								const FDMXFixtureMatrix& Matrix = MatrixItem->GetFixtureMatrix();
								return FText::Format(LOCTEXT("MatrixCellConfiguration", "Cell configuration : {0}x{1}"), FText::AsNumber(Matrix.XCells), FText::AsNumber(Matrix.YCells));
							})
						.Font(FCoreStyle::GetDefaultFontStyle("Regular", 10))
						.Margin(FMargin(2.f, 15.f, 2.f, 5.f))
					]

					// Num Channels

					+ SVerticalBox::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Top)
					.Padding(FMargin(8.f, 0.f, 0.f, 0.f))
					[
						SNew(STextBlock)
						.Text_Lambda([this]()
							{
								const int32 NumChannels = MatrixItem->GetNumChannels();
								return FText::Format(LOCTEXT("MatrixNumChannels", "Number of DMX Channels : {0}"), FText::AsNumber(NumChannels));
							})
						.Font(FCoreStyle::GetDefaultFontStyle("Regular", 10))
						.Margin(FMargin(2.f, 15.f, 2.f, 5.f))
					]
				];
		}
		else if (ColumnName == SDMXFixtureTypeFunctionsEditor::FCollumnID::Attribute)
		{
			return
				SNew(SBorder)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Top)
				.Padding(4.f)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				[
					SNew(SDMXFixtureTypeMatrixFunctionsEditor, DMXEditor.ToSharedRef(), FixtureType, ModeIndex)
				];
		}
	}

	return SNullWidget::NullWidget;
}

FReply SDMXFixtureTypeFunctionsEditorMatrixRow::OnRowDrop(const FDragDropEvent& DragDropEvent)
{
	// No need to handle matrix drag drop ops, that would always mean to drop the row on itself
	if(const TSharedPtr<FDMXFixtureFunctionDragDropOp> FixtureFunctionDragDropOp = DragDropEvent.GetOperationAs<FDMXFixtureFunctionDragDropOp>())	
	{
		if (const TSharedPtr<SDMXFixtureTypeFunctionsEditorFunctionRow> DroppedRow = FixtureFunctionDragDropOp->Row.Pin())
		{
			if(const TSharedPtr<FDMXFixtureTypeFunctionsEditorFunctionItem> DroppedFunctionItem = DroppedRow->GetFunctionItem())
			{ 
				if (const TSharedPtr<FDMXFixtureTypeSharedData>& SharedData = DroppedFunctionItem->GetFixtureTypeSharedData())
				{
					if (UDMXEntityFixtureType* ParentFixtureType = MatrixItem->GetFixtureType().Get())
					{
						// Drop on the same editor
						if (MatrixItem->GetFixtureType() == ParentFixtureType)
						{
							const int32 ModeIndex = MatrixItem->GetModeIndex();

							if (ParentFixtureType->Modes.IsValidIndex(ModeIndex))
							{
								const int32 FunctionToReorderIndex = DroppedFunctionItem->GetFunctionIndex();
							
								const FScopedTransaction ReorderFunctionTransaction(LOCTEXT("ReorderFunctionTransaction", "Reorder Fixture Function"));
								ParentFixtureType->PreEditChange(FDMXFixtureFunction::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FDMXFixtureFunction, Channel)));

								ParentFixtureType->ReorderMatrix(ModeIndex, FunctionToReorderIndex);

								ParentFixtureType->PostEditChange();

								// Select the Function, unselect the Matrix
								constexpr bool bMatrixSelected = false;
								SharedData->SetFunctionAndMatrixSelection(TArray<int32>{ FunctionToReorderIndex }, bMatrixSelected);

								return FReply::Handled();
							}
						}
					}
				}
			}
		}
	}

	return FReply::Unhandled();
}

void SDMXFixtureTypeFunctionsEditorMatrixRow::OnRowDragEnter(const FDragDropEvent& DragDropEvent)
{
	if (const TSharedPtr<FDMXFixtureFunctionDragDropOp> FixtureFunctionDragDropOp = DragDropEvent.GetOperationAs<FDMXFixtureFunctionDragDropOp>())
	{
		bIsDragDropTarget = true;
	}
}

void SDMXFixtureTypeFunctionsEditorMatrixRow::OnRowDragLeave(const FDragDropEvent& DragDropEvent)
{
	if (const TSharedPtr<FDMXFixtureFunctionDragDropOp> FixtureFunctionDragDropOp = DragDropEvent.GetOperationAs<FDMXFixtureFunctionDragDropOp>())
	{
		bIsDragDropTarget = false;
	}
}

bool SDMXFixtureTypeFunctionsEditorMatrixRow::OnVerifyStartingChannelChanged(const FText& InNewText, FText& OutErrorMessage)
{
	FString StringValue = InNewText.ToString();
	int32 Value;
	if (LexTryParseString<int32>(Value, *StringValue))
	{
		if (Value > 0 && Value <= DMX_MAX_ADDRESS)
		{
			return true;
		}
	}

	OutErrorMessage = LOCTEXT("InvalidStartingChannelError", "Channel must be set to a value between 1 and 512");

	return false;
}

void SDMXFixtureTypeFunctionsEditorMatrixRow::OnStartingChannelCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	FString StringValue = InNewText.ToString();
	int32 StartingChannel;
	if (LexTryParseString<int32>(StartingChannel, *StringValue))
	{
		StartingChannel = FMath::Clamp(StartingChannel, 1, DMX_MAX_ADDRESS);
		MatrixItem->SetStartingChannel(StartingChannel);
	}
	else
	{
		StartingChannel = MatrixItem->GetStartingChannel();
	}

	StartingChannelEditableTextBlock->SetText(FText::AsNumber(StartingChannel));
}

#undef LOCTEXT_NAMESPACE
