// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class FDMXFixtureTypeFunctionsEditorItemBase;
class FDMXFixtureTypeFunctionsEditorMatrixItem;
class UDMXEntityFixtureType;

class SInlineEditableTextBlock;


/** The Matrix as row in a list of Functions and the Matrix row */
class SDMXFixtureTypeFunctionsEditorMatrixRow
	: public SMultiColumnTableRow<TSharedPtr<FDMXFixtureTypeFunctionsEditorItemBase>>
{
public:
	SLATE_BEGIN_ARGS(SDMXFixtureTypeFunctionsEditorMatrixRow)
	{}
		/** Callback to check if the row is selected (should be hooked up if a parent widget is handling selection or focus) */
		SLATE_EVENT(FIsSelected, IsSelected)

	SLATE_END_ARGS()

	/** Constructs this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, const TSharedRef<FDMXFixtureTypeFunctionsEditorMatrixItem>& InMatrixItem);

	/** Returns the Matrix Item */
	const TSharedPtr<FDMXFixtureTypeFunctionsEditorMatrixItem> GetMatrixItem() const { return MatrixItem; }

	/** Returns true if the row is being dragged */
	bool IsBeginDragged() const { return bIsBeingDragged; }

	/** Sets if the row is being dragged in a drag drop op */
	void SetIsBeingDragged(bool bDragged) { bIsBeingDragged = bDragged; }

protected:
	//~ Begin SWidget interface
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	//~ End SWidget interface

	//~ Begin SMultiColumnTableRow interface
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
	//~ End SMultiColumnTableRow interface

private:
	/** Called when a dragged row is dropped onto this row */
	FReply OnRowDrop(const FDragDropEvent& DragDropEvent);

	/** Called when a dragged row enters this row */
	void OnRowDragEnter(const FDragDropEvent& DragDropEvent);

	/** Called when a dragged row leaves this row */
	void OnRowDragLeave(const FDragDropEvent& DragDropEvent);

	/** Called to verify the Starting Channel when the User is interactively changing it */
	bool OnVerifyStartingChannelChanged(const FText& InNewText, FText& OutErrorMessage);

	/** Called when the Starting Channel was comitted */
	void OnStartingChannelCommitted(const FText& InNewText, ETextCommit::Type InTextCommit);

	/** True while this is being dragged in a drag drop op */
	bool bIsBeingDragged = false;

	/** True while this is hoverd in a drag drop op */
	bool bIsDragDropTarget = false;

	/** The Mode Item this row displays */
	TSharedPtr<FDMXFixtureTypeFunctionsEditorMatrixItem> MatrixItem;

	/** Text block to display and edit the matrix starting channel */
	TSharedPtr<SInlineEditableTextBlock> StartingChannelEditableTextBlock;

	// Slate arguments
	FIsSelected IsSelected;
};
