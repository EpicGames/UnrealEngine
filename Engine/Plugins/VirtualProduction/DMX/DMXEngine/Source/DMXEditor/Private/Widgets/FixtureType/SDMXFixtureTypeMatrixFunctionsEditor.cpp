// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXFixtureTypeMatrixFunctionsEditor.h"

#include "DMXEditor.h"
#include "Library/DMXEntityFixtureType.h"
#include "Widgets/FixtureType/DMXFixtureTypeMatrixFunctionsEditorItem.h"
#include "Widgets/FixtureType/SDMXFixtureTypeMatrixFunctionsEditorCategoryRow.h"
#include "Widgets/FixtureType/SDMXFixtureTypeMatrixFunctionsEditorMatrixRow.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"


#define LOCTEXT_NAMESPACE "SDMXFixtureTypeMatrixFunctionsEditor"

const FName FDMXFixtureTypeMatrixFunctionsEditorCollumnIDs::Status = "Status";
const FName FDMXFixtureTypeMatrixFunctionsEditorCollumnIDs::Channel = "Channel";
const FName FDMXFixtureTypeMatrixFunctionsEditorCollumnIDs::Attribute = "Attribute";
const FName FDMXFixtureTypeMatrixFunctionsEditorCollumnIDs::DeleteAttribute = "DeleteAttribute";

void SDMXFixtureTypeMatrixFunctionsEditor::Construct(const FArguments& InArgs, const TSharedRef<FDMXEditor>& InDMXEditor, TWeakObjectPtr<UDMXEntityFixtureType> InFixtureType, int32 InModeIndex)
{
	WeakDMXEditor = InDMXEditor;
	FixtureType = InFixtureType;
	ModeIndex = InModeIndex;

	UDMXEntityFixtureType::GetOnFixtureTypeChanged().AddSP(this, &SDMXFixtureTypeMatrixFunctionsEditor::OnFixtureTypeChanged);	
	
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
		.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f))
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			[
				SNew(SDMXFixtureTypeMatrixFunctionsEditorCategoryRow, InDMXEditor)
			]

			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			.AutoHeight()
			[
				SAssignNew(CellAttributesListView, SListView<TSharedPtr<FDMXFixtureTypeMatrixFunctionsEditorItem>>)
				.HeaderRow(GenerateHeaderRow())
				.ListItemsSource(&CellAttributesListSource)
				.OnGenerateRow(this, &SDMXFixtureTypeMatrixFunctionsEditor::OnGenerateCellAttributeRow)
				.SelectionMode(ESelectionMode::None)
			]
		]
	];

	RefreshList();
}

TSharedRef<SHeaderRow> SDMXFixtureTypeMatrixFunctionsEditor::GenerateHeaderRow()
{
	const float StatusColumnWidth = FMath::Max(FAppStyle::GetBrush("Icons.Warning")->GetImageSize().X + 8.f, FAppStyle::GetBrush("Icons.Error")->GetImageSize().X + 8.f);
	const float DeleteAttributeColumnWidth = FAppStyle::GetBrush("Icons.Delete")->GetImageSize().X + 8.f;

	TAttribute<EVisibility> HeaderRowVisiblityAttribute = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([this]()
		{
			return CellAttributesListSource.Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed;
		}));

	TSharedRef<SHeaderRow> HeaderRow = SNew(SHeaderRow).Visibility(HeaderRowVisiblityAttribute);
	SHeaderRow::FColumn::FArguments ColumnArgs;

	HeaderRow->AddColumn(
		SHeaderRow::FColumn::FArguments()
		.ColumnId(FDMXFixtureTypeMatrixFunctionsEditorCollumnIDs::Status)
		.DefaultLabel(FText())
		.FixedWidth(StatusColumnWidth)
	);

	HeaderRow->AddColumn(
		SHeaderRow::FColumn::FArguments()
		.ColumnId(FDMXFixtureTypeMatrixFunctionsEditorCollumnIDs::Channel)
		.DefaultLabel(LOCTEXT("ChannelColumnLabel", "Rel. Ch."))
		.FixedWidth(56.f)
	);

	HeaderRow->AddColumn(
		SHeaderRow::FColumn::FArguments()
		.ColumnId(FDMXFixtureTypeMatrixFunctionsEditorCollumnIDs::Attribute)
		.DefaultLabel(LOCTEXT("AttributeColumnLabel", "Attribute"))
		.FillWidth(1.f)
	);

	HeaderRow->AddColumn(
		SHeaderRow::FColumn::FArguments()
		.ColumnId(FDMXFixtureTypeMatrixFunctionsEditorCollumnIDs::DeleteAttribute)
		.DefaultLabel(FText())
		.FixedWidth(DeleteAttributeColumnWidth)
	);

	return HeaderRow;
}

TSharedRef<ITableRow> SDMXFixtureTypeMatrixFunctionsEditor::OnGenerateCellAttributeRow(TSharedPtr<FDMXFixtureTypeMatrixFunctionsEditorItem> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<SDMXFixtureTypeMatrixFunctionsEditorMatrixRow> CellAttributeRow = 
		SNew(SDMXFixtureTypeMatrixFunctionsEditorMatrixRow, OwnerTable, InItem.ToSharedRef())
		.OnRequestDelete(this, &SDMXFixtureTypeMatrixFunctionsEditor::OnCellAttributeRowRequestDelete, InItem);

	return CellAttributeRow;
}

void SDMXFixtureTypeMatrixFunctionsEditor::OnCellAttributeRowRequestDelete(TSharedPtr<FDMXFixtureTypeMatrixFunctionsEditorItem> RowItem)
{
	RowItem->RemoveFromFixtureType();
		
	RefreshList();
}

void SDMXFixtureTypeMatrixFunctionsEditor::OnFixtureTypeChanged(const UDMXEntityFixtureType* ChangedFixtureType)
{
	RefreshList();
}

void SDMXFixtureTypeMatrixFunctionsEditor::RefreshList()
{
	CellAttributesListSource.Reset();

	// Rebuild List Source
	if (TSharedPtr<FDMXEditor> DMXEditor = WeakDMXEditor.Pin())
	{
		if (FixtureType.IsValid() && FixtureType->Modes.IsValidIndex(ModeIndex))
		{
			const FDMXFixtureMode& Mode = FixtureType->Modes[ModeIndex]; 

			TMap<FDMXAttributeName, TSet<TSharedPtr<FDMXFixtureTypeMatrixFunctionsEditorItem>>> AttributeToConflictingItemsMap;
			int32 ChannelOffset = 0;
			for (int32 CellAttributeIndex = 0; CellAttributeIndex < Mode.FixtureMatrixConfig.CellAttributes.Num(); CellAttributeIndex++)
			{
				const FDMXFixtureCellAttribute& CellAttribute = Mode.FixtureMatrixConfig.CellAttributes[CellAttributeIndex];
				const int32 ChannelNumber = CellAttributeIndex + 1 + ChannelOffset;

				TSharedPtr<FDMXFixtureTypeMatrixFunctionsEditorItem> MatrixFunctionItem = MakeShared<FDMXFixtureTypeMatrixFunctionsEditorItem>(DMXEditor.ToSharedRef(), FixtureType, ModeIndex, CellAttributeIndex);
				MatrixFunctionItem->ChannelNumberText = FText::AsNumber(ChannelNumber);
				CellAttributesListSource.Add(MatrixFunctionItem);

				const FDMXAttributeName& Attribute = CellAttribute.Attribute.Name;
				AttributeToConflictingItemsMap.FindOrAdd(Attribute).Add(MatrixFunctionItem);

				ChannelOffset += CellAttribute.GetNumChannels() - 1;
			}

			// Create item status
			for (const TTuple<FDMXAttributeName, TSet<TSharedPtr<FDMXFixtureTypeMatrixFunctionsEditorItem>>>& AttributeToConflictingItemsPair : AttributeToConflictingItemsMap)
			{
				if (AttributeToConflictingItemsPair.Value.Num() > 1 && AttributeToConflictingItemsPair.Key.Name != NAME_None)
				{
					for (const TSharedPtr<FDMXFixtureTypeMatrixFunctionsEditorItem>& ConflictingItem : AttributeToConflictingItemsPair.Value)
					{
						ConflictingItem->WarningStatus = FText::Format(LOCTEXT("AmbiguousCellAttribute", "Attribute {0} used more than once."), FText::FromName(AttributeToConflictingItemsPair.Key.Name));
					}
				}
			}
		}
	}

	CellAttributesListView->RequestListRefresh();
}

#undef LOCTEXT_NAMESPACE
