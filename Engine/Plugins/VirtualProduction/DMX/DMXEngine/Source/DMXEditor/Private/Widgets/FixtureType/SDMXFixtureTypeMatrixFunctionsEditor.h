// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"

class FDMXEditor;
class FDMXFixtureTypeMatrixFunctionsEditorItem;
class UDMXEntityFixtureType;

class ITableRow;
class SHeaderRow;
template <typename ItemType> class SListView;
class STableViewBase;
class SVerticalBox;


/** Collumn IDs in the Matrix Functions Editor */
struct FDMXFixtureTypeMatrixFunctionsEditorCollumnIDs
{
	static const FName Status;
	static const FName Channel;
	static const FName Attribute;
	static const FName DeleteAttribute;
};


/** Editor for the Cell Attributes Array of the Modes in a Fixture Type */
class SDMXFixtureTypeMatrixFunctionsEditor
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMXFixtureTypeMatrixFunctionsEditor)
	{}
		
	SLATE_END_ARGS()

	/** Constructs this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<FDMXEditor>& InDMXEditor, TWeakObjectPtr<UDMXEntityFixtureType> InFixtureType, int32 InModeIndex);

private:
	/** Generates a heaer row for the Function List */
	TSharedRef<SHeaderRow> GenerateHeaderRow();

	/** Called when a Cell Attributes List View row is generated */
	TSharedRef<ITableRow> OnGenerateCellAttributeRow(TSharedPtr<FDMXFixtureTypeMatrixFunctionsEditorItem> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Called when a Cell Attribute Row wants to be deleted */
	void OnCellAttributeRowRequestDelete(TSharedPtr<FDMXFixtureTypeMatrixFunctionsEditorItem> RowItem);

	/** Called when a fixture type changed */
	void OnFixtureTypeChanged(const UDMXEntityFixtureType* ChangedFixtureType);

	/** Refreshes the List of Matrix Functions */
	void RefreshList();

	/** The index of the mode in the Fixture Type's Modes array */
	int32 ModeIndex = INDEX_NONE;

	/** The Fixture Type which owns the Mode */
	TWeakObjectPtr<UDMXEntityFixtureType> FixtureType;

	/** List View of Cell Attributes */
	TSharedPtr<SListView<TSharedPtr<FDMXFixtureTypeMatrixFunctionsEditorItem>>> CellAttributesListView;

	/** Source for the Cell Attributes List View */
	TArray<TSharedPtr<FDMXFixtureTypeMatrixFunctionsEditorItem>> CellAttributesListSource;

	/** Pointer back to the DMXEditor tool that owns this widget */
	TWeakPtr<FDMXEditor> WeakDMXEditor;
};

