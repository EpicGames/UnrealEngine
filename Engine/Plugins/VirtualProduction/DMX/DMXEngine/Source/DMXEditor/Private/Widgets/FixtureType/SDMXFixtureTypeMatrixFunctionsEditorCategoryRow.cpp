// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXFixtureTypeMatrixFunctionsEditorCategoryRow.h"

#include "DMXEditor.h"
#include "DMXEditorStyle.h"
#include "DMXFixtureTypeSharedData.h"
#include "Commands/DMXEditorCommands.h"
#include "Library/DMXEntityFixtureType.h"

#include "Styling/AppStyle.h"
#include "ScopedTransaction.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "SDMXFixtureTypeMatrixFunctionsEditorCategoryRow"

void SDMXFixtureTypeMatrixFunctionsEditorCategoryRow::Construct(const FArguments& InArgs, const TSharedRef<FDMXEditor>& InDMXEditor)
{
	WeakDMXEditor = InDMXEditor;
	SharedData = InDMXEditor->GetFixtureTypeSharedData();

	ChildSlot
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Center)
	[
		SNew(SBorder)
		.Padding(3.0f)
		.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
		.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f))
		[
			SNew(SBox)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(FMargin(10.f, 2.f, 2.f, 2.f))
			[
				SNew(SButton)
				.IsEnabled(this, &SDMXFixtureTypeMatrixFunctionsEditorCategoryRow::GetIsAddCellAttributeButtonEnabled)
				.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
				.ForegroundColor(FLinearColor::White)
				.ToolTipText(this, &SDMXFixtureTypeMatrixFunctionsEditorCategoryRow::GetAddCellAttributeButtonTooltipText)
				.ContentPadding(FMargin(5.0f, 1.0f))
				.OnClicked(this, &SDMXFixtureTypeMatrixFunctionsEditorCategoryRow::OnAddCellAttributeButtonClicked)
				.Content()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(0.f, 1.f))
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Plus"))
					]
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(FMargin(2.f, 0.f, 2.f, 0.f))
					[
						SNew(STextBlock)
						.Text(this, &SDMXFixtureTypeMatrixFunctionsEditorCategoryRow::GetAddCellAttributeButtonText)
					]
				]
			]
		]
	];
}

bool SDMXFixtureTypeMatrixFunctionsEditorCategoryRow::GetIsAddCellAttributeButtonEnabled() const
{
	const TArray<TWeakObjectPtr<UDMXEntityFixtureType>>& SelectedFixtureTypes = SharedData->GetSelectedFixtureTypes();
	const TArray<int32>& SelectedModeIndices = SharedData->GetSelectedModeIndices();

	return SelectedFixtureTypes.Num() == 1 && SelectedModeIndices.Num() == 1;
}

FText SDMXFixtureTypeMatrixFunctionsEditorCategoryRow::GetAddCellAttributeButtonTooltipText() const
{
	if (GetIsAddCellAttributeButtonEnabled())
	{
		return FDMXEditorCommands::Get().AddNewFixtureTypeMode->GetDescription();
	}

	return NSLOCTEXT("SDMXFixtureTypeMatrixFunctionsEditorCategoryRow", "AddCellAttributeButtonDisabledTooltip", "Please select a single Mode to add a Cell Attribute to it.");
}

FReply SDMXFixtureTypeMatrixFunctionsEditorCategoryRow::OnAddCellAttributeButtonClicked() const
{
	const TArray<TWeakObjectPtr<UDMXEntityFixtureType>>& SelectedFixtureTypes = SharedData->GetSelectedFixtureTypes();
	if (ensureMsgf(SelectedFixtureTypes.Num() == 1, TEXT("Trying to Add Cell Attribute, but many or no Fixture Type is selected. This should not be possible.")))
	{
		if (UDMXEntityFixtureType* FixtureType = SelectedFixtureTypes[0].Get())
		{
			const TArray<int32>& SelectedModeIndices = SharedData->GetSelectedModeIndices();
			if (ensureMsgf(SelectedModeIndices.Num() == 1, TEXT("Trying to Add Cell Attribute, but many or no Mode is selected. This should not be possible.")))
			{
				const FScopedTransaction AddCellAttributeTransaction(LOCTEXT("AddCellAttributeTransaction", "Add Cell Attribute"));
				FixtureType->PreEditChange(UDMXEntityFixtureType::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FDMXFixtureMatrix, CellAttributes)));

				FixtureType->AddCellAttribute(SelectedModeIndices[0]);

				FixtureType->PostEditChange();
			}

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

FText SDMXFixtureTypeMatrixFunctionsEditorCategoryRow::GetAddCellAttributeButtonText() const
{
	if (GetIsAddCellAttributeButtonEnabled())
	{
		return NSLOCTEXT("SDMXFixtureTypeMatrixFunctionsEditorCategoryRow", "AddCellAttributeButtonText", "Add Cell Attribute");
	}

	return NSLOCTEXT("SDMXFixtureTypeMatrixFunctionsEditorCategoryRow", "AddCellAttributeButtonDisabledText", "Please select a single Mode to add a Function to it.");
}

#undef LOCTEXT_NAMESPACE
