// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXFixtureTypeMatrixFunctionsEditorMatrixRow.h"

#include "DMXEditor.h"
#include "DMXEditorStyle.h"
#include "Library/DMXEntityFixtureType.h"
#include "Widgets/SNameListPicker.h"
#include "Widgets/FixtureType/DMXFixtureTypeMatrixFunctionsEditorItem.h"
#include "Widgets/FixtureType/SDMXFixtureTypeMatrixFunctionsEditor.h"

#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Notifications/SPopUpErrorText.h"
#include "Styling/AppStyle.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"


#define LOCTEXT_NAMESPACE "SDMXFixtureTypeMatrixFunctionsEditorMatrixRow"

void SDMXFixtureTypeMatrixFunctionsEditorMatrixRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, TSharedRef<FDMXFixtureTypeMatrixFunctionsEditorItem> InCellAttributeItem)
{
	CellAttributeItem = InCellAttributeItem;
	OnRequestDelete = InArgs._OnRequestDelete;

	SMultiColumnTableRow<TSharedPtr<FDMXFixtureTypeMatrixFunctionsEditorItem>>::Construct(
		FSuperRowType::FArguments(),
		OwnerTable
	);
}

TSharedRef<SWidget> SDMXFixtureTypeMatrixFunctionsEditorMatrixRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == FDMXFixtureTypeMatrixFunctionsEditorCollumnIDs::Status)
	{
		return
			SNew(SBorder)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			[
				SNew(SImage)
				.Image_Lambda([this]()
					{
						if (!CellAttributeItem->ErrorStatus.IsEmpty())
						{
							return FAppStyle::GetBrush("Icons.Error");
						}

						if (!CellAttributeItem->WarningStatus.IsEmpty())
						{
							return FAppStyle::GetBrush("Icons.Warning");
						}
						static const FSlateBrush EmptyBrush = FSlateNoResource();
						return &EmptyBrush;
					})
				.ToolTipText_Lambda([this]()
					{
						if (!CellAttributeItem->ErrorStatus.IsEmpty())
						{
							return CellAttributeItem->ErrorStatus;
						}
						else if (!CellAttributeItem->WarningStatus.IsEmpty())
						{
							return CellAttributeItem->WarningStatus;
						}
						return FText::GetEmpty();
					})
			];
	}
	else if (ColumnName == FDMXFixtureTypeMatrixFunctionsEditorCollumnIDs::Channel)
	{
		return
			SNew(SBorder)
			.Padding(4.f)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			[
				SNew(STextBlock)
				.Text(CellAttributeItem->ChannelNumberText)
				.Font(FCoreStyle::GetDefaultFontStyle("Regular", 10))
			];
	}
	else if (ColumnName == FDMXFixtureTypeMatrixFunctionsEditorCollumnIDs::Attribute)
	{
		return
			SNew(SNameListPicker)
			.OptionsSource(MakeAttributeLambda(&FDMXAttributeName::GetPredefinedValues))
			.Value(this, &SDMXFixtureTypeMatrixFunctionsEditorMatrixRow::GetCellAttributeName)
			.bDisplayWarningIcon(true)
			.OnValueChanged(this, &SDMXFixtureTypeMatrixFunctionsEditorMatrixRow::SetCellAttributeName);
	}
	else if (ColumnName == FDMXFixtureTypeMatrixFunctionsEditorCollumnIDs::DeleteAttribute)
	{
		return
			SNew(SButton)
			.ContentPadding(2.0f)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.OnClicked(this, &SDMXFixtureTypeMatrixFunctionsEditorMatrixRow::OnDeleteCellAttributeClicked)
			.ToolTipText(LOCTEXT("RemoveCellAttributeTooltip", "Removes the Cell Attribute"))
			.ForegroundColor(FSlateColor::UseForeground())
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Content()
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.Delete"))
				.ColorAndOpacity(FLinearColor(0.6f, 0.6f, 0.6f))
			];
	}

	return SNullWidget::NullWidget;
}

FReply SDMXFixtureTypeMatrixFunctionsEditorMatrixRow::OnDeleteCellAttributeClicked()
{
	OnRequestDelete.ExecuteIfBound();

	return FReply::Handled();
}

FName SDMXFixtureTypeMatrixFunctionsEditorMatrixRow::GetCellAttributeName() const
{
	return CellAttributeItem->GetCellAttributeName().Name;
}

void SDMXFixtureTypeMatrixFunctionsEditorMatrixRow::SetCellAttributeName(FName NewValue)
{
	FDMXAttributeName NewAttributeName;
	NewAttributeName.SetFromName(NewValue);

	CellAttributeItem->SetCellAttributeName(NewAttributeName);
}

#undef LOCTEXT_NAMESPACE
