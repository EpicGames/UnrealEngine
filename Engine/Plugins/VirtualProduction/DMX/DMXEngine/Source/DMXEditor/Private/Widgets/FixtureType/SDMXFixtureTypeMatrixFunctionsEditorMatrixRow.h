// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

struct FDMXAttributeName;
class FDMXEditor;
class FDMXFixtureTypeMatrixFunctionsEditorItem;
class FDMXFixtureTypeSharedData;
class UDMXEntityFixtureType;

class SPopupErrorText;


/** A Matrix Function as a row in a list */
class SDMXFixtureTypeMatrixFunctionsEditorMatrixRow
	: public SMultiColumnTableRow<TSharedPtr<FDMXFixtureTypeMatrixFunctionsEditorItem>>
{
public:
	SLATE_BEGIN_ARGS(SDMXFixtureTypeMatrixFunctionsEditorMatrixRow)
	{}

		SLATE_EVENT(FSimpleDelegate, OnRequestDelete)

	SLATE_END_ARGS()

	/** Constructs this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, TSharedRef<FDMXFixtureTypeMatrixFunctionsEditorItem> InCellAttributeItem);

protected:
	//~ Begin SMultiColumnTableRow interface
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
	//~ End SMultiColumnTableRow interface

private:
	/** Called when the delete Cell Attribute Button was clicked */
	FReply OnDeleteCellAttributeClicked();

	/** Returns the Name of the Cell Attribute */
	FName GetCellAttributeName() const;

	/** Sets the Cell Attribute of the Matrix */
	void SetCellAttributeName(FName NewValue);

	/** The popup error displayed when an invalid name is set */
	TSharedPtr<SPopupErrorText> PopupErrorText;

	/** The Mode Item this row displays */
	TSharedPtr<FDMXFixtureTypeMatrixFunctionsEditorItem> CellAttributeItem;

	/** Delegate executed when the Matrix Row requests to be deleted */
	FSimpleDelegate OnRequestDelete;
};
