// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXFixtureTypeModesEditor.h"

#include "Algo/AnyOf.h"
#include "DMXEditor.h"
#include "DMXFixtureTypeSharedData.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Library/DMXEntityFixtureType.h"
#include "ScopedTransaction.h"
#include "SDMXFixtureTypeModesEditorCategoryRow.h"
#include "SDMXFixtureTypeModesEditorModeRow.h"
#include "Styling/AppStyle.h"
#include "Widgets/FixtureType/DMXFixtureTypeModesEditorModeItem.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SDMXFixtureTypeModesEditor"

void SDMXFixtureTypeModesEditor::Construct(const FArguments& InArgs, const TSharedRef<FDMXEditor>& InDMXEditor)
{
	WeakDMXEditor = InDMXEditor;
	FixtureTypeSharedData = InDMXEditor->GetFixtureTypeSharedData();
	
	FixtureTypeSharedData->OnFixtureTypesSelected.AddSP(this, &SDMXFixtureTypeModesEditor::RebuildList);
	FixtureTypeSharedData->OnModesSelected.AddSP(this, &SDMXFixtureTypeModesEditor::OnFixtureTypeSharedDataSelectedModes);

	UDMXEntityFixtureType::GetOnFixtureTypeChanged().AddSP(this, &SDMXFixtureTypeModesEditor::OnFixtureTypeChanged);

	static const FTableViewStyle TableViewStyle = FAppStyle::Get().GetWidgetStyle<FTableViewStyle>("TreeView");

	ChildSlot
	[
		SNew(SVerticalBox)
		
		// Header
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		.AutoHeight()
		[
			SNew(SDMXFixtureTypeModesEditorCategoryRow, InDMXEditor)
			.OnSearchTextChanged(this, &SDMXFixtureTypeModesEditor::OnSearchTextChanged)
		]

		// Modes 
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.FillHeight(1.f)
		[
			SAssignNew(ListContentBorder, SBorder)
			.Padding(2.0f)
			.BorderImage(&TableViewStyle.BackgroundBrush)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
		]

		// Revision
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.AutoHeight()
		[
			SNew(SBorder)
			.Padding(2.0f)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.ToolTipText(LOCTEXT("ShowAllRevisionsTooltip", "When checked, all revisions of the GDTF Modes in this GDTF are displayed, otherwise only the latest revisions are displayed.\nThis option is available as this Fixture Type is based on a GDTF and contains Modes with more than one revision."))
			.Visibility(this, &SDMXFixtureTypeModesEditor::GetRevisionCheckBoxVisibility)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(8.f, 0.f)
				.FillWidth(1.f)
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
					.Text(LOCTEXT("ShowAllRevisionsCheckboxLabel", "Show all revisions"))
				]

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SAssignNew(RevisionCheckBox, SCheckBox)
					.OnCheckStateChanged(this, &SDMXFixtureTypeModesEditor::OnRevisionCheckBoxStateChanged) 			
				]
			]
		]
	];

	RegisterCommands();
	RebuildList();
}

 FReply SDMXFixtureTypeModesEditor::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) 
{
	 if (CommandList->ProcessCommandBindings(InKeyEvent))
	 {
		 return FReply::Handled();
	 }

	 return FReply::Unhandled();
}

void SDMXFixtureTypeModesEditor::OnSearchTextChanged(const FText& InSearchText)
{
	SearchText = InSearchText;
	
	RebuildList();
}

 void SDMXFixtureTypeModesEditor::OnFixtureTypeChanged(const UDMXEntityFixtureType* ChangedFixtureType)
 {
	 if (ChangedFixtureType)
	 {	
		 RebuildList();
	 }
 }

void SDMXFixtureTypeModesEditor::RebuildList()
{
	ModeRows.Reset();
	ListSource.Reset();
	ListContentBorder->ClearContent();

	if (TSharedPtr<FDMXEditor> DMXEditor = WeakDMXEditor.Pin())
	{
		TArray<TWeakObjectPtr<UDMXEntityFixtureType>> SelectedFixtureTypes = FixtureTypeSharedData->GetSelectedFixtureTypes();

		// Only single fixture type editing is supported
		if (SelectedFixtureTypes.Num() == 1 && SelectedFixtureTypes[0].IsValid())
		{
			const UDMXEntityFixtureType& FixtureType = *SelectedFixtureTypes[0];

			// Update the revision check box visibility 
			bHasModeRevisions = false;

			// Rebuild the list source
			const FString SearchString = SearchText.ToString();
			
			for (int32 ModeIndex = 0; ModeIndex < FixtureType.Modes.Num(); ModeIndex++)
			{
				if (SearchString.IsEmpty() || FixtureType.Modes[ModeIndex].ModeName.Contains(SearchString, ESearchCase::IgnoreCase))
				{
					const TSharedRef<FDMXFixtureTypeModesEditorModeItem> ModeItem = MakeShared<FDMXFixtureTypeModesEditorModeItem>(DMXEditor.ToSharedRef(), SelectedFixtureTypes[0], ModeIndex);

					const bool bIsNewerRevision = 
						!ListSource.IsEmpty() &&
						!FixtureType.GDTFSource.IsNull() &&
						ModeItem->GetGDTFModeNumber() != INDEX_NONE &&
						ModeItem->GetGDTFModeNumber() == ListSource.Last()->GetGDTFModeNumber();

					bHasModeRevisions |= bIsNewerRevision;

					// If only the latest revision should be shown, pop the previous item if it uses the same GDTF mode number.
					if (bHasModeRevisions && bIsNewerRevision && FixtureType.bShowOnlyLatestGDTFModeRevisions)
					{				
						ListSource.Pop();
					}

					ListSource.Add(ModeItem);
				}
			}

			// Create the list
			ListContentBorder->SetContent
			(
				SAssignNew(ListView, SListView<TSharedPtr<FDMXFixtureTypeModesEditorModeItem>>)
				.ListItemsSource(&ListSource)
				.OnGenerateRow(this, &SDMXFixtureTypeModesEditor::OnGenerateModeRow)
				.OnSelectionChanged(this, &SDMXFixtureTypeModesEditor::OnListSelectionChanged)
				.OnContextMenuOpening(this, &SDMXFixtureTypeModesEditor::OnListContextMenuOpening)
				.ReturnFocusToSelection(false)
			);

			RebuildSelection();
		}
		else if (SelectedFixtureTypes.Num() == 0)
		{
			bHasModeRevisions = false;

			// Show a warning when no fixture type is selected
			ListContentBorder->SetContent
			(
				SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("NoFixtureTypeSelectedWarning", "No Fixture Type selected"))
				]
			);
		}
		else
		{
			bHasModeRevisions = false;

			// Show a warning when multiple fixture types are selected
			ListContentBorder->SetContent
			(
				SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("MultiEditingModesNotSupportedWarning", "Multi-Editing Fixture Types is not supported."))
				]
			);
		}
	}
}

void SDMXFixtureTypeModesEditor::RebuildSelection()
{
	// Select what's selected in Fixture Type Shared Data
	TArray<int32> SelectedModeIndices = FixtureTypeSharedData->GetSelectedModeIndices();
	TArray<TSharedPtr<FDMXFixtureTypeModesEditorModeItem>> NewSelection;
	for (const TSharedPtr<FDMXFixtureTypeModesEditorModeItem>& Item : ListSource)
	{
		if (SelectedModeIndices.Contains(Item->GetModeIndex()))
		{
			NewSelection.Add(Item);
		}
	}

	if (NewSelection.Num() > 0)
	{
		ListView->SetItemSelection(NewSelection, true, ESelectInfo::Direct);
		ListView->RequestScrollIntoView(NewSelection[0]);
	}
	else if (ListSource.Num() > 0)
	{
		// Since there was no selection, select any, as if the user clicked it
		ListView->SetSelection(ListSource[0], ESelectInfo::OnMouseClick);
	}
}

TSharedRef<ITableRow> SDMXFixtureTypeModesEditor::OnGenerateModeRow(TSharedPtr<FDMXFixtureTypeModesEditorModeItem> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<SDMXFixtureTypeModesEditorModeRow> ModeRow = 
		SNew(SDMXFixtureTypeModesEditorModeRow, OwnerTable, InItem.ToSharedRef())
		.IsSelected(this, &SDMXFixtureTypeModesEditor::IsItemSelectedExclusively, InItem);

	ModeRows.Add(ModeRow);

	return ModeRow;
}

bool SDMXFixtureTypeModesEditor::IsItemSelectedExclusively(TSharedPtr<FDMXFixtureTypeModesEditorModeItem> InItem) const
{
	return
		ListView->GetNumItemsSelected() == 1 &&
		ListView->IsItemSelected(InItem);
}

void SDMXFixtureTypeModesEditor::OnListSelectionChanged(TSharedPtr<FDMXFixtureTypeModesEditorModeItem> NewlySelectedItem, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo != ESelectInfo::Direct)
	{
		// Never clear the selection
		if (ListView.IsValid() && ListView->GetNumItemsSelected() == 0)
		{
			RebuildSelection();
		}

		FixtureTypeSharedData->SelectModes(GetListSelectionAsModeIndices());
	}
	else
	{
		ListView->RequestScrollIntoView(NewlySelectedItem);
	}
}

void SDMXFixtureTypeModesEditor::OnFixtureTypeSharedDataSelectedModes()
{
	if (ListView.IsValid())
	{
		const TArray<int32>& SelectedModeIndices = FixtureTypeSharedData->GetSelectedModeIndices();

		TArray<TSharedPtr<FDMXFixtureTypeModesEditorModeItem>> NewSelection;
		for (const TSharedPtr<FDMXFixtureTypeModesEditorModeItem>& Item : ListSource)
		{
			if (SelectedModeIndices.Contains(Item->GetModeIndex()))
			{
				NewSelection.Add(Item);
			}
		}

		ListView->ClearSelection();
		ListView->SetItemSelection(NewSelection, true, ESelectInfo::Direct);
	}
}

void SDMXFixtureTypeModesEditor::OnRevisionCheckBoxStateChanged(ECheckBoxState NewCheckState)
{
	const TArray<TWeakObjectPtr<UDMXEntityFixtureType>> SelectedFixtureTypes = FixtureTypeSharedData->GetSelectedFixtureTypes();
	if (ensureMsgf(SelectedFixtureTypes.Num() == 1, TEXT("Trying to set if latest mode revisions are shown for fixture type, but fixture type is invalid.")) &&
		ensureMsgf(!SelectedFixtureTypes[0]->GDTFSource.IsNull(), TEXT("Trying to set if latest mode revisions are shown for fixture type, but fixture type has no GDTF.")) &&
		SelectedFixtureTypes[0].IsValid())
	{
		SelectedFixtureTypes[0]->bShowOnlyLatestGDTFModeRevisions = NewCheckState != ECheckBoxState::Checked;

		RebuildList();
	}
}

EVisibility SDMXFixtureTypeModesEditor::GetRevisionCheckBoxVisibility() const
{
	return bHasModeRevisions ? EVisibility::Visible : EVisibility::Collapsed;
}

TArray<int32> SDMXFixtureTypeModesEditor::GetListSelectionAsModeIndices() const
{
	TArray<TSharedPtr<FDMXFixtureTypeModesEditorModeItem>> SelectedModeItems;
	ListView->GetSelectedItems(SelectedModeItems);

	TArray<int32> SelectedModeIndices;
	for (const TSharedPtr<FDMXFixtureTypeModesEditorModeItem>& ModeItem : SelectedModeItems)
	{
		if (ModeItem.IsValid() && ModeItem->GetModeIndex() != INDEX_NONE)
		{
			SelectedModeIndices.Add(ModeItem->GetModeIndex());
		}
	}

	return SelectedModeIndices;
}

TSharedPtr<SWidget> SDMXFixtureTypeModesEditor::OnListContextMenuOpening()
{
	const bool bCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bCloseWindowAfterMenuSelection, CommandList);

	MenuBuilder.BeginSection("BasicOperations");
	{
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SDMXFixtureTypeModesEditor::RegisterCommands()
{
	// Listen to common editor shortcuts for copy/paste etc
	if (!CommandList.IsValid())
	{
		CommandList = MakeShared<FUICommandList>();

		CommandList->MapAction(FGenericCommands::Get().Cut,
			FUIAction(FExecuteAction::CreateSP(this, &SDMXFixtureTypeModesEditor::OnCutSelectedItems),
				FCanExecuteAction::CreateSP(this, &SDMXFixtureTypeModesEditor::CanCutItems))
		);

		CommandList->MapAction(FGenericCommands::Get().Copy,
			FUIAction(FExecuteAction::CreateSP(this, &SDMXFixtureTypeModesEditor::OnCopySelectedItems),
				FCanExecuteAction::CreateSP(this, &SDMXFixtureTypeModesEditor::CanCopyItems))
		);

		CommandList->MapAction(FGenericCommands::Get().Paste,
			FUIAction(FExecuteAction::CreateSP(this, &SDMXFixtureTypeModesEditor::OnPasteItems),
				FCanExecuteAction::CreateSP(this, &SDMXFixtureTypeModesEditor::CanPasteItems))
		);

		CommandList->MapAction(FGenericCommands::Get().Duplicate,
			FUIAction(FExecuteAction::CreateSP(this, &SDMXFixtureTypeModesEditor::OnDuplicateItems),
				FCanExecuteAction::CreateSP(this, &SDMXFixtureTypeModesEditor::CanDuplicateItems))
		);

		CommandList->MapAction(FGenericCommands::Get().Delete,
			FUIAction(FExecuteAction::CreateSP(this, &SDMXFixtureTypeModesEditor::OnDeleteItems),
				FCanExecuteAction::CreateSP(this, &SDMXFixtureTypeModesEditor::CanDeleteItems))
		);

		CommandList->MapAction(FGenericCommands::Get().Rename,
			FUIAction(FExecuteAction::CreateSP(this, &SDMXFixtureTypeModesEditor::OnRenameItem),
				FCanExecuteAction::CreateSP(this, &SDMXFixtureTypeModesEditor::CanRenameItem))
		);
	}
}

bool SDMXFixtureTypeModesEditor::CanCutItems() const
{
	int32 NumSelectedItems = ListView->GetNumItemsSelected();
	return NumSelectedItems > 0;
}

void SDMXFixtureTypeModesEditor::OnCutSelectedItems()
{
	int32 NumSelectedItems = ListView->GetNumItemsSelected();
	const FText TransactionText = FText::Format(LOCTEXT("CutModeTransaction", "Cut Fixture {0}|plural(one=Mode, other=Modes)"), GetListSelectionAsModeIndices().Num());
	const FScopedTransaction CutModeTransaction(TransactionText);

	OnCopySelectedItems();
	OnDeleteItems();
}

bool SDMXFixtureTypeModesEditor::CanCopyItems() const
{
	return FixtureTypeSharedData->CanCopyModesToClipboard();
}

void SDMXFixtureTypeModesEditor::OnCopySelectedItems()
{
	FixtureTypeSharedData->CopyModesToClipboard();
}

bool SDMXFixtureTypeModesEditor::CanPasteItems() const
{
	return FixtureTypeSharedData->CanPasteModesFromClipboard();
}	

void SDMXFixtureTypeModesEditor::OnPasteItems()
{
	TArray<int32> NewlyAddedModeIndices;
	FixtureTypeSharedData->PasteModesFromClipboard(NewlyAddedModeIndices);

	FixtureTypeSharedData->SelectModes(NewlyAddedModeIndices);
}

bool SDMXFixtureTypeModesEditor::CanDuplicateItems() const
{
	int32 NumSelectedItems = ListView->GetNumItemsSelected();
	return NumSelectedItems > 0;
}

void SDMXFixtureTypeModesEditor::OnDuplicateItems()
{
	const TArray<TWeakObjectPtr<UDMXEntityFixtureType>> SelectedFixtureTypes = FixtureTypeSharedData->GetSelectedFixtureTypes();
	if (ensureMsgf(SelectedFixtureTypes.Num() == 1, TEXT("Trying to Duplicate Modes, but many or no Fixture Type is selected. This should not be possible.")))
	{
		if (UDMXEntityFixtureType* FixtureType = SelectedFixtureTypes[0].Get())
		{
			const FText TransactionText = FText::Format(LOCTEXT("DuplicateModesTransaction", "Duplicate Fixture {0}|plural(one=Mode, other=Modes)"), GetListSelectionAsModeIndices().Num());
			const FScopedTransaction DuplicateModesTransaction(TransactionText);

			FixtureType->PreEditChange(UDMXEntityFixtureType::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UDMXEntityFixtureType, Modes)));

			TArray<int32> NewlyAddedModeIndices;
			FixtureType->DuplicateModes(GetListSelectionAsModeIndices(), NewlyAddedModeIndices);

			FixtureType->PostEditChange();

			// Select the newly added Modes
			FixtureTypeSharedData->SelectModes(NewlyAddedModeIndices);
		}
	}
}

bool SDMXFixtureTypeModesEditor::CanDeleteItems() const
{
	int32 NumSelectedItems = ListView->GetNumItemsSelected();
	return NumSelectedItems > 0;
}

void SDMXFixtureTypeModesEditor::OnDeleteItems()
{
	const TArray<TWeakObjectPtr<UDMXEntityFixtureType>> SelectedFixtureTypes = FixtureTypeSharedData->GetSelectedFixtureTypes();
	if (ensureMsgf(SelectedFixtureTypes.Num() == 1, TEXT("Trying to Delete Modes, but many or no Fixture Type is selected. This should not be possible.")))
	{
		if (UDMXEntityFixtureType* FixtureType = SelectedFixtureTypes[0].Get())
		{
			const FText TransactionText = FText::Format(LOCTEXT("DeleteModesTransaction", "Delete Fixture {0}|plural(one=Mode, other=Modes)"), GetListSelectionAsModeIndices().Num());
			const FScopedTransaction DeleteModesTransaction(TransactionText);

			FixtureType->PreEditChange(UDMXEntityFixtureType::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UDMXEntityFixtureType, Modes)));

			const TArray<int32>& SelectedModeIndices = GetListSelectionAsModeIndices();
			FixtureType->RemoveModes(SelectedModeIndices);

			FixtureType->PostEditChange();

			// Select something reasonable around the deleted items
			if (FixtureType->Modes.Num() == 0)
			{
				FixtureTypeSharedData->SelectModes(TArray<int32>());
			}
			else if (SelectedModeIndices.Num() > 0)
			{
				const int32 DesiredSelection = SelectedModeIndices.Last() - SelectedModeIndices.Num() + 1;
				if (FixtureType->Modes.IsValidIndex(DesiredSelection))
				{
					FixtureTypeSharedData->SelectModes(TArray<int32>({ DesiredSelection }));
				}
				else
				{
					FixtureTypeSharedData->SelectModes(TArray<int32>({ FixtureType->Modes.Num() - 1 }));
				}
			}
		}
	}
}

bool SDMXFixtureTypeModesEditor::CanRenameItem() const
{
	int32 NumSelectedItems = ListView->GetNumItemsSelected();
	return NumSelectedItems == 1;
}

void SDMXFixtureTypeModesEditor::OnRenameItem()
{
	TArray<TSharedPtr<FDMXFixtureTypeModesEditorModeItem>> SelectedModeItems;
	ListView->GetSelectedItems(SelectedModeItems);
	
	if (SelectedModeItems.Num() == 1)
	{
		const TSharedPtr<FDMXFixtureTypeModesEditorModeItem>& SelectedItem = SelectedModeItems[0];
		const TSharedPtr<SDMXFixtureTypeModesEditorModeRow>* SelectedRowPtr = ModeRows.FindByPredicate([SelectedItem](const TSharedPtr<SDMXFixtureTypeModesEditorModeRow>& ModeRow)
			{
				return ModeRow->GetModeItem() == SelectedItem;
			});

		if (SelectedRowPtr)
		{
			(*SelectedRowPtr)->EnterModeNameEditingMode();
		}
	}
}

#undef LOCTEXT_NAMESPACE
