// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateTypes.h"
#include "Widgets/SCompoundWidget.h"

class FDMXEditor;
struct FDMXFixtureMode;
class FDMXFixtureTypeSharedData;
class FDMXFixtureTypeModesEditorModeItem;
class FUICommandList;
class ITableRow;
class SBorder;
class SCheckBox;
class SDMXFixtureTypeModesEditorModeRow;
template <typename ItemType> class SListView;
class STableViewBase;
class SVerticalBox;
class UDMXEntityFixtureType;
class UDMXLibrary;

/** The editor for the Fixure Type Modes array */
class SDMXFixtureTypeModesEditor
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMXFixtureTypeModesEditor)
	{}
		
	SLATE_END_ARGS()

	/** Constructs this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<FDMXEditor>& InDMXEditor);

protected:
	//~ Begin SWidget Interface
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget Interface

private:
	/** Rebuilds the List of Modes */
	void RebuildList();

	/** Rebuilds selection from what's selected in Fixture Type Shared Data */
	void RebuildSelection();

	/** Called when a fixture type changed */
	void OnFixtureTypeChanged(const UDMXEntityFixtureType* ChangedFixtureType);

	/** Called when the Search Text changed */
	void OnSearchTextChanged(const FText& SearchText);

	/** Generates a row in the List of Modes */
	TSharedRef<ITableRow> OnGenerateModeRow(TSharedPtr<FDMXFixtureTypeModesEditorModeItem> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Returns true if the item is selected */
	bool IsItemSelectedExclusively(TSharedPtr<FDMXFixtureTypeModesEditorModeItem> InItem) const;

	/** Called when the selelction in the list changed */
	void OnListSelectionChanged(TSharedPtr<FDMXFixtureTypeModesEditorModeItem> NewlySelectedItem, ESelectInfo::Type SelectInfo);

	/** Called when modes were selected in Fixture Type Shared Data */
	void OnFixtureTypeSharedDataSelectedModes();

	/** Called when the show all revisions check box state changed */
	void OnRevisionCheckBoxStateChanged(ECheckBoxState NewCheckState);

	/** Returns the visibility of the Revisions section */
	EVisibility GetRevisionCheckBoxVisibility() const;

	/** Returns the selected Modes in the List as Indices in the Modes array */
	TArray<int32> GetListSelectionAsModeIndices() const;

	/** Called when the context menu of the list is opening */
	TSharedPtr<SWidget> OnListContextMenuOpening();

	/** Cached modes for comparison */
	TArray<FDMXFixtureMode> CachedModes;

	/** Border that holds the list content, or widgets showing reasons why no list can be drawn */
	TSharedPtr<SBorder> ListContentBorder;

	/** The the list of Modes */
	TSharedPtr<SListView<TSharedPtr<FDMXFixtureTypeModesEditorModeItem>>> ListView;

	/** Source array for the Modes List */
	TArray<TSharedPtr<FDMXFixtureTypeModesEditorModeItem>> ListSource;

	/** Rows of Mode widgets in the list */
	TArray<TSharedPtr<SDMXFixtureTypeModesEditorModeRow>> ModeRows;

	/** The revision check box */
	TSharedPtr<SCheckBox> RevisionCheckBox;

	/** True if mode revisions are available */
	bool bHasModeRevisions = false;

	/** Search Text entered in the header row */
	FText SearchText;

	/** Shared data for Fixture Types */
	TSharedPtr<FDMXFixtureTypeSharedData> FixtureTypeSharedData;

	/** The Fixture Type Editor that owns this widget */
	TWeakPtr<FDMXEditor> WeakDMXEditor;
	

private:
	///////////////////////////////////////////////////
	// Context menu Commands related

	/** Registers the commands */
	void RegisterCommands();

	bool CanCutItems() const;
	void OnCutSelectedItems();
	bool CanCopyItems() const;
	void OnCopySelectedItems();
	bool CanPasteItems() const;
	void OnPasteItems();
	bool CanDuplicateItems() const;
	void OnDuplicateItems();
	bool CanDeleteItems() const;
	void OnDeleteItems();
	bool CanRenameItem() const;
	void OnRenameItem();

	TSharedPtr<FUICommandList> CommandList;
};
