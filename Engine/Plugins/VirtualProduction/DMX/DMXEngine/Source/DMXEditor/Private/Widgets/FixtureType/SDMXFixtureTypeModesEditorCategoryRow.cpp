// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXFixtureTypeModesEditorCategoryRow.h"

#include "DMXEditor.h"
#include "DMXFixtureTypeSharedData.h"
#include "Commands/DMXEditorCommands.h"
#include "Library/DMXEntityFixtureType.h"

#include "Styling/AppStyle.h"
#include "ScopedTransaction.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "SDMXFixtureTypeModesEditorCategoryRow"

void SDMXFixtureTypeModesEditorCategoryRow::Construct(const FArguments& InArgs, const TSharedRef<FDMXEditor>& InDMXEditor)
{
	WeakDMXEditor = InDMXEditor;
	SharedData = InDMXEditor->GetFixtureTypeSharedData();

	ChildSlot
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Top)
	[
		SNew(SBorder)
		.Padding(3.0f)
		.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
		.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f))
		[
			SNew(SHorizontalBox)
				
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			[
				// Search box
				SNew(SSearchBox)
				.HintText(LOCTEXT("SearchModesHint", "Search"))
				.OnTextChanged(InArgs._OnSearchTextChanged)
			]
					
			// 'Add Mode' Button
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			.AutoWidth()
			.Padding(10.f, 2.f, 2.f, 2.f)
			[
				SNew(SButton)
				.IsEnabled(this, &SDMXFixtureTypeModesEditorCategoryRow::GetIsAddModeButtonEnabled)
				.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
				.ForegroundColor(FLinearColor::White)
				.ToolTipText(this, &SDMXFixtureTypeModesEditorCategoryRow::GetAddModeButtonTooltipText)
				.ContentPadding(FMargin(5.0f, 1.0f))
				.OnClicked(this, &SDMXFixtureTypeModesEditorCategoryRow::OnAddModeButtonClicked)
				.Content()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(0.f, 1.f))
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Plus"))
					]
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(FMargin(2.f, 0.f, 2.f, 0.f))
					[
						SNew(STextBlock)
						.Text(this, &SDMXFixtureTypeModesEditorCategoryRow::GetAddModeButtonText)
					]
				]
			]
		]
	];
}

bool SDMXFixtureTypeModesEditorCategoryRow::GetIsAddModeButtonEnabled() const
{
	const TArray<TWeakObjectPtr<UDMXEntityFixtureType>>& SelectedFixtureTypes = SharedData->GetSelectedFixtureTypes();

	return SelectedFixtureTypes.Num() == 1;
}

FText SDMXFixtureTypeModesEditorCategoryRow::GetAddModeButtonTooltipText() const
{
	if (GetIsAddModeButtonEnabled())
	{
		return FDMXEditorCommands::Get().AddNewFixtureTypeMode->GetDescription();
	}

	return NSLOCTEXT("SDMXFixtureTypeModesEditorCategoryRow", "AddModeButtonDisabledTooltip", "Please select a single Fixture Type to add a Mode to it.");
}

FReply SDMXFixtureTypeModesEditorCategoryRow::OnAddModeButtonClicked() const
{
	const TArray<TWeakObjectPtr<UDMXEntityFixtureType>>& SelectedFixtureTypes = SharedData->GetSelectedFixtureTypes();
	if (ensureMsgf(SelectedFixtureTypes.Num() == 1, TEXT("Trying to Add Mode, but many or no Fixture Type is selected. This should not be possible.")))
	{
		if (UDMXEntityFixtureType* FixtureType = SelectedFixtureTypes[0].Get())
		{
			const FScopedTransaction AddModeTransaction(LOCTEXT("AddModeTransaction", "Add Fixture Mode"));
			FixtureType->PreEditChange(UDMXEntityFixtureType::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UDMXEntityFixtureType, Modes)));

			const int32 NewModeIndex = FixtureType->AddMode();

			FixtureType->PostEditChange();

			if (NewModeIndex != INDEX_NONE)
			{
				SharedData->SelectModes(TArray<int32>({ NewModeIndex }));
			}

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

FText SDMXFixtureTypeModesEditorCategoryRow::GetAddModeButtonText() const
{
	if (GetIsAddModeButtonEnabled())
	{
		return FDMXEditorCommands::Get().AddNewFixtureTypeMode->GetLabel();
	}

	return NSLOCTEXT("SDMXFixtureTypeModesEditorCategoryRow", "AddModeButtonDisabled", "Please select a single Fixture Type to add a Mode to it.");
}

#undef LOCTEXT_NAMESPACE
