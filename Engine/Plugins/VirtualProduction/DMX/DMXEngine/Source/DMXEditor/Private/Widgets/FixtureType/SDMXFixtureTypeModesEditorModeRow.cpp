// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXFixtureTypeModesEditorModeRow.h"

#include "DMXEditor.h"
#include "DMXEditorUtils.h"
#include "Library/DMXEntityFixtureType.h"
#include "Widgets/FixtureType/DMXFixtureTypeModesEditorModeItem.h"

#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Notifications/SPopUpErrorText.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"


#define LOCTEXT_NAMESPACE "SDMXFixtureTypeModesEditorModeRow"

void SDMXFixtureTypeModesEditorModeRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, TSharedRef<FDMXFixtureTypeModesEditorModeItem> InModeItem)
{
	ModeItem = InModeItem;
	IsSelected = InArgs._IsSelected;

	STableRow<TSharedPtr<FDMXFixtureTypeModesEditorModeItem>>::Construct(typename STableRow<TSharedPtr<FDMXFixtureTypeModesEditorModeItem>>::FArguments(), OwnerTable);

	ChildSlot
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Top)
	[
		SNew(SHorizontalBox)
	
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		.FillWidth(1)
		[
			SAssignNew(InlineEditableTextBlock, SInlineEditableTextBlock)					
			.Text_Lambda([this]()
				{
					return ModeItem->GetModeName();
				})
			.Font(FCoreStyle::GetDefaultFontStyle("Regular", 10))
			.IsReadOnly(false)
			.OnVerifyTextChanged(this, &SDMXFixtureTypeModesEditorModeRow::OnVerifyModeNameChanged)
			.OnTextCommitted(this, &SDMXFixtureTypeModesEditorModeRow::OnModeNameCommitted)
			.IsSelected(IsSelected)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(3.f, 0)
		[
			SAssignNew(PopupErrorText, SPopupErrorText)
		]		
	];

	PopupErrorText->SetError(FText::GetEmpty());
}

void SDMXFixtureTypeModesEditorModeRow::EnterModeNameEditingMode()
{
	InlineEditableTextBlock->EnterEditingMode();
}

bool SDMXFixtureTypeModesEditorModeRow::OnVerifyModeNameChanged(const FText& InNewText, FText& OutErrorMessage)
{
	FText InvalidReason;
	const bool bValidModeName = ModeItem->IsValidModeName(InNewText, InvalidReason);

	if (bValidModeName)
	{
		PopupErrorText->SetError(FText::GetEmpty());
	}
	else
	{
		PopupErrorText->SetError(InvalidReason);
	}

	return true;
}

void SDMXFixtureTypeModesEditorModeRow::OnModeNameCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	if (InNewText.IsEmpty())
	{
		return;
	}

	FText UniqueModeName;
	ModeItem->SetModeName(InNewText, UniqueModeName);

	InlineEditableTextBlock->SetText(UniqueModeName);

	PopupErrorText->SetError(FText::GetEmpty());
}

#undef LOCTEXT_NAMESPACE
