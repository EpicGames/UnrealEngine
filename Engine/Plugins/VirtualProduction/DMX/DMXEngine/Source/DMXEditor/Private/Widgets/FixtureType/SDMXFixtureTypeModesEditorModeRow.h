// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class FDMXEditor;
class FDMXFixtureTypeModesEditorModeItem;
class FDMXFixtureTypeSharedData;
class UDMXEntityFixtureType;

class SInlineEditableTextBlock;
class SPopupErrorText;


/** Mode as a row in a list */
class SDMXFixtureTypeModesEditorModeRow
	: public STableRow<TSharedPtr<FDMXFixtureTypeModesEditorModeItem>>
{
public:
	SLATE_BEGIN_ARGS(SDMXFixtureTypeModesEditorModeRow)
	{}

		/** Callback to check if the row is selected (should be hooked up if a parent widget is handling selection or focus) */
		SLATE_EVENT(FIsSelected, IsSelected)

	SLATE_END_ARGS()

	/** Constructs this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, TSharedRef<FDMXFixtureTypeModesEditorModeItem> InModeItem);

	/** Enters editing the Mode name in its text box */
	void EnterModeNameEditingMode();

	/** Returns the Mode Item */
	const TSharedPtr<FDMXFixtureTypeModesEditorModeItem> GetModeItem() const { return ModeItem; }

private:
	/** Called to verify the Mode Name when the User is interactively changing it */
	bool OnVerifyModeNameChanged(const FText& InNewText, FText& OutErrorMessage);

	/** Called when Mode Name was comitted */
	void OnModeNameCommitted(const FText& InNewText, ETextCommit::Type InTextCommit);

	/** The mode name edit widget */
	TSharedPtr<SInlineEditableTextBlock> InlineEditableTextBlock;

	/** The popup error displayed when an invalid name is set */
	TSharedPtr<SPopupErrorText> PopupErrorText;

	/** The Mode Item this row displays */
	TSharedPtr<FDMXFixtureTypeModesEditorModeItem> ModeItem;

	// Slate arguments
	FIsSelected IsSelected;
};
