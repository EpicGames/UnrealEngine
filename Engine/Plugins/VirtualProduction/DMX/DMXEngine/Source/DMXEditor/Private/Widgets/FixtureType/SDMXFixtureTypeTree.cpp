// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXFixtureTypeTree.h"

#include "Algo/Transform.h"
#include "Commands/DMXEditorCommands.h"
#include "Dialogs/Dialogs.h"
#include "DMXEditor.h"
#include "DMXEditorUtils.h"
#include "DMXFixturePatchSharedData.h"
#include "DMXFixtureTypeSharedData.h"
#include "DMXRuntimeUtils.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/DMXLibrary.h"
#include "Misc/MessageDialog.h"
#include "ScopedTransaction.h"
#include "SDMXFixtureTypeTreeCategoryRow.h"
#include "SDMXFixtureTypeTreeFixtureTypeRow.h"
#include "Widgets/DMXEntityTreeNode.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDMXFixtureTypeTree"

void SDMXFixtureTypeTree::Construct(const FArguments& InArgs)
{
	SDMXEntityTreeViewBase::FArguments BaseArguments =
		SDMXEntityTreeViewBase::FArguments()
		.DMXEditor(InArgs._DMXEditor);

	SDMXEntityTreeViewBase::Construct(BaseArguments);
	
	if (const TSharedPtr<FDMXEditor> PinnedDMXEditor = DMXEditor.Pin())
	{
		CommandList->MapAction(FDMXEditorCommands::Get().AddNewEntityFixtureType,
			FUIAction(FExecuteAction::CreateSP(this, &SDMXFixtureTypeTree::AddNewFixtureType))
		);

		FixtureTypeSharedData = PinnedDMXEditor->GetFixtureTypeSharedData();
		FixtureTypeSharedData->OnFixtureTypesSelected.AddSP(this, &SDMXFixtureTypeTree::OnFixtureTypesSelected);
		PinnedDMXEditor->GetFixturePatchSharedData()->OnFixturePatchSelectionChanged.AddSP(this, &SDMXFixtureTypeTree::OnFixturePatchesSelected);

		// Bind to external changes
		PinnedDMXEditor->GetDMXLibrary()->GetOnEntitiesAdded().AddSP(this, &SDMXFixtureTypeTree::OnEntitiesAdded);
		PinnedDMXEditor->GetDMXLibrary()->GetOnEntitiesRemoved().AddSP(this, &SDMXFixtureTypeTree::OnEntitiesRemoved);
		UDMXEntityFixtureType::GetOnFixtureTypeChanged().AddSP(this, &SDMXFixtureTypeTree::OnFixtureTypeChanged);

		// Make an initial selection
		TArray<UDMXEntityFixtureType*> FixtureTypes = GetDMXLibrary()->GetEntitiesTypeCast<UDMXEntityFixtureType>();
		if (FixtureTypes.Num() > 0)
		{
			TArray<TWeakObjectPtr<UDMXEntityFixtureType>> InitialSelection = { FixtureTypes[0] };
			FixtureTypeSharedData->SelectFixtureTypes(InitialSelection);
		}
	}
}

TSharedPtr<FDMXEntityTreeEntityNode> SDMXFixtureTypeTree::CreateEntityNode(UDMXEntity* Entity)
{
	check(Entity);
	TSharedPtr<FDMXEntityTreeEntityNode> NewNode = MakeShared<FDMXEntityTreeEntityNode>(Entity);
	RefreshFilteredState(NewNode, false);

	// Error status
	FText InvalidReason;
	if (!Entity->IsValidEntity(InvalidReason))
	{
		NewNode->SetErrorStatus(InvalidReason);
	}

	return NewNode;
}

TSharedRef<SWidget> SDMXFixtureTypeTree::GenerateAddNewEntityButton()
{
	FText AddButtonLabel = FDMXEditorCommands::Get().AddNewEntityFixtureType->GetLabel();
	FText AddButtonToolTip = FDMXEditorCommands::Get().AddNewEntityFixtureType->GetDescription();

	return
		SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
			.ForegroundColor(FLinearColor::White)
			.ToolTipText(AddButtonToolTip)
			.ContentPadding(FMargin(5.0f, 1.0f))
			.OnClicked(this, &SDMXFixtureTypeTree::OnAddNewFixtureTypeClicked)
			.Content()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(0, 1))
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Plus"))
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(FMargin(2, 0, 2, 0))
				[
					SNew(STextBlock)
					.Text(AddButtonLabel)
				]
			];
}

void SDMXFixtureTypeTree::RebuildNodes(const TSharedPtr<FDMXEntityTreeRootNode>& InRootNode)
{
	EntityNodeToEntityRowMap.Reset();

	UDMXLibrary* Library = GetDMXLibrary();
	check(IsValid(Library));

	Library->ForEachEntityOfType<UDMXEntityFixtureType>([this](UDMXEntityFixtureType* FixtureType)
		{
			// Create this entity's node
			TSharedPtr<FDMXEntityTreeEntityNode> FixtureTypeNode = CreateEntityNode(FixtureType);

			// For each Entity, we find or create a category node then add the entity as its child
			const FDMXFixtureCategory DMXCategory = FixtureType->DMXCategory;
			const FText DMXCategoryName = FText::FromName(DMXCategory);

			// Get the category if already existent or create it
			constexpr FDMXEntityTreeCategoryNode::ECategoryType CategoryType = FDMXEntityTreeCategoryNode::ECategoryType::DMXCategory;
			TSharedPtr<FDMXEntityTreeNodeBase> CategoryNode = GetOrCreateCategoryNode(CategoryType, DMXCategoryName, FixtureType->DMXCategory);

			CategoryNode->AddChild(FixtureTypeNode);
		}
	);

	InRootNode->SortChildren();

	// Restore Selection
	TArray<TWeakObjectPtr<UDMXEntityFixtureType>> SelectedFixtureTypes = FixtureTypeSharedData->GetSelectedFixtureTypes();
	TArray<UDMXEntity*> SelectedEntities;
	for (TWeakObjectPtr<UDMXEntityFixtureType> WeakFixtureType : SelectedFixtureTypes)
	{
		if (WeakFixtureType.IsValid())
		{
			SelectedEntities.Add(WeakFixtureType.Get());
		}
	}
	SelectItemsByEntities(SelectedEntities);
}

TSharedRef<ITableRow> SDMXFixtureTypeTree::OnGenerateRow(TSharedPtr<FDMXEntityTreeNodeBase> Node, const TSharedRef<STableViewBase>& OwnerTable)
{
	// Create the node of the appropriate type
	if (Node->GetNodeType() == FDMXEntityTreeNodeBase::ENodeType::CategoryNode)
	{
		SetNodeExpansion(Node, true);

		const TSharedPtr<FDMXEntityTreeCategoryNode> CategoryNode = StaticCastSharedPtr<FDMXEntityTreeCategoryNode>(Node);
		constexpr bool bIsRootCategory = true;

		return 
			SNew(SDMXFixtureTypeTreeCategoryRow, OwnerTable, CategoryNode, bIsRootCategory, SharedThis(this))
			.OnFixtureTypeOrderChanged(OnEntityOrderChangedDelegate)
			[
				SNew(STextBlock)
				.Text(Node->GetDisplayNameText())
				.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
			];
	}
	else
	{
		const TSharedPtr<FDMXEntityTreeEntityNode> EntityNode = StaticCastSharedPtr<FDMXEntityTreeEntityNode>(Node);

		TSharedRef<SDMXFixtureTypeTreeFixtureTypeRow> FixtureTypeRow = SNew(SDMXFixtureTypeTreeFixtureTypeRow, EntityNode, OwnerTable, SharedThis(this))
			.OnGetFilterText(this, &SDMXFixtureTypeTree::GetFilterText)
			.OnEntityDragged(this, &SDMXFixtureTypeTree::OnEntitiesDragged)
			.OnFixtureTypeOrderChanged(OnEntityOrderChangedDelegate);

		EntityNodeToEntityRowMap.Add(EntityNode.ToSharedRef(), FixtureTypeRow);

		return FixtureTypeRow;
	}
}

TSharedPtr<SWidget> SDMXFixtureTypeTree::OnContextMenuOpen()
{
	const bool bCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bCloseWindowAfterMenuSelection, CommandList);

	if (GetSelectedEntities().Num() > 0)
	{
		MenuBuilder.BeginSection("BasicOperations");
		{
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
		}
		MenuBuilder.EndSection();
	}
	else
	{
		MenuBuilder.BeginSection("BasicOperations");
		{
			MenuBuilder.AddMenuEntry(FDMXEditorCommands::Get().AddNewEntityFixtureType);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);
		}
		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
}

void SDMXFixtureTypeTree::OnSelectionChanged(TSharedPtr<FDMXEntityTreeNodeBase> InSelectedNodePtr, ESelectInfo::Type SelectInfo)
{
	if (!bChangingSelection)
	{
		TGuardValue<bool> RecursionGuard(bChangingSelection, true);

		const TArray<UDMXEntity*> NewSelection = GetSelectedEntities();

		// Never clear the selection
		if (GetSelectedEntities().Num() == 0)
		{
			TArray<TWeakObjectPtr<UDMXEntityFixtureType>> OldSelection = FixtureTypeSharedData->GetSelectedFixtureTypes();
			TArray<UDMXEntity*> OldSelectionAsEntities;
			for (TWeakObjectPtr<UDMXEntityFixtureType> SelectedFixtureType : OldSelection)
			{
				if (UDMXEntityFixtureType* SelectedEntity = SelectedFixtureType.Get())
				{
					OldSelectionAsEntities.Add(SelectedEntity);
				}
			}

			if (OldSelectionAsEntities.Num() > 0)
			{
				SelectItemsByEntities(OldSelectionAsEntities);
			}
		}
		else
		{
			// Select selected Fixture Types in Fixture Type Shared Data
			TArray<TWeakObjectPtr<UDMXEntityFixtureType>> SelectedFixtureTypes;

			for (UDMXEntity* Entity : NewSelection)
			{
				if (UDMXEntityFixtureType* FixtureType = Cast<UDMXEntityFixtureType>(Entity))
				{
					SelectedFixtureTypes.Add(FixtureType);
				}
			}

			FixtureTypeSharedData->SelectFixtureTypes(SelectedFixtureTypes);
		}
	}
}

void SDMXFixtureTypeTree::OnCutSelectedNodes()
{
	TArray<UDMXEntity*>&& SelectedNodes = GetSelectedEntities();
	const FScopedTransaction Transaction(SelectedNodes.Num() > 1 ? LOCTEXT("CutFixtureTypes", "Cut Fixture Types") : LOCTEXT("CutFixtureType", "Cut Fixture Type"));

	OnCopySelectedNodes();
	OnDeleteNodes();
}

bool SDMXFixtureTypeTree::CanCutNodes() const
{
	return CanCopyNodes() && CanDeleteNodes();
}

void SDMXFixtureTypeTree::OnCopySelectedNodes()
{
	TArray<UDMXEntity*>&& EntitiesToCopy = GetSelectedEntities();

	// Copy the entities to the clipboard
	FDMXEditorUtils::CopyEntities(MoveTemp(EntitiesToCopy));
}

bool SDMXFixtureTypeTree::CanCopyNodes() const
{
	TArray<UDMXEntity*>&& EntitiesToCopy = GetSelectedEntities();
	return EntitiesToCopy.Num() > 0;
}

void SDMXFixtureTypeTree::OnPasteNodes()
{
	// Get the library that's being edited
	UDMXLibrary* Library = GetDMXLibrary();
	check(IsValid(Library));

	// Get the Entities to paste from the clipboard
	TArray<UDMXEntity*> NewObjects = FDMXEditorUtils::CreateEntitiesFromClipboard(Library);

	if (NewObjects.Num() != 0)
	{
		// Start transaction for Undo and take a snapshot of the current Library state
		const FScopedTransaction PasteEntities(NewObjects.Num() > 1 ? LOCTEXT("PasteFixtureTypes", "Paste Fixture Types") : LOCTEXT("PasteFixtureType", "Paste Fixture Type"));
		Library->PreEditChange(UDMXLibrary::StaticClass()->FindPropertyByName(UDMXLibrary::GetEntitiesPropertyName()));

		// Add each pasted Entity to the Library
		TArray<TWeakObjectPtr<UDMXEntityFixtureType>> NewFixtureTypes;
		for (UDMXEntity* NewEntity : NewObjects)
		{
			check(NewEntity);

			// Move the Entity from the transient package into the Library package
			NewEntity->Rename(*MakeUniqueObjectName(Library, NewEntity->GetClass()).ToString(), Library, REN_DoNotDirty | REN_DontCreateRedirectors);

			// Make sure the Entity's name won't collide with existing ones
			NewEntity->SetName(FDMXRuntimeUtils::FindUniqueEntityName(Library, NewEntity->GetClass(), NewEntity->GetDisplayName()));

			// Update the library and ID
			NewEntity->SetParentLibrary(Library);
			NewEntity->RefreshID();

			NewFixtureTypes.Add(CastChecked<UDMXEntityFixtureType>(NewEntity));
		}

		Library->PostEditChange();

		FixtureTypeSharedData->SelectFixtureTypes(NewFixtureTypes);

		UpdateTree();
	}
}

bool SDMXFixtureTypeTree::CanPasteNodes() const
{
	UDMXLibrary* Library = GetDMXLibrary();
	check(IsValid(Library));

	return FDMXEditorUtils::CanPasteEntities(Library);
}

void SDMXFixtureTypeTree::OnDuplicateNodes()
{
	UDMXLibrary* Library = GetDMXLibrary();
	check(IsValid(Library));

	TArray<UDMXEntity*> SelectedEntities = GetSelectedEntities();

	if (Library && SelectedEntities.Num() > 0)
	{
		const FScopedTransaction Transaction(SelectedEntities.Num() > 1 ? LOCTEXT("DuplicateFixtureTypes", "Duplicate Fixture Types") : LOCTEXT("DuplicateFixtureType", "Duplicate Fixture Type"));
		Library->PreEditChange(UDMXLibrary::StaticClass()->FindPropertyByName(UDMXLibrary::GetEntitiesPropertyName()));

		TArray<TWeakObjectPtr<UDMXEntityFixtureType>> NewFixtureTypes;
		NewFixtureTypes.Reserve(SelectedEntities.Num());

		// We'll have the duplicates be placed right after their original counterparts
		int32 NewEntityIndex = Library->FindEntityIndex(SelectedEntities.Last(0));
		for (UDMXEntity* Entity : SelectedEntities)
		{
			FObjectDuplicationParameters DuplicationParams(Entity, GetDMXLibrary());
			
			if (UDMXEntityFixtureType* EntityCopy = CastChecked<UDMXEntityFixtureType>(StaticDuplicateObjectEx(DuplicationParams)))
			{
				EntityCopy->SetName(FDMXRuntimeUtils::FindUniqueEntityName(Library, EntityCopy->GetClass(), EntityCopy->GetDisplayName()));
				NewFixtureTypes.Add(EntityCopy);
				
				Library->SetEntityIndex(EntityCopy, ++NewEntityIndex);
			}
		}

		Library->PostEditChange();

		FixtureTypeSharedData->SelectFixtureTypes(NewFixtureTypes);

		UpdateTree();
	}
}

bool SDMXFixtureTypeTree::CanDuplicateNodes() const
{
	return CanCopyNodes();
}

void SDMXFixtureTypeTree::OnDeleteNodes()
{
	// Gather Fixture Types to remove
	TArray<UDMXEntity*> EntitiesToDelete = GetSelectedEntities();

	TArray<UDMXEntityFixtureType*> FixtureTypesToDelete;
	for (UDMXEntity* Entity : EntitiesToDelete)
	{
		if (UDMXEntityFixtureType* FixtureType = Cast<UDMXEntityFixtureType>(Entity))
		{
			FixtureTypesToDelete.Add(FixtureType);
		}
	}

	if (FixtureTypesToDelete.Num() == 0)
	{
		return;
	}

	// Find Fixture Types that are used by Fixture Patches
	UDMXLibrary* DMXLibrary = GetDMXLibrary();
	TArray<UDMXEntity*> EntitiesInUse;
	for (UDMXEntity* Entity : EntitiesToDelete)
	{
		if (FDMXEditorUtils::IsEntityUsed(DMXLibrary, Entity))
		{
			EntitiesInUse.Add(Entity);
		}
	}

	// Confirm deletion of Fixture Types in use
	const bool bConfirmDelete = [&EntitiesInUse]()
	{
		if (EntitiesInUse.Num() > 0)
		{
			const FText ConfirmDeleteText =
				FText::Format(LOCTEXT("ConfirmDeleteEntityInUse", "Fixture Patches use {0}|plural(one = this, other = these) Fixture {0}|plural(one = Type, other = Types). Do you really want to delete {0}|plural(one = it, other = them)?"),
					EntitiesInUse.Num() == 1);

			const FText DialogTitleText = LOCTEXT("DeleteFixtureTypesDialogTitle", "Delete Fixture Types");

			EAppReturnType::Type DialogResult = FMessageDialog::Open(EAppMsgType::YesNo, EAppReturnType::No, ConfirmDeleteText, DialogTitleText);
			if (DialogResult == EAppReturnType::No)
			{
				return false;
			}
		}

		return true;
	}();

	if (bConfirmDelete)
	{
		// Find a new selection
		const TSharedPtr<FDMXEntityTreeEntityNode> EntityNodeToSelect = [&FixtureTypesToDelete, this]() -> TSharedPtr<FDMXEntityTreeEntityNode>
		{
			const TArray<TSharedPtr<FDMXEntityTreeEntityNode>> EntityNodes = GetEntityNodes();

			for (const TSharedPtr<FDMXEntityTreeEntityNode>& EntityNode : EntityNodes)
			{
				if (EntityNode.IsValid() && !FixtureTypesToDelete.Contains(EntityNode->GetEntity()))
				{
					return EntityNode;
				}
			}
			return nullptr;
		}();

		// Apply the new selection
		if (EntityNodeToSelect.IsValid())
		{
			if (UDMXEntityFixtureType* FixtureTypeToSelect = Cast<UDMXEntityFixtureType>(EntityNodeToSelect->GetEntity()))
			{
				SelectItemByNode(EntityNodeToSelect.ToSharedRef());

				const TArray<TWeakObjectPtr<UDMXEntityFixtureType>> FixtureTypesToSelect = { FixtureTypeToSelect };
				FixtureTypeSharedData->SelectFixtureTypes(FixtureTypesToSelect);
			}
		}
		else
		{		
			// Clear selection if no fixture types remain
			FixtureTypeSharedData->SelectFixtureTypes(TArray<TWeakObjectPtr<UDMXEntityFixtureType>>());
		}

		const FScopedTransaction Transaction(EntitiesToDelete.Num() > 1 ? LOCTEXT("RemoveEntities", "Remove Entities") : LOCTEXT("RemoveEntity", "Remove Entity"));

		// Modify affected fixture patches
		const TArray<UDMXEntityFixturePatch*> FixturePatches = DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>();
		TArray<UDMXEntityFixturePatch*> AffectedFixturePatches;
		Algo::TransformIf(FixturePatches, AffectedFixturePatches,
			[&FixtureTypesToDelete](const UDMXEntityFixturePatch* FixturePatch)
			{
				return FixturePatch && FixtureTypesToDelete.Contains(FixturePatch->GetFixtureType());
			},
			[&FixtureTypesToDelete](UDMXEntityFixturePatch* FixturePatch)
			{
				return FixturePatch;
			});
		for (UDMXEntityFixturePatch* FixturePatch : AffectedFixturePatches)
		{
			FixturePatch->Modify();
		}

		// Remove the Fixture Types from the DMX 
		DMXLibrary->PreEditChange(UDMXLibrary::StaticClass()->FindPropertyByName(UDMXLibrary::GetEntitiesPropertyName()));
		for (UDMXEntityFixtureType* FixtureType : FixtureTypesToDelete)
		{
			ensureMsgf(DMXLibrary == FixtureType->GetParentLibrary(), TEXT("Unexpected DMX Library of Fixture Type and DMX Library of Editor do not match when removing Fixture Types."));
			const FDMXEntityFixtureTypeRef FixtureTypeRef(FixtureType);
			UDMXEntityFixtureType::RemoveFixtureTypeFromLibrary(FixtureTypeRef);
		}
		DMXLibrary->PostEditChange();

		UpdateTree();
	}
}

bool SDMXFixtureTypeTree::CanDeleteNodes() const
{
	return GetSelectedEntities().Num() > 0;
}

void SDMXFixtureTypeTree::OnRenameNode()
{
	TArray<TSharedPtr<FDMXEntityTreeNodeBase>> SelectedItems = GetSelectedNodes();

	if (SelectedItems.Num() == 1 && SelectedItems[0].IsValid() && SelectedItems[0]->GetNodeType() == FDMXEntityTreeNodeBase::ENodeType::EntityNode)
	{
		RequestScrollIntoView(SelectedItems[0].ToSharedRef());

		const TSharedPtr<FDMXEntityTreeEntityNode> EntityNode = StaticCastSharedPtr<FDMXEntityTreeEntityNode>(SelectedItems[0]);
		const TSharedPtr<SDMXFixtureTypeTreeFixtureTypeRow> FixtureTypeRow = FindEntityRowByNode(EntityNode.ToSharedRef());

		if (FixtureTypeRow.IsValid())
		{
			FixtureTypeRow->EnterRenameMode();
		}
	}
}

bool SDMXFixtureTypeTree::CanRenameNode() const
{
	TArray<TSharedPtr<FDMXEntityTreeNodeBase>> SelectedItems = GetSelectedNodes();

	return 
		SelectedItems.Num() == 1 && 
		SelectedItems[0].IsValid() &&
		SelectedItems[0]->GetNodeType() == FDMXEntityTreeNodeBase::ENodeType::EntityNode;
}

TSharedPtr<SDMXFixtureTypeTreeFixtureTypeRow> SDMXFixtureTypeTree::FindEntityRowByNode(const TSharedRef<FDMXEntityTreeEntityNode>& EntityNode)
{
	if (const TSharedRef<SDMXFixtureTypeTreeFixtureTypeRow>* RowPtr = EntityNodeToEntityRowMap.Find(EntityNode))
	{
		return *RowPtr;
	}

	return nullptr;
}	

void SDMXFixtureTypeTree::OnFixtureTypesSelected()
{
	if (!bChangingSelection)
	{
		TArray<TWeakObjectPtr<UDMXEntityFixtureType>> SelectedFixtureTypes = FixtureTypeSharedData->GetSelectedFixtureTypes();

		TArray<UDMXEntity*> NewSelection;
		for (TWeakObjectPtr<UDMXEntityFixtureType> WeakFixtureType : SelectedFixtureTypes)
		{
			if (UDMXEntityFixtureType* FixtureType = WeakFixtureType.Get())
			{
				NewSelection.Add(FixtureType);
			}
		}

		SelectItemsByEntities(NewSelection);
	}
}

void SDMXFixtureTypeTree::OnFixturePatchesSelected()
{
	if (!bChangingSelection)
	{
		if (TSharedPtr<FDMXEditor> PinnedDMXEditor = DMXEditor.Pin())
		{
			TArray<TWeakObjectPtr<UDMXEntityFixturePatch>> SelectedFixturePatches = PinnedDMXEditor->GetFixturePatchSharedData()->GetSelectedFixturePatches();

			// Select the corresponding Fixture Type
			if (SelectedFixturePatches.Num() == 1)
			{
				if (UDMXEntityFixturePatch* FixturePatch = SelectedFixturePatches[0].Get())
				{
					if (UDMXEntityFixtureType* FixtureType = FixturePatch->GetFixtureType())
					{
						SelectItemsByEntities(TArray<UDMXEntity*>({ FixtureType }));
					}
				}
			}
		}
	}
}

void SDMXFixtureTypeTree::OnEntitiesAdded(UDMXLibrary* Library, TArray<UDMXEntity*> Entities)
{
	if (TSharedPtr<FDMXEditor> PinnedEditor = DMXEditor.Pin())
	{
		Entities.RemoveAll([](UDMXEntity* Entity)
			{
				return 
					!IsValid(Entity) ||
					Entity->GetClass() != UDMXEntityFixtureType::StaticClass();
			});

		if (PinnedEditor->GetDMXLibrary() == Library && Entities.Num() > 0)
		{
			FixtureTypeSharedData->SelectModes(TArray<int32>());
			constexpr bool bMatrixSelected = false;
			FixtureTypeSharedData->SetFunctionAndMatrixSelection(TArray<int32>(), bMatrixSelected);

			UpdateTree();
		}
	}
}

void SDMXFixtureTypeTree::OnEntitiesRemoved(UDMXLibrary* Library, TArray<UDMXEntity*> Entities)
{
	if (TSharedPtr<FDMXEditor> PinnedEditor = DMXEditor.Pin())
	{
		Entities.RemoveAll([](UDMXEntity* Entity)
			{
				return
					!IsValid(Entity) ||
					Entity->GetClass() != UDMXEntityFixtureType::StaticClass();
			});

		if (PinnedEditor->GetDMXLibrary() == Library && Entities.Num() > 0)
		{
			UpdateTree();
		}
	}
}

void SDMXFixtureTypeTree::OnFixtureTypeChanged(const UDMXEntityFixtureType* FixtureType)
{
	if (TSharedPtr<FDMXEditor> PinnedEditor = DMXEditor.Pin())
	{
		if (FixtureType && PinnedEditor->GetDMXLibrary() == FixtureType->GetParentLibrary())
		{
			TArray<UDMXEntity*> SelectedEntities = GetSelectedEntities();
			UpdateTree();
			SelectItemsByEntities(SelectedEntities);
		}
	}
}

FReply SDMXFixtureTypeTree::OnAddNewFixtureTypeClicked()
{
	AddNewFixtureType();
	return FReply::Handled();
}

void SDMXFixtureTypeTree::AddNewFixtureType()
{
	if (TSharedPtr<FDMXEditor> PinnedEditor = DMXEditor.Pin())
	{
		const FScopedTransaction Transaction(LOCTEXT("CreateFixtureTypeTransaction", "Create DMX Fixture Type"));

		FDMXEntityFixtureTypeConstructionParams FixtureTypeConstructionParams;
		FixtureTypeConstructionParams.ParentDMXLibrary = PinnedEditor->GetDMXLibrary();

		UDMXEntityFixtureType* NewFixtureType = UDMXEntityFixtureType::CreateFixtureTypeInLibrary(FixtureTypeConstructionParams);

		FixtureTypeSharedData->SelectFixtureTypes(TArray<TWeakObjectPtr<UDMXEntityFixtureType>>({ NewFixtureType }));

		UpdateTree();
	}
}

#undef LOCTEXT_NAMESPACE
