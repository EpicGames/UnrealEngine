// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SDMXEntityTreeViewBase.h"

class FDMXEntityTreeEntityNode;
class FDMXFixtureTypeSharedData;
class SDMXFixtureTypeTreeFixtureTypeRow;
class UDMXEntity;
class UDMXEntityFixtureType;
class UDMXLibrary;


/**
 * A tree of Fixture Types in a DMX Library.
 */
class SDMXFixtureTypeTree
	: public SDMXEntityTreeViewBase
{
public:
	SLATE_BEGIN_ARGS(SDMXFixtureTypeTree)
	{}
		/** The DMX Editor that owns this widget */
		SLATE_ARGUMENT(TWeakPtr<FDMXEditor>, DMXEditor)

	SLATE_END_ARGS()

	/** Constructs the widget */
	void Construct(const FArguments& InArgs);

	/** Destructor */
	virtual ~SDMXFixtureTypeTree() {}

	/** Creates a Node for the entity */
	TSharedPtr<FDMXEntityTreeEntityNode> CreateEntityNode(UDMXEntity* Entity);

protected:
	//~ Begin SDMXEntityTreeViewBase interface
	virtual TSharedRef<SWidget> GenerateAddNewEntityButton();
	virtual void RebuildNodes(const TSharedPtr<FDMXEntityTreeRootNode>& InRootNode);
	virtual TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FDMXEntityTreeNodeBase> Node, const TSharedRef<STableViewBase>& OwnerTable);
	virtual TSharedPtr<SWidget> OnContextMenuOpen();
	virtual void OnSelectionChanged(TSharedPtr<FDMXEntityTreeNodeBase> InSelectedNodePtr, ESelectInfo::Type SelectInfo);
	virtual void OnCutSelectedNodes();
	virtual bool CanCutNodes() const;
	virtual void OnCopySelectedNodes();
	virtual bool CanCopyNodes() const;
	virtual void OnPasteNodes();
	virtual bool CanPasteNodes() const;
	virtual bool CanDuplicateNodes() const;
	virtual void OnDuplicateNodes();
	virtual void OnDeleteNodes();
	virtual bool CanDeleteNodes() const;
	virtual void OnRenameNode();
	virtual bool CanRenameNode() const;
	//~ End SDMXEntityTreeViewBase interface

private:
	/** Called when Fixture Types were selected in Fixture Type Shared Data */
	void OnFixtureTypesSelected();

	/** Called when Fixture Patches were selected in Fixture Patch Shared Data */
	void OnFixturePatchesSelected();

	/** Called when an Entity was added from the edited DMX Library */
	void OnEntitiesAdded(UDMXLibrary* Library, TArray<UDMXEntity*> Entities);

	/** Called when an Entity was removed from the edited DMX Library */
	void OnEntitiesRemoved(UDMXLibrary* Library, TArray<UDMXEntity*> Entities);

	/** Called when a Fixture Type changed */
	void OnFixtureTypeChanged(const UDMXEntityFixtureType* FixtureType);

	/** Called when the Add button was clicked */
	FReply OnAddNewFixtureTypeClicked();

	/** Adds a new Fixture Type to the Library */
	void AddNewFixtureType();

	/** Returns the row that corresponds to the node */
	TSharedPtr<SDMXFixtureTypeTreeFixtureTypeRow> FindEntityRowByNode(const TSharedRef<FDMXEntityTreeEntityNode>& EntityNode);

	/** True while the widget is changing the selection */
	bool bChangingSelection = false;

	/** Map of entity nodes and their row in the tree */
	TMap<TSharedRef<FDMXEntityTreeEntityNode>, TSharedRef<SDMXFixtureTypeTreeFixtureTypeRow>> EntityNodeToEntityRowMap;

	/** Shared Data for Fixture Types */
	TSharedPtr<FDMXFixtureTypeSharedData> FixtureTypeSharedData;
};
