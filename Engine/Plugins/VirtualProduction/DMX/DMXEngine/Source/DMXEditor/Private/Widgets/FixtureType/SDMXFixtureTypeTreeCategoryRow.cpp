// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXFixtureTypeTreeCategoryRow.h"

#include "DMXEditorUtils.h"
#include "SDMXFixtureTypeTree.h"
#include "DragDrop/DMXEntityDragDropOp.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/DMXLibrary.h"
#include "Widgets/DMXEntityTreeNode.h"

#include "Styling/AppStyle.h"
#include "ScopedTransaction.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Views/SExpanderArrow.h"


#define LOCTEXT_NAMESPACE "SDMXFixtureTypeTreeCategoryRow"

void SDMXFixtureTypeTreeCategoryRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedPtr<FDMXEntityTreeCategoryNode> InCategoryNode, bool bInIsRootCategory, TWeakPtr<SDMXFixtureTypeTree> InFixtureTypeTree)
{
	check(InCategoryNode.IsValid());

	WeakFixtureTypeTree = InFixtureTypeTree;
	WeakCategoryNode = InCategoryNode;
	
	OnFixtureTypeOrderChanged = InArgs._OnFixtureTypeOrderChanged;

	// background color tint
	const FLinearColor BackgroundTint(0.6f, 0.6f, 0.6f, bInIsRootCategory ? 1.0f : 0.3f);

	// rebuilds the whole table row from scratch
	ChildSlot
	.Padding(0.0f, 2.0f, 0.0f, 0.0f)
	[
		SAssignNew(ContentBorder, SBorder)
		.BorderImage(this, &SDMXFixtureTypeTreeCategoryRow::GetBackgroundImageBrush)
		.Padding(FMargin(0.0f, 3.0f))
		.BorderBackgroundColor(BackgroundTint)
		.ToolTipText(InCategoryNode->GetToolTip())
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 2.0f, 2.0f, 2.0f)
			.AutoWidth()
			[
				SNew(SExpanderArrow, STableRow<TSharedPtr<FDMXEntityTreeCategoryNode>>::SharedThis(this))
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				InArgs._Content.Widget
			]
		]
	];

	STableRow<TSharedPtr<FDMXEntityTreeCategoryNode>>::ConstructInternal(
		STableRow<TSharedPtr<FDMXEntityTreeCategoryNode>>::FArguments()
			.Style(FAppStyle::Get(), "DetailsView.TreeView.TableRow")
			.ShowSelection(false),
		InOwnerTableView
	);
}

void SDMXFixtureTypeTreeCategoryRow::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (TSharedPtr<FDMXEntityDragDropOperation> EntityDragDropOp = DragDropEvent.GetOperationAs<FDMXEntityDragDropOperation>())
	{
		TestCanDropWithFeedback(EntityDragDropOp.ToSharedRef());
	}
}

FReply SDMXFixtureTypeTreeCategoryRow::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	const TSharedPtr<FDMXEntityDragDropOperation> EntityDragDropOp = DragDropEvent.GetOperationAs<FDMXEntityDragDropOperation>();
	const TSharedPtr<SDMXFixtureTypeTree> FixtureTypeTree = WeakFixtureTypeTree.Pin();
	const TSharedPtr<FDMXEntityTreeCategoryNode> CategoryNode = WeakCategoryNode.Pin();
	UDMXLibrary* DMXLibrary = GetDMXLibrary();

	if (EntityDragDropOp.IsValid() && CategoryNode.IsValid() && DMXLibrary)
	{
		if (TestCanDropWithFeedback(EntityDragDropOp.ToSharedRef()))
		{
			const TArray<TWeakObjectPtr<UDMXEntity>> DraggedEntities = EntityDragDropOp->GetDraggedEntities();

			// Register transaction and current DMX library state for Undo
			const FScopedTransaction ChangeCategoryTransaction = FScopedTransaction(
				FText::Format(LOCTEXT("ChangeEntitiesCategory", "Change {0}|plural(one=Entity, other=Entities) category"), DraggedEntities.Num())
			);

			DMXLibrary->Modify();

			const FDMXFixtureCategory& FixtureCategory = CategoryNode->GetCategoryValue();
			for (TWeakObjectPtr<UDMXEntity> Entity : DraggedEntities)
			{
				if (UDMXEntityFixtureType* FixtureType = Cast<UDMXEntityFixtureType>(Entity))
				{
					FixtureType->Modify();

					FixtureType->PreEditChange(nullptr);
					FixtureType->DMXCategory = FixtureCategory;
					FixtureType->PostEditChange();
				}
			}

			if (CategoryNode->GetChildren().Num() > 0)
			{
				if (TSharedPtr<FDMXEntityTreeEntityNode> EntityNode = StaticCastSharedPtr<FDMXEntityTreeEntityNode>(CategoryNode->GetChildren().Last(0)))
				{
					// Index after last entity in hovered category
					UDMXEntity* LastEntityInCategory = EntityNode->GetEntity();
					const int32 LastEntityIndex = DMXLibrary->FindEntityIndex(LastEntityInCategory);
					check(LastEntityIndex != INDEX_NONE);

					// Move dragged entities after the last ones in the category
					// Reverse for to keep dragged entities order.
					int32 DesiredIndex = LastEntityIndex + 1;
					for (int32 EntityIndex = DraggedEntities.Num() - 1; EntityIndex > -1; --EntityIndex)
					{
						if (UDMXEntity* Entity = DraggedEntities[EntityIndex].Get())
						{
							DMXLibrary->SetEntityIndex(Entity, DesiredIndex);
						}
					}
				}
			}

			FixtureTypeTree->UpdateTree();

			OnFixtureTypeOrderChanged.ExecuteIfBound();

			return FReply::Handled().EndDragDrop();
		}
	}

	return FReply::Unhandled();
}

bool SDMXFixtureTypeTreeCategoryRow::TestCanDropWithFeedback(const TSharedRef<FDMXEntityDragDropOperation>& EntityDragDropOp) const
{
	if (const TSharedPtr<FDMXEntityTreeCategoryNode> CategoryNode = WeakCategoryNode.Pin())
	{
		// Only handle entity drag drop ops that contain fixture types only 
		const bool bOnlyFixturePatchesAreDragged = [EntityDragDropOp]()
		{
			for (UClass* Class : EntityDragDropOp->GetDraggedEntityTypes())
			{
				if (Class != UDMXEntityFixtureType::StaticClass())
				{
					return false;
				}
			}
			return true;
		}();

		// Only handle entity drag drop ops that were dragged from and to the same dmx library
		const bool bRowAndEntitiesAreOfSameLibrary = [EntityDragDropOp, this]()
		{
			UDMXLibrary* DMXLibrary = GetDMXLibrary();
			for (TWeakObjectPtr<UDMXEntity> Entity : EntityDragDropOp->GetDraggedEntities())
			{
				if (Entity.IsValid() && Entity->GetParentLibrary() != DMXLibrary)
				{
					return false;
				}
			}
			return true;
		}();

		if (bOnlyFixturePatchesAreDragged && bRowAndEntitiesAreOfSameLibrary)
		{
			if (CategoryNode->CanDropOntoCategory())
			{
				const TArray<TWeakObjectPtr<UDMXEntity>> DraggedEntities = EntityDragDropOp->GetDraggedEntities();
				const TArray<TSharedRef<FDMXEntityTreeCategoryNode>> DraggedFromCategories = GetCategoryNodesFromDragDropOp(EntityDragDropOp);

				if (DraggedFromCategories.Num() == 1 && DraggedFromCategories[0] == CategoryNode)
				{
					// There wouldn't be any change by dragging the items into their own category.
					EntityDragDropOp->SetFeedbackMessageError(FText::Format(
						LOCTEXT("DragIntoSelfCategory", "The selected {0} {1}|plural(one=is, other=are) already in this category"),
						FDMXEditorUtils::GetEntityTypeNameText(UDMXEntityFixtureType::StaticClass(), DraggedEntities.Num() > 1),
						DraggedEntities.Num()
					));

					return false;
				}
				else
				{
					// Generate 'OK' feedback per category
					const FText ListTypeText = LOCTEXT("Property_DMXCategory", "DMX Category");
					const FText CategoryText = CategoryNode->GetDisplayNameText();

					EntityDragDropOp->SetFeedbackMessageOK(FText::Format(
						LOCTEXT("ReorderBeforeAndSetProperty", "Reorder {0} before '{1}'\nSet {2} = '{3}'"),
						EntityDragDropOp->GetDraggedEntitiesName(),
						CategoryNode->GetDisplayNameText(),
						ListTypeText,
						CategoryText
					));

					return true;
				}
			}
		}
	}

	return false;
}

TArray<TSharedRef<FDMXEntityTreeCategoryNode>> SDMXFixtureTypeTreeCategoryRow::GetCategoryNodesFromDragDropOp(const TSharedRef<FDMXEntityDragDropOperation>& DragDropOp) const
{
	TArray<TSharedRef<FDMXEntityTreeCategoryNode>> CategoryNodes;
	if (const TSharedPtr<SDMXFixtureTypeTree>& FixtureTypeTree = WeakFixtureTypeTree.Pin())
	{
		TArray<TWeakObjectPtr<UDMXEntity>> DraggedEntities = DragDropOp->GetDraggedEntities();

		for (TWeakObjectPtr<UDMXEntity> Entity : DraggedEntities)
		{
			if (UDMXEntityFixtureType* FixtureType = Cast<UDMXEntityFixtureType>(Entity.Get()))
			{
				TSharedPtr<FDMXEntityTreeCategoryNode> CategoryNode = FixtureTypeTree->FindCategoryNodeOfEntity(FixtureType);
				if (CategoryNode.IsValid())
				{
					CategoryNodes.Add(CategoryNode.ToSharedRef());
				}
			}
		}
	}

	return CategoryNodes;
}

const FSlateBrush* SDMXFixtureTypeTreeCategoryRow::GetBackgroundImageBrush() const
{
	if (IsHovered())
	{
		return IsItemExpanded()
			? FAppStyle::GetBrush("DetailsView.CategoryTop_Hovered")
			: FAppStyle::GetBrush("DetailsView.CollapsedCategory_Hovered");
	}
	else
	{
		return IsItemExpanded()
			? FAppStyle::GetBrush("DetailsView.CategoryTop")
			: FAppStyle::GetBrush("DetailsView.CollapsedCategory");
	}
}

UDMXLibrary* SDMXFixtureTypeTreeCategoryRow::GetDMXLibrary() const
{
	if (const TSharedPtr<SDMXFixtureTypeTree>& FixtureTypeTree = WeakFixtureTypeTree.Pin())
	{
		return FixtureTypeTree->GetDMXLibrary();
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
