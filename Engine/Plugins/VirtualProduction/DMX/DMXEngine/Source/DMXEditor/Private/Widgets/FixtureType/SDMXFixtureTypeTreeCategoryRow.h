// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Views/SListView.h"

#include "Widgets/Views/STableRow.h"

class FDMXEntityDragDropOperation;
class FDMXEntityTreeCategoryNode;
class SDMXFixtureTypeTree;
class UDMXLibrary;

class STableViewBase;


/** A row displaying a Category in a Fixture Type Tree */
class SDMXFixtureTypeTreeCategoryRow
	: public STableRow<TSharedPtr<FDMXEntityTreeCategoryNode>>
{
public:
	SLATE_BEGIN_ARGS(SDMXFixtureTypeTreeCategoryRow)
	{}
		SLATE_DEFAULT_SLOT(typename SDMXFixtureTypeTreeCategoryRow::FArguments, Content)

		SLATE_EVENT(FSimpleDelegate, OnFixtureTypeOrderChanged)

	SLATE_END_ARGS()

	/** Constructs this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedPtr<FDMXEntityTreeCategoryNode> InCategoryNode, bool bInIsRootCategory, TWeakPtr<SDMXFixtureTypeTree> InFixtureTypeTree);

protected:
	//~ SWidget interface begin
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	//~ SWidget interface end

private:
	/** Returns true if the drag drop op can be dropped on this row */
	bool TestCanDropWithFeedback(const TSharedRef<FDMXEntityDragDropOperation>& EntityDragDropOp) const;

	/** Returns the categories nodes the entities were dragged out from */
	TArray<TSharedRef<FDMXEntityTreeCategoryNode>> GetCategoryNodesFromDragDropOp(const TSharedRef<FDMXEntityDragDropOperation>& DragDropOp) const;

	/** Returns the background image brush */
	const FSlateBrush* GetBackgroundImageBrush() const;

	/** Returns the DMXLibrary of this row */
	UDMXLibrary* GetDMXLibrary() const;

	/** Pointer to the node this row displays */
	TWeakPtr<FDMXEntityTreeCategoryNode> WeakCategoryNode;

	/** The owning Fixture Patch List */
	TWeakPtr<SDMXFixtureTypeTree> WeakFixtureTypeTree;

	/** Border that holds the content */
	TSharedPtr<SBorder> ContentBorder;

	/** Broadcast when the order of fixture types in the library changed */
	FSimpleDelegate OnFixtureTypeOrderChanged;
};
