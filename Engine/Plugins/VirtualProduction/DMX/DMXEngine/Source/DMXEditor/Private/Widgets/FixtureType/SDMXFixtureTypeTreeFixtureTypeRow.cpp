// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXFixtureTypeTreeFixtureTypeRow.h"

#include "DMXEditorUtils.h"
#include "DMXEditorStyle.h"
#include "SDMXFixtureTypeTree.h"
#include "DragDrop/DMXEntityDragDropOp.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/DMXLibrary.h"
#include "Widgets/DMXEntityTreeNode.h"

#include "Styling/AppStyle.h"
#include "ScopedTransaction.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "SDMXFixtureTypeTreeFixtureTypeRow"

void SDMXFixtureTypeTreeFixtureTypeRow::Construct(const FArguments& InArgs, TSharedPtr<FDMXEntityTreeEntityNode> InEntityNode, TSharedPtr<STableViewBase> InOwnerTableView, TWeakPtr<SDMXFixtureTypeTree> InFixtureTypeTree)
{
	// Without ETableRowSignalSelectionMode::Instantaneous, when the user is editing a property in
	// the inspector panel and then clicks on a different row on the list panel, the selection event
	// is deferred. But because we update the tree right after a property change and that triggers
	// selection changes too, the selection change event is triggered only from UpdateTree, with
	// Direct selection mode, which doesn't trigger the SDMXEntityList::OnSelectionUpdated event.
	// This setting forces the event with OnMouseClick selection type to be fired as soon as the
	// row is clicked.
	STableRow<TSharedPtr<FDMXEntityTreeEntityNode>>::FArguments BaseArguments = STableRow<TSharedPtr<FDMXEntityTreeEntityNode>>::FArguments()
		.SignalSelectionMode(ETableRowSignalSelectionMode::Instantaneous)
		.OnDragDetected(this, &SDMXFixtureTypeTreeFixtureTypeRow::HandleOnDragDetected);

	STableRow<TSharedPtr<FDMXEntityTreeEntityNode>>::Construct(BaseArguments, InOwnerTableView.ToSharedRef());

	WeakEntityNode = InEntityNode;
	WeakFixtureTypeTree = InFixtureTypeTree;

	OnEntityDragged = InArgs._OnEntityDragged;
	OnGetFilterText = InArgs._OnGetFilterText;
	OnFixtureTypeOrderChanged = InArgs._OnFixtureTypeOrderChanged;

	const FSlateFontInfo NameFont = FCoreStyle::GetDefaultFontStyle("Regular", 10);

	ChildSlot
	[
		SNew(SHorizontalBox)
		// Status icon to show the user if there's an error with the Entity's usability
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(this, &SDMXFixtureTypeTreeFixtureTypeRow::GetStatusIcon)
			.ToolTipText(this, &SDMXFixtureTypeTreeFixtureTypeRow::GetStatusToolTip)
		]
		
		// Entity's name
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		.Padding(3.0f, 0.0f)
		[
			SAssignNew(InlineEditableFixtureTypeNameWidget, SInlineEditableTextBlock)
			.Text(this, &SDMXFixtureTypeTreeFixtureTypeRow::GetFixtureTypeName)
			.Font(NameFont)
			.HighlightText(this, &SDMXFixtureTypeTreeFixtureTypeRow::GetFilterText)
			.OnTextCommitted(this, &SDMXFixtureTypeTreeFixtureTypeRow::OnNameTextCommit)
			.OnVerifyTextChanged(this, &SDMXFixtureTypeTreeFixtureTypeRow::OnNameTextVerifyChanged)
			.IsSelected(this, &SDMXFixtureTypeTreeFixtureTypeRow::IsSelected)
		]
	];
}

void SDMXFixtureTypeTreeFixtureTypeRow::EnterRenameMode()
{
	InlineEditableFixtureTypeNameWidget->EnterEditingMode();
}

FReply SDMXFixtureTypeTreeFixtureTypeRow::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	const TSharedPtr<SDMXFixtureTypeTree> FixtureTypeTree = WeakFixtureTypeTree.Pin();
	const TSharedPtr<FDMXEntityTreeEntityNode> EntityNode = WeakEntityNode.Pin();

	if (FixtureTypeTree.IsValid() && EntityNode.IsValid())
	{
		FixtureTypeTree->SelectItemByNode(EntityNode.ToSharedRef());

		InlineEditableFixtureTypeNameWidget->EnterEditingMode();

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SDMXFixtureTypeTreeFixtureTypeRow::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{	
	if (TSharedPtr<FDMXEntityDragDropOperation> EntityDragDropOp = DragDropEvent.GetOperationAs<FDMXEntityDragDropOperation>())
	{
		TestCanDropWithFeedback(EntityDragDropOp.ToSharedRef());
	}
}

FReply SDMXFixtureTypeTreeFixtureTypeRow::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	const TSharedPtr<FDMXEntityDragDropOperation> EntityDragDropOp = DragDropEvent.GetOperationAs<FDMXEntityDragDropOperation>();
	const TSharedPtr<SDMXFixtureTypeTree> FixtureTypeTree = WeakFixtureTypeTree.Pin();
	const TSharedPtr<FDMXEntityTreeEntityNode> EntityNode = WeakEntityNode.Pin();
	UDMXLibrary* DMXLibrary = FixtureTypeTree.IsValid() ? FixtureTypeTree->GetDMXLibrary() : nullptr;
	UDMXEntityFixtureType* HoveredFixtureType = EntityNode.IsValid() ? Cast<UDMXEntityFixtureType>(EntityNode->GetEntity()) : nullptr;
 
	if (EntityDragDropOp.IsValid() && FixtureTypeTree.IsValid() && EntityNode.IsValid() && DMXLibrary && HoveredFixtureType)
	{
		if(TestCanDropWithFeedback(EntityDragDropOp.ToSharedRef()))
		{
			const TArray<TWeakObjectPtr<UDMXEntity>> DraggedEntities = EntityDragDropOp->GetDraggedEntities();

			// Register transaction and current DMX library state for Undo
			const FScopedTransaction ReorderTransaction = FScopedTransaction(
				FText::Format(LOCTEXT("ReorderEntities", "Reorder {0}|plural(one=Entity, other=Entities)"), DraggedEntities.Num())
			);

			DMXLibrary->Modify();

			// The index of the Entity, we're about to insert the dragged one before
			const int32 InsertBeforeIndex = DMXLibrary->FindEntityIndex(HoveredFixtureType);
			check(InsertBeforeIndex != INDEX_NONE);

			// Reverse for to keep dragged entities order
			for (int32 EntityIndex = DraggedEntities.Num() - 1; EntityIndex > -1; --EntityIndex)
			{
				if (DraggedEntities[EntityIndex].IsValid())
				{
					UDMXEntity* Entity = DraggedEntities[EntityIndex].Get();
					DMXLibrary->SetEntityIndex(Entity, InsertBeforeIndex);
				}
			}

			// Handle cases where it was was dropped on another category
			TSharedPtr<FDMXEntityTreeCategoryNode> TargetCategoryNode = StaticCastSharedPtr<FDMXEntityTreeCategoryNode>(EntityNode->GetParent().Pin());
			check(TargetCategoryNode.IsValid());

			const FDMXFixtureCategory& FixtureCategory = TargetCategoryNode->GetCategoryValue();
			for (TWeakObjectPtr<UDMXEntity> Entity : DraggedEntities)
			{
				if (UDMXEntityFixtureType* FixtureType = Cast<UDMXEntityFixtureType>(Entity))
				{
					FixtureType->Modify();
					FixtureType->DMXCategory = FixtureCategory;
				}
			}

			// Display the changes in the Entities list
			FixtureTypeTree->UpdateTree();

			OnFixtureTypeOrderChanged.ExecuteIfBound();

			return FReply::Handled().EndDragDrop();
		}
	}

	return FReply::Unhandled();
}

bool SDMXFixtureTypeTreeFixtureTypeRow::TestCanDropWithFeedback(const TSharedRef<FDMXEntityDragDropOperation>& EntityDragDropOp) const
{
	const TSharedPtr<SDMXFixtureTypeTree> FixtureTypeTree = WeakFixtureTypeTree.Pin();
	const TSharedPtr<FDMXEntityTreeEntityNode> EntityNode = WeakEntityNode.Pin();
	UDMXEntityFixtureType* HoveredFixtureType = EntityNode.IsValid() ? Cast<UDMXEntityFixtureType>(EntityNode->GetEntity()) : nullptr;

	if (FixtureTypeTree.IsValid() && EntityNode.IsValid() && HoveredFixtureType)
	{
		const TArray<TWeakObjectPtr<UDMXEntity>> DraggedEntities = EntityDragDropOp->GetDraggedEntities();

		const bool bOnlyFixtureTypesAreDragged = [DraggedEntities]()
		{
			for (TWeakObjectPtr<UDMXEntity> Entity : DraggedEntities)
			{
				if (Entity.IsValid() && Entity->GetClass() != UDMXEntityFixtureType::StaticClass())
				{
					return false;
				}
			}
			return true;
		}();

		const bool bRowAndEntitiesAreOfSameDMXLibrary = [DraggedEntities, this]()
		{
			UDMXLibrary* DMXLibrary = GetDMXLibrary();
			for (TWeakObjectPtr<UDMXEntity> Entity : DraggedEntities)
			{
				if (Entity.IsValid() && Entity->GetParentLibrary() != DMXLibrary)
				{
					return false;
				}
			}
			return true;
		}();

		if (bOnlyFixtureTypesAreDragged && bRowAndEntitiesAreOfSameDMXLibrary)
		{
			// Test the hovered entity is not the dragged entitiy
			if (DraggedEntities.Contains(HoveredFixtureType))
			{
				EntityDragDropOp->SetFeedbackMessageError(FText::Format(
					LOCTEXT("ReorderBeforeItself", "Drop {0} on itself"),
					EntityDragDropOp->GetDraggedEntitiesName()
				));

				return false;
			}

			check(EntityNode->GetParent().IsValid());
			TSharedPtr<FDMXEntityTreeCategoryNode> TargetCategoryNode = StaticCastSharedPtr<FDMXEntityTreeCategoryNode>(EntityNode->GetParent().Pin());
			check(TargetCategoryNode.IsValid());

			// Generate 'OK' feedback per category
			FText ListTypeText = LOCTEXT("Property_DMXCategory", "DMX Category");
			FText CategoryText = TargetCategoryNode->GetDisplayNameText();

			EntityDragDropOp->SetFeedbackMessageOK(FText::Format(
				LOCTEXT("ReorderBeforeAndSetProperty", "Reorder {0} before '{1}'\nSet {2} = '{3}'"),
				EntityDragDropOp->GetDraggedEntitiesName(),
				FText::FromString(HoveredFixtureType->GetDisplayName()),
				ListTypeText,
				CategoryText
			));

			return true;
		}
	}

	return false;
}

FText SDMXFixtureTypeTreeFixtureTypeRow::GetFilterText() const
{
	if (OnGetFilterText.IsBound())
	{
		return OnGetFilterText.Execute();
	}
	return FText();
}

void SDMXFixtureTypeTreeFixtureTypeRow::OnNameTextCommit(const FText& InNewName, ETextCommit::Type InTextCommit)
{
	TSharedPtr<SDMXFixtureTypeTree> FixtureTypeTree = WeakFixtureTypeTree.Pin();
	TSharedPtr<FDMXEntityTreeEntityNode> EntityNode = WeakEntityNode.Pin();
	UDMXEntity* Entity = EntityNode.IsValid() ? EntityNode->GetEntity() : nullptr;
	UDMXLibrary* DMXLibrary = GetDMXLibrary();

	if (FixtureTypeTree.IsValid() && EntityNode.IsValid() && Entity && DMXLibrary)
	{
		const FString NewNameString = InNewName.ToString();

		// Check if the name is unchanged
		if (NewNameString.Equals(EntityNode->GetDisplayNameString()))
		{
			return;
		}

		const FScopedTransaction Transaction(LOCTEXT("RenameEntity", "Rename Entity"));
		Entity->PreEditChange(UDMXEntityFixtureType::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UDMXEntityFixtureType, Name)));

		FDMXEditorUtils::RenameEntity(DMXLibrary, Entity, NewNameString);

		Entity->PostEditChange();

		FixtureTypeTree->SelectItemByName(NewNameString, ESelectInfo::OnMouseClick);
	}
}

bool SDMXFixtureTypeTreeFixtureTypeRow::OnNameTextVerifyChanged(const FText& InNewText, FText& OutErrorMessage)
{
	TSharedPtr<FDMXEntityTreeEntityNode> EntityNode = WeakEntityNode.Pin();
	UDMXEntity* Entity = EntityNode.IsValid() ? EntityNode->GetEntity() : nullptr;
	UDMXLibrary* DMXLibrary = GetDMXLibrary();

	if (EntityNode.IsValid() && Entity && DMXLibrary)
	{
		FString TextAsString = InNewText.ToString();
		if (TextAsString.Equals(EntityNode->GetDisplayNameString()))
		{
			return true;
		}

		return FDMXEditorUtils::ValidateEntityName(
			TextAsString,
			DMXLibrary,
			Entity->GetClass(),
			OutErrorMessage
		);
	}

	return false;
}

FText SDMXFixtureTypeTreeFixtureTypeRow::GetFixtureTypeName() const
{
	if (TSharedPtr<FDMXEntityTreeEntityNode> EntityNode = WeakEntityNode.Pin())
	{
		return EntityNode->GetDisplayNameText();
	}
	return LOCTEXT("InvalidNodeLabel", "Invalid Node");
}

FReply SDMXFixtureTypeTreeFixtureTypeRow::HandleOnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) && OnEntityDragged.IsBound())
	{
		if (TSharedPtr<FDMXEntityTreeEntityNode> EntityNode = WeakEntityNode.Pin())
		{
			return OnEntityDragged.Execute(EntityNode, MouseEvent);
		}
	}

	return FReply::Unhandled();
}

const FSlateBrush* SDMXFixtureTypeTreeFixtureTypeRow::GetStatusIcon() const
{
	static FSlateNoResource EmptyBrush = FSlateNoResource();

	if (TSharedPtr<FDMXEntityTreeEntityNode> EntityNode = WeakEntityNode.Pin())
	{
		if (!EntityNode->GetErrorStatus().IsEmpty())
		{
			return FAppStyle::GetBrush("Icons.Error");
		}

		if (!EntityNode->GetWarningStatus().IsEmpty())
		{
			return FAppStyle::GetBrush("Icons.Warning");
		}
	}

	return &EmptyBrush;
}

FText SDMXFixtureTypeTreeFixtureTypeRow::GetStatusToolTip() const
{
	if (TSharedPtr<FDMXEntityTreeEntityNode> EntityNode = WeakEntityNode.Pin())
	{
		const FText& ErrorStatus = EntityNode->GetErrorStatus();
		if (!ErrorStatus.IsEmpty())
		{
			return ErrorStatus;
		}

		const FText& WarningStatus = EntityNode->GetWarningStatus();
		if (!WarningStatus.IsEmpty())
		{
			return WarningStatus;
		}
	}

	return FText::GetEmpty();
}

UDMXLibrary* SDMXFixtureTypeTreeFixtureTypeRow::GetDMXLibrary() const
{
	if (const TSharedPtr<SDMXFixtureTypeTree>& FixtureTypeTree = WeakFixtureTypeTree.Pin())
	{
		return FixtureTypeTree->GetDMXLibrary();
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
