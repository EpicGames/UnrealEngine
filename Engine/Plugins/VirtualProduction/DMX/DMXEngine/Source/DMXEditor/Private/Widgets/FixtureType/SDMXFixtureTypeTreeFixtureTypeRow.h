// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Views/SListView.h"

#include "Widgets/Views/STableRow.h"

class FDMXEntityDragDropOperation;
class FDMXEntityTreeEntityNode;
class FDMXEntityTreeNodeBase;
class SDMXFixtureTypeTree;
class UDMXLibrary;

class SInlineEditableTextBlock;


/** A fixture type row in a fixture patch tree */
class SDMXFixtureTypeTreeFixtureTypeRow
	: public STableRow<TSharedPtr<FDMXEntityTreeEntityNode>>
{
	DECLARE_DELEGATE_RetVal_TwoParams(FReply, FOnEntityDragged, TSharedPtr<FDMXEntityTreeNodeBase>, const FPointerEvent&);
	DECLARE_DELEGATE_RetVal(FText, FOnGetFilterText);
	DECLARE_DELEGATE_OneParam(FOnAutoAssignChannelStateChanged, bool);

public:
	SLATE_BEGIN_ARGS(SDMXFixtureTypeTreeFixtureTypeRow)
	{}
		SLATE_DEFAULT_SLOT(typename SDMXFixtureTypeTreeFixtureTypeRow::FArguments, Content)

		SLATE_EVENT(FOnEntityDragged, OnEntityDragged)

		SLATE_EVENT(FOnGetFilterText, OnGetFilterText)

		SLATE_EVENT(FOnAutoAssignChannelStateChanged, OnAutoAssignChannelStateChanged)

		SLATE_EVENT(FSimpleDelegate, OnFixtureTypeOrderChanged)

	SLATE_END_ARGS()

	/** Constructs this widget */
	void Construct(const FArguments& InArgs, TSharedPtr<FDMXEntityTreeEntityNode> InEntityNode, TSharedPtr<STableViewBase> InOwnerTableView, TWeakPtr<SDMXFixtureTypeTree> InFixtureTypeTree);

	/** Starts renaming the widget (enters editing mode on its Editable Text Block) */
	void EnterRenameMode();

protected:
	//~ SWidget interface begin
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	//~ SWidget interface end

private:
	/** Returns true if the drag drop op can be dropped on this row */
	bool TestCanDropWithFeedback(const TSharedRef<FDMXEntityDragDropOperation>& EntityDragDropOp) const;

	/** Called when the row is being dragged */
	FReply HandleOnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/** Get the current filter text from the search box */
	FText GetFilterText() const;

	/** Verifies the name of the component when changing it */
	bool OnNameTextVerifyChanged(const FText& InNewText, FText& OutErrorMessage);

	/** Commits the new name of the component */
	void OnNameTextCommit(const FText& InNewName, ETextCommit::Type InTextCommit);

	/** Returns the name of the patch as text */
	FText GetFixtureTypeName() const;

	/** Get the icon for the Entity usability status. If it's all good, it's an emtpy image. */
	const FSlateBrush* GetStatusIcon() const;

	/** Get the tool tip text for the status icon */
	FText GetStatusToolTip() const;

	/** Returns the DMXLibrary of this row */
	UDMXLibrary* GetDMXLibrary() const;

	/** The node this row displays */
	TWeakPtr<FDMXEntityTreeEntityNode> WeakEntityNode;

	/** The fixture type tree widget that owns this row */
	TWeakPtr<SDMXFixtureTypeTree> WeakFixtureTypeTree;

	/** Called when the entity list changed auto assign channel state */
	FOnAutoAssignChannelStateChanged OnAutoAssignChannelStateChanged;

	/** Delegate broadcast when an entity was dragged */
	FOnEntityDragged OnEntityDragged;

	/** Delegate broadcast when the filter text is requested */
	FOnGetFilterText OnGetFilterText;

	/** Called when the entity list changed order of the library's entity array */
	FSimpleDelegate OnFixtureTypeOrderChanged;

	/** The editable text block to display and change the name of the fixture type */
	TSharedPtr<SInlineEditableTextBlock> InlineEditableFixtureTypeNameWidget;
};
