// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXLibraryEditorTab.h"

#include "Customizations/DMXLibraryDetails.h"
#include "Library/DMXLibrary.h"

#include "IDetailsView.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"


void SDMXLibraryEditorTab::Construct(const FArguments& InArgs)
{
	DMXEditor = InArgs._DMXEditor;

	// Initialize property view widget
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bCustomNameAreaLocation = false;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	const TSharedRef<IDetailsView> DMXLibraryDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	DMXLibraryDetailsView->RegisterInstancedCustomPropertyLayout(UDMXLibrary::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(FDMXLibraryDetails::MakeInstance));

	if (IsValid(InArgs._DMXLibrary))
	{
		DMXLibraryDetailsView->SetObject(InArgs._DMXLibrary, true);

		ChildSlot
			[
				DMXLibraryDetailsView
			];
	}
}
