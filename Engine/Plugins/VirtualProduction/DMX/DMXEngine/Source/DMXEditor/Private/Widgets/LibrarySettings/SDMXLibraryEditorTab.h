// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXEditor.h"

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"

class FDMXEditor;
class UDMXLibrary;

class IDetailsView;


/** Editor for Fixture Patches */
class SDMXLibraryEditorTab
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMXLibraryEditorTab)
	{}

		SLATE_ARGUMENT(UDMXLibrary*, DMXLibrary)
	
		SLATE_ARGUMENT(TWeakPtr<FDMXEditor>, DMXEditor)

	SLATE_END_ARGS()

public:
	/** Constructs this widget */
	void Construct(const FArguments& InArgs);

private:
	/** The DMXEditor that owns this tab */
	TWeakPtr<FDMXEditor> DMXEditor;
};