// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXConflictMonitorActiveObjectItem.h"

namespace UE::DMX
{
	FDMXConflictMonitorActiveObjectItem::FDMXConflictMonitorActiveObjectItem(const FName& InObjectName, const FSoftObjectPath& InObjectPath)
		: ObjectName(InObjectName)
		, ObjectPath(InObjectPath)
	{}
}
