// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"
#include "UObject/SoftObjectPath.h"

namespace UE::DMX
{
	/** A row in the Conflict Monitor's Active Object list */
	class FDMXConflictMonitorActiveObjectItem
		: public TSharedFromThis<FDMXConflictMonitorActiveObjectItem>
	{
	public:
		/** Constructor. Object Path is allowed to be invalid */
		FDMXConflictMonitorActiveObjectItem(const FName& ObjectName, const FSoftObjectPath& ObjectPath);

		/** The name of the object */
		const FName ObjectName;

		/** The object path. Can be invalid. */
		const FSoftObjectPath ObjectPath;
	};
}
