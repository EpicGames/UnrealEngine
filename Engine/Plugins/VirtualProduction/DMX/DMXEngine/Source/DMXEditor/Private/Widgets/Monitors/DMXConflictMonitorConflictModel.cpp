// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXConflictMonitorConflictModel.h"

#include "Algo/Sort.h"
#include "DMXEditorSettings.h"
#include "Framework/Text/ITextDecorator.h"
#include "IO/DMXConflictMonitor.h"
#include "IO/DMXOutputPort.h"
#include "Internationalization/Regex.h"


#define LOCTEXT_NAMESPACE "FDMXConflictMonitorConflictModel"

namespace UE::DMX
{
	FDMXConflictMonitorConflictModel::FDMXConflictMonitorConflictModel(const TArray<TSharedRef<FDMXMonitoredOutboundDMXData>>& InConflicts)
		: Conflicts(InConflicts)
	{
		ParseConflict();
	}

	FString FDMXConflictMonitorConflictModel::GetConflictAsString(bool bRichTextMarkup) const
	{
		if (bRichTextMarkup)
		{
			FString Result = Title;
			for (const FString& Detail : Details)
			{
				Result.Append(TEXT("\n") + Detail);
			}

			return Result;
		}
		else
		{
			FString Result = GetStringNoMarkup(Title);
			for (const FString& Detail : Details)
			{
				Result.Append(TEXT("\n") + GetStringNoMarkup(Detail));
			}

			return Result;
		}
	}

	void FDMXConflictMonitorConflictModel::ParseConflict()
	{
		TArray<FName> NameStacks;
		for (const TSharedRef<FDMXMonitoredOutboundDMXData>& Conflict : Conflicts)
		{
			NameStacks.AddUnique(Conflict->Trace);
		}

		// Lexical sort
		Algo::Sort(NameStacks, [](const FName& SenderA, const FName& SenderB)
			{
				return SenderA.Compare(SenderB) < 0;
			});

		const UDMXEditorSettings* EditorSettings = GetDefault<UDMXEditorSettings>();
		const uint8 Depth = EditorSettings->ConflictMonitorSettings.Depth;

		int32 CountStack = 0;
		TArray<FString> PrimaryNames;
		for (const FName& NameStack : NameStacks)
		{
			++CountStack;
			const bool bFirstNameInStack = &NameStack == &NameStacks[0];

			TArray<FString> Substrings;
			NameStack.ToString().ParseIntoArray(Substrings, TEXT(","));

			// Depth filter
			if (Substrings.Num() > Depth)
			{
				Substrings.SetNum(Depth);
			}

			FString DetailsString;
			for (int32 SubstringIndex = 0; SubstringIndex < Substrings.Num(); SubstringIndex++)
			{
				const bool bPrimaryName = &Substrings[SubstringIndex] == &Substrings[0];
				const FString CleanSubstring = FPaths::GetBaseFilename(Substrings[SubstringIndex]);

				// Create the title
				if (bPrimaryName && bFirstNameInStack)
				{
					FString FirstTraceInStack = LOCTEXT("ConflictDetected", "Send DMX Conflict: ").ToString() + CleanSubstring;
					FirstTraceInStack = StyleString(FirstTraceInStack, TEXT("ConflictLog.Title"));

					PrimaryNames.Add(CleanSubstring);
					Title.Append(FirstTraceInStack);
				}
				else if (bPrimaryName)
				{
					FString FirstTraceInStack = TEXT(" / ") + CleanSubstring;
					FirstTraceInStack = StyleString(FirstTraceInStack, TEXT("ConflictLog.Title"));
					
					// Show primary names of conflicting objects only once, even if there are many conflicts.
					// E.g. "Obj1 / Obj2", not "Obj1 / Obj2 / Obj2".
					if (!PrimaryNames.Contains(CleanSubstring))
					{
						PrimaryNames.Add(CleanSubstring);
						Title.Append(FirstTraceInStack);
					}
				}

				// Don't add details if depth is 1, details would be the same as the title.
				if (Depth == 1)
				{
					break;
				}

				// Create details
				if (bPrimaryName)
				{
					const FString Number = TEXT("\t\t") + FString::FromInt(CountStack) + TEXT(". ");
					const FString FirstTraceInStack = StyleString(Number + CleanSubstring, TEXT("ConflictLog.Warning"));
					DetailsString = FirstTraceInStack;
				}
				else
				{
					const FString NextTraceInStack = StyleString(TEXT(" -> ") + CleanSubstring, TEXT("ConflictLog.Warning"));
					DetailsString.Append(NextTraceInStack);
				}
			}

			Details.Add(DetailsString);
		}

		if (Depth > 1)
		{
			// Add port and universe/channel info
			const FString Port = StyleString(LOCTEXT("PortInfo", "\t\tPort: ").ToString() + GetPortNameString(), TEXT("ConflictLog.Error"));
			const FString Universe = StyleString(LOCTEXT("UniverseInfo", "Universe: ").ToString() + GetUniverseString(), TEXT("ConflictLog.Error"));
			const FString Channels = StyleString(LOCTEXT("ChannelInfo", "Channel: ").ToString() + GetChannelsString(), TEXT("ConflictLog.Error"));

			const FString DetailsSeparator = StyleString(TEXT(" - "), TEXT("ConflictLog.Error"));
			Details.Add(Port + DetailsSeparator + Universe + DetailsSeparator + Channels);
			Details.Add(TEXT(""));
		}
	}

	FString FDMXConflictMonitorConflictModel::GetPortNameString() const
	{
		static const FText InvalidPortName = LOCTEXT("InvalidPortName", "<Invalid Port>");

		TArray<FName> PortNames;
		for (const TSharedRef<FDMXMonitoredOutboundDMXData>& Conflict : Conflicts)
		{
			const FName PortName = Conflict->OutputPort.IsValid() ? *Conflict->OutputPort.Pin()->GetPortName() : *InvalidPortName.ToString();
			PortNames.AddUnique(PortName);
		}
		
		FString UniquePortString;
		for (const FName& PortName : PortNames)
		{
			UniquePortString.Append(PortName.ToString());

			if (&PortName != &PortNames.Last())
			{
				UniquePortString.Append(TEXT(", "));
			}
		}

		return UniquePortString;
	}

	FString FDMXConflictMonitorConflictModel::GetUniverseString() const
	{
		if (!Conflicts.IsEmpty())
		{
			return FString::FromInt(Conflicts[0]->LocalUniverseID);
		}
		return FString();
	}

	FString FDMXConflictMonitorConflictModel::GetChannelsString() const
	{
		TArray<int32> Channels;

		// Find conflicting channels
		for (const TSharedRef<FDMXMonitoredOutboundDMXData>& Conflict : Conflicts)
		{
			for (const TSharedRef<FDMXMonitoredOutboundDMXData>& Other : Conflicts)
			{
				for (const TTuple<int32, uint8>& ChannelToValuePair : Conflict->ChannelToValueMap)
				{
					if (Other->ChannelToValueMap.Contains(ChannelToValuePair.Key))
					{
						Channels.AddUnique(ChannelToValuePair.Key);
					}
				}
			}
		}
		Algo::Sort(Channels);
		
		FString ChannelsString;
		constexpr int32 MaxIndex = 31;
		for (int32 ChannelIndex = 0; ChannelIndex < Channels.Num(); ChannelIndex++)
		{
			ChannelsString.Append(FString::FromInt(Channels[ChannelIndex]));

			if (ChannelIndex >= MaxIndex)
			{
				break;
			}

			if (&Channels[ChannelIndex] != &Channels.Last())
			{
				ChannelsString.Append(TEXT(", "));
			}
		}

		if (Channels.Num() >= MaxIndex + 1)
		{
			const FText MoreText = FText::Format(LOCTEXT("MoreText", " (and {0} more)"), FText::FromString(FString::FromInt(Channels.Num() - MaxIndex + 1)));
			ChannelsString.Append(MoreText.ToString());
		}

		return ChannelsString;
	}

	FString FDMXConflictMonitorConflictModel::StyleString(FString String, FString MarkupString) const
	{
		return FString::Printf(TEXT("<%s>%s</>"), *MarkupString, *String);
	}

	FString FDMXConflictMonitorConflictModel::GetStringNoMarkup(const FString& String) const
	{
		const FRegexPattern RegexPattern(TEXT("(<.*?>)"));
		FRegexMatcher Matcher(RegexPattern, String);

		FString Result = String;
		while (Matcher.FindNext())
		{
			Result.ReplaceInline(*Matcher.GetCaptureGroup(1), TEXT(""));
		}

		return Result;
	}
}

#undef LOCTEXT_NAMESPACE
