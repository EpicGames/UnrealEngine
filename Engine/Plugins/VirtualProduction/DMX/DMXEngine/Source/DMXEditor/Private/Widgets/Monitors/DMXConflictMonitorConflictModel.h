// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

namespace UE::DMX
{
	struct FDMXMonitoredOutboundDMXData;

	/** An item in the pixel mapping conflict list */
	class FDMXConflictMonitorConflictModel
		: public TSharedFromThis<FDMXConflictMonitorConflictModel>
	{
		friend class SharedPointerInternals::TIntrusiveReferenceController<FDMXConflictMonitorConflictModel, ESPMode::ThreadSafe>;

	public:
		FDMXConflictMonitorConflictModel(const TArray<TSharedRef<FDMXMonitoredOutboundDMXData>>& InConflicts);

		FString GetConflictAsString(bool bRichTextMarkup = false) const;

	private:
		/** Parses the conflict as string, stores it in Title and Details members */
		void ParseConflict();

		FString Title;
		TArray<FString> Details;

		/** Returns the name of the ports in which a conflict occurs */
		FString GetPortNameString() const;

		/** Returns the universe string */
		FString GetUniverseString() const;

		/** Returns the channels string */
		FString GetChannelsString() const;

		/** Applies the style to the string */
		[[nodiscard]] FString StyleString(FString String, FString MarkupString) const;

		/** Returns the string without markup */
		FString GetStringNoMarkup(const FString& String) const;

		FDMXConflictMonitorConflictModel() = default;

		const TArray<TSharedRef<FDMXMonitoredOutboundDMXData>> Conflicts;
	};
}
