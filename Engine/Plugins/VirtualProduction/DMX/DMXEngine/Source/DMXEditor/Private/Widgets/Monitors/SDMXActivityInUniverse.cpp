// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Monitors/SDMXActivityInUniverse.h"

#include "DMXProtocolConstants.h"
#include "DMXEditorStyle.h"
#include "Interfaces/IDMXProtocol.h"
#include "Widgets/SDMXChannel.h"

#include "Brushes/SlateColorBrush.h"
#include "Engine/Font.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Views/SListView.h"
#include "UObject/ConstructorHelpers.h"

struct FDMXMonitorUniverseCounter;


#define LOCTEXT_NAMESPACE "SDMXActivityInUniverse"

void FDMXUniverseMonitorChannelItem::SetValue(uint8 NewValue)
{
	check (Widget.IsValid())
	
	Widget->SetValue(NewValue);
	Value = NewValue;
}

TSharedRef<SDMXChannel> FDMXUniverseMonitorChannelItem::GetWidgetChecked() const
{
	check(Widget.IsValid());
	return Widget.ToSharedRef();
}

void FDMXUniverseMonitorChannelItem::CreateWidget(uint8 InitialValue)
{
	// Don't create the widget twice, use HasWidget to test if it exists.
	check(!Widget.IsValid());
	
	Widget =
		SNew(SDMXChannel)
		.ChannelID(ChannelID)
		.Value(InitialValue);
}

SDMXActivityInUniverse::~SDMXActivityInUniverse()
{
	TSharedPtr<FActiveTimerHandle> PinnedTimerHandle = AnimationTimerHandle.Pin();
	if (PinnedTimerHandle.IsValid())
	{
		UnRegisterActiveTimer(PinnedTimerHandle.ToSharedRef());
	}
}

void SDMXActivityInUniverse::Construct(const FArguments& InArgs)
{
	SetVisibility(EVisibility::SelfHitTestInvisible);
	SetCanTick(false);

	UniverseID = InArgs._UniverseID;

	ChildSlot
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()	
			.HAlign(HAlign_Fill)			
			[
				SNew(SHorizontalBox)

				// Universe Label
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)		
				.AutoWidth()
				[
					SNew(SBox)
					.WidthOverride(60.0f)
					.MaxDesiredWidth(60.0f)
					.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
					.HAlign(HAlign_Left)
					[
						SNew(STextBlock)
						.Text(this, &SDMXActivityInUniverse::GetIDLabel)
						.ColorAndOpacity(FSlateColor(IDColor))
						.Justification(ETextJustify::Left)
						.Font(FDMXEditorStyle::Get().GetFontStyle("DMXEditor.Font.InputUniverseID"))
					]
				]
					
				// Channels View
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				.FillWidth(1.0f)
				[					
					SAssignNew(ChannelList, SListView<TSharedPtr<FDMXUniverseMonitorChannelItem>>)
					.OnGenerateRow(this, &SDMXActivityInUniverse::GenerateChannelRow)
					.ListItemsSource(&ChannelListSource)
					.Orientation(EOrientation::Orient_Horizontal)
					.ScrollbarVisibility(EVisibility::Collapsed)
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SSeparator)
			]
		];
	
	// Initialize channel items but not their widgets
	for (int ChannelID = 1; ChannelID <= DMX_MAX_ADDRESS; ChannelID++)
	{
		Channels.Add(MakeShared<FDMXUniverseMonitorChannelItem>(ChannelID));
	}
}	

void SDMXActivityInUniverse::VisualizeBuffer(const TArray<uint8, TFixedAllocator<DMX_UNIVERSE_SIZE>>& Values)
{
	// is NewValue a different value from current one?
	if (Buffer != Values)
	{
		// Activate timer to animate value bar color
		if (!AnimationTimerHandle.IsValid())
		{
			AnimationTimerHandle = RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateSP(this, &SDMXActivityInUniverse::UpdateValueChangedAnim));
		}
		// Reset value change animation
		NewValueFreshness = 1.0f;

		Buffer = Values;
		UpdateChannels();
	}	
}

void SDMXActivityInUniverse::UpdateChannels()
{
	bool bNumChannelsChanged = false;
	for (const TSharedPtr<FDMXUniverseMonitorChannelItem>& ChannelItem : Channels)
	{
		int32 ChannelIndex = ChannelItem->GetChannelID() - 1;
		if (!Buffer.IsValidIndex(ChannelIndex))
		{
			return;
		}

		// Update channels that aren't of zero value or changed to zero to zero value
		if (ChannelItem->GetValue() != Buffer[ChannelIndex])
		{
			if (ChannelItem->HasWidget())
			{
				ChannelItem->SetValue(Buffer[ChannelIndex]);
			}
			else
			{				
				// Instantiate the widget
				ChannelItem->CreateWidget(Buffer[ChannelIndex]);
				ChannelItem->SetValue(Buffer[ChannelIndex]);

				// Add the item the list source
				ChannelListSource.Add(ChannelItem);
				bNumChannelsChanged = true;
			}
		}
	}

	if (bNumChannelsChanged)
	{
		// Sort channels ascending
		ChannelListSource.Sort([](const TSharedPtr<FDMXUniverseMonitorChannelItem>& ChannelA, const TSharedPtr<FDMXUniverseMonitorChannelItem>& ChannelB) {
			return ChannelA->GetChannelID() < ChannelB->GetChannelID();
		});	

		check(ChannelList.IsValid());
		ChannelList->RequestListRefresh();
	}
}

TSharedRef<ITableRow> SDMXActivityInUniverse::GenerateChannelRow(TSharedPtr<FDMXUniverseMonitorChannelItem> ChannelItem, const TSharedRef<STableViewBase>& OwnerTable)
{	
	return
		SNew(STableRow<TSharedPtr<SDMXChannel>>, OwnerTable)
		[
			ChannelItem->GetWidgetChecked()
		];
}

EActiveTimerReturnType SDMXActivityInUniverse::UpdateValueChangedAnim(double InCurrentTime, float InDeltaTime)
{
	NewValueFreshness = FMath::Max(NewValueFreshness - InDeltaTime / NewValueChangedAnimDuration, 0.0f);

	// Disable timer when the value bar color animation ends
	if (NewValueFreshness <= 0.0f)
	{
		TSharedPtr<FActiveTimerHandle> PinnedTimerHandle = AnimationTimerHandle.Pin();
		if (PinnedTimerHandle.IsValid())
		{
			UnRegisterActiveTimer(PinnedTimerHandle.ToSharedRef());
		}
	}
	return EActiveTimerReturnType::Continue;
}

FText SDMXActivityInUniverse::GetIDLabel() const
{
	return FText::AsNumber(UniverseID.Get());
}

FSlateColor SDMXActivityInUniverse::GetBackgroundColor() const
{
	const float CurrentPercent = 0.5f;

	// Intensities to be animated when a new value is set and then multiplied by the background color
	static const float NormalIntensity = 0.3f;
	static const float FreshValueIntensity = 0.7f;
	// lerp intensity depending on NewValueFreshness^2 to make it pop for a while when it has just been updated
	const float ValueFreshnessIntensity = FMath::Lerp(NormalIntensity, FreshValueIntensity, NewValueFreshness * NewValueFreshness);

	// color variations for low and high channel values
	static const FVector LowValueColor = FVector(0.0f, 0.045f, 0.15f);
	static const FVector HighValueColor = FVector(0.0f, 0.3f, 1.0f);
	const FVector ColorFromChannelValue = FMath::Lerp(LowValueColor, HighValueColor, CurrentPercent);

	// returning a FVector, a new FSlateColor will be created from it with (RGB = vector, Alpha = 1.0)
	FVector Result = ColorFromChannelValue * ValueFreshnessIntensity;
	return FSlateColor(FLinearColor(Result.X, Result.Y, Result.Z));
}

#undef LOCTEXT_NAMESPACE
