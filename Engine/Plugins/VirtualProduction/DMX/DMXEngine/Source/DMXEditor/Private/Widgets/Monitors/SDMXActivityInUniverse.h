// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolConstants.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class SDMXChannel;


/** An item that references a channel, potentially along with a UI element */
class FDMXUniverseMonitorChannelItem
	: public TSharedFromThis<FDMXUniverseMonitorChannelItem>
{
public:
	FDMXUniverseMonitorChannelItem(int32 InChannelID)
		: ChannelID(InChannelID)
	{}

	int32 GetChannelID() const { return ChannelID; }
	uint8 GetValue() const { return Value; }
	void SetValue(uint8 NewValue);

	TSharedRef<SDMXChannel> GetWidgetChecked() const;
	bool HasWidget() const { return Widget.IsValid(); }
	void CreateWidget(uint8 InitialValue);

private:
	int32 ChannelID = 0;
	uint8 Value = 0;

	TSharedPtr<SDMXChannel> Widget;
};


/** Visualization of the DMX activity in a single DMX Universe */
class SDMXActivityInUniverse	
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMXActivityInUniverse)
		: _UniverseID(0)
	{}

		/** The universe ID this widget represents */
		SLATE_ATTRIBUTE(uint32, UniverseID)

	SLATE_END_ARGS()

	virtual ~SDMXActivityInUniverse();

	/** Constructs the widget */
	void Construct(const FArguments& InArgs);
	
	/** Visualizes specified buffer */
	void VisualizeBuffer(const TArray<uint8, TFixedAllocator<DMX_UNIVERSE_SIZE>>& Values);

	/** Gets the universe ID this widget represents */
	uint32 GetUniverseID() const { return UniverseID.Get(); }

protected:
	/** Updates Channels along with their UI representation */
	void UpdateChannels();

	/** Generates a widget for the Channels List View */
	TSharedRef<class ITableRow> GenerateChannelRow(TSharedPtr<class FDMXUniverseMonitorChannelItem> InChannelView, const TSharedRef<STableViewBase>& OwnerTable);

private:
	/** The universe ID this widget represents */
	TAttribute<uint32> UniverseID;

	/** The buffer currently displayed */
	TArray<uint8, TFixedAllocator<DMX_UNIVERSE_SIZE>> Buffer;

	/** List view widget that displays current non-zero values for all universes */
	TSharedPtr<SListView<TSharedPtr<class FDMXUniverseMonitorChannelItem>>> ChannelList;

	/** Array of all channels in the universe, as channel items */
	TArray<TSharedPtr<class FDMXUniverseMonitorChannelItem>> Channels;

	/** Cached source of channels for the channel list */
	TArray<TSharedPtr<class FDMXUniverseMonitorChannelItem>> ChannelListSource;

private:
	/**
	* Updates the variable that controls the color animation progress for the Value Bar.
	* This is called by a timer.
	 */
	EActiveTimerReturnType UpdateValueChangedAnim(double InCurrentTime, float InDeltaTime);

	/** Returns the universe ID in Text form to display it in the UI */
	FText GetIDLabel() const;

	/** Returns the fill color for the ValueBar */
	FSlateColor GetBackgroundColor() const;

	/** Used to stop the animation timer once the animation is completed */
	TWeakPtr<FActiveTimerHandle> AnimationTimerHandle;

	/**
	 * Used to animate the color when the value changes.
	 * 0..1 range: 1 = value has just changed, 0 = standard color
	 */
	float NewValueFreshness = 0.0f;

	/** How long it takes to become standard color again after a new value is set */
	const float NewValueChangedAnimDuration = 0.8f;

	/** Color of the ID label */
	const FLinearColor IDColor = FLinearColor(1.0f, 1.0f, 1.0f, 0.6f);

	/** Color of the Value label */
	const FLinearColor ValueColor = FLinearColor(1.0f, 1.0f, 1.0f, 0.9f);
};

