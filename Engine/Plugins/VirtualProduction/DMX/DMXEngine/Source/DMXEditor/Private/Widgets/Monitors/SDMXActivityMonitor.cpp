// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Monitors/SDMXActivityMonitor.h"

#include "Algo/RemoveIf.h"
#include "Containers/UnrealString.h"
#include "DMXEditorSettings.h"
#include "DMXEditorStyle.h"
#include "DMXProtocolTypes.h"
#include "DMXSubsystem.h"
#include "IO/DMXInputPort.h"
#include "IO/DMXOutputPort.h"
#include "IO/DMXPortManager.h"
#include "IO/DMXRawListener.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Monitors/SDMXActivityInUniverse.h"
#include "Widgets/Monitors/SDMXMonitorSourceSelector.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SDMXActivityMonitor"

SDMXActivityMonitor::SDMXActivityMonitor()
	: MinUniverseID(1)
	, MaxUniverseID(100)
	, AnalyticsProvider("ActivityMonitor")
{}

SDMXActivityMonitor::~SDMXActivityMonitor()
{
	check(SourceSelector.IsValid());

	for (const TSharedRef<FDMXRawListener>& Input : DMXListeners)
	{
		Input->Stop();
	}
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SDMXActivityMonitor::Construct(const FArguments& InArgs)
{
	SetCanTick(true);

	const TSharedRef<SScrollBar> HorizontalScrollBar = SNew(SScrollBar)
		.Orientation(Orient_Horizontal);

	const TSharedRef<SScrollBar> VerticalScrollBar = SNew(SScrollBar)
		.Orientation(Orient_Vertical);

	ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			.AutoHeight()
			[
				SNew(SWrapBox)
				.InnerSlotPadding(FVector2D(35.0f, 10.0f))
				.UseAllottedSize(true)

				// Source Selector
				+ SWrapBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SAssignNew(SourceSelector, SDMXMonitorSourceSelector)
					.OnSourceSelected(this, &SDMXActivityMonitor::OnSourceSelected)
				]

				// Min & Max Universe ID
				+ SWrapBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
						.Text(LOCTEXT("LocalUniversesLabel", "Monitored Local Universes"))
					] 

					+ SHorizontalBox::Slot()
					.Padding(10.0f, 0.0f, 0.0f, 0.0f)
					.AutoWidth()
					[
						SNew(SBox)
						.MinDesiredWidth(40.f)
						[
							SAssignNew(MinUniverseIDEditableTextBox, SEditableTextBox)
							.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
							.Text(FText::FromString(FString::FromInt(MinUniverseID)))
							.OnTextCommitted(this, &SDMXActivityMonitor::OnMinUniverseIDValueCommitted)
						]
					]

					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(5.0f, 0.0f, 0.0f, 0.0f)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ToLabel", "to"))
					]

					+ SHorizontalBox::Slot()
					.Padding(5.0f, 0.0f, 0.0f, 0.0f)
					.AutoWidth()
					[	
						SNew(SBox)
						.MinDesiredWidth(40.f)
						[
							SAssignNew(MaxUniverseIDEditableTextBox, SEditableTextBox)
							.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
							.Text(FText::FromString(FString::FromInt(MaxUniverseID)))
							.OnTextCommitted(this, &SDMXActivityMonitor::OnMaxUniverseIDValueCommitted)
						]
					]
				]

				// Clear button
				+ SWrapBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.Text(LOCTEXT("ClearTextLabel", "Clear DMX Buffers"))
					.OnClicked(this, &SDMXActivityMonitor::OnClearButtonClicked)
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(3.0f)
			[
				SNew(SSeparator)
				.Orientation(Orient_Horizontal)
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Top)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("LocalUniverseLabel", "Universe"))
					.Font(FDMXEditorStyle::Get().GetFontStyle("DMXEditor.Font.InputUniverseHeader"))
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(12.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Top)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ChannelValueLabel", "Addr / Value"))
					.Font(FDMXEditorStyle::Get().GetFontStyle("DMXEditor.Font.InputUniverseHeader"))
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			[
				SNew(SSeparator)
				.Orientation(Orient_Horizontal)
			]

			+ SVerticalBox::Slot()
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				[
					SNew(SScrollBox)
					.Orientation(EOrientation::Orient_Horizontal)
					.ExternalScrollbar(HorizontalScrollBar)

					+ SScrollBox::Slot()
					.FillSize(1.f)
					[
						SNew(SScrollBox)
						.Orientation(EOrientation::Orient_Vertical)
						.ExternalScrollbar(VerticalScrollBar)

						+ SScrollBox::Slot()				
						[
							SAssignNew(UniverseList, SListView<TSharedPtr<SDMXActivityInUniverse>>)
							.ListItemsSource(&UniverseListSource)
							.Visibility(EVisibility::Visible)
							.OnGenerateRow(this, &SDMXActivityMonitor::OnGenerateUniverseRow)
						]
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					VerticalScrollBar
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				HorizontalScrollBar
			]
		];

	LoadMonitorSettings();

	UpdateListenerRegistration();

	FDMXPortManager::Get().OnPortsChanged.AddSP(this, &SDMXActivityMonitor::OnPortsChanged);
	FDMXPortManager::Get().GetOnBuffersCleared().AddSP(this, &SDMXActivityMonitor::OnPortBuffersCleared);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


void SDMXActivityMonitor::ResizeDataMapToUniverseRange()
{
	const TMap<int32, TArray<uint8, TFixedAllocator<DMX_UNIVERSE_SIZE>>> CachedUniverseToDataMap = UniverseToDataMap;
	
	for (const TTuple<int32, TArray<uint8, TFixedAllocator<DMX_UNIVERSE_SIZE>>>& UniverseToDataKvp : CachedUniverseToDataMap)
	{
		if (UniverseToDataKvp.Key < MinUniverseID || UniverseToDataKvp.Key > MaxUniverseID)
		{
			UniverseToDataMap.Remove(UniverseToDataKvp.Key);
		}
	}
}

void SDMXActivityMonitor::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	check(SourceSelector.IsValid());
	// Tick universe listeners with latest signals on tick

	FDMXSignalSharedPtr DMXSignal;
	for (const TSharedRef<FDMXRawListener>& Input : DMXListeners)
	{
		int32 LocalUniverseID;
		while (Input->DequeueSignal(this, DMXSignal, LocalUniverseID))
		{
			if (LocalUniverseID >= MinUniverseID && LocalUniverseID <= MaxUniverseID)
			{
				UniverseToDataMap.FindOrAdd(LocalUniverseID) = DMXSignal->ChannelData;
			}
		}
	}

	for (const TTuple<int32, TArray<uint8, TFixedAllocator<DMX_UNIVERSE_SIZE>>>& UniverseToDataKvp : UniverseToDataMap)
	{
		const TSharedRef<SDMXActivityInUniverse>& ActivityWidget = GetOrCreateActivityWidget(UniverseToDataKvp.Key);
		ActivityWidget->VisualizeBuffer(UniverseToDataKvp.Value);
	}
}
TSharedRef<ITableRow> SDMXActivityMonitor::OnGenerateUniverseRow(TSharedPtr<SDMXActivityInUniverse> ActivityWidget, const TSharedRef<STableViewBase>& OwnerTable)
{
	return
		SNew(STableRow<TSharedPtr<SDMXActivityInUniverse>>, OwnerTable)
		[
			ActivityWidget.ToSharedRef()
		];
}

void SDMXActivityMonitor::LoadMonitorSettings()
{
	check(SourceSelector.IsValid());
	check(MinUniverseIDEditableTextBox.IsValid());
	check(MaxUniverseIDEditableTextBox.IsValid());

	// Restore from config
	UDMXEditorSettings* DMXEditorSettings = GetMutableDefault<UDMXEditorSettings>();

	FText MinUniverseIDText = FText::FromString(FString::FromInt(DMXEditorSettings->ActivityMonitorMinUniverseID));
	MinUniverseIDEditableTextBox->SetText(MinUniverseIDText);
	OnMinUniverseIDValueCommitted(MinUniverseIDText, ETextCommit::Default);

	FText MaxUniverseIDText = FText::FromString(FString::FromInt(DMXEditorSettings->ActivityMonitorMaxUniverseID));
	MaxUniverseIDEditableTextBox->SetText(MaxUniverseIDText);
	OnMaxUniverseIDValueCommitted(MaxUniverseIDText, ETextCommit::Default);

	// Setting the values on the source selector widget will trigger updates of the actual values
	SourceSelector->SetMonitorAllPorts(DMXEditorSettings->ChannelsMonitorSource.bMonitorAllPorts);
	SourceSelector->SetMonitorInputPorts(DMXEditorSettings->ChannelsMonitorSource.bMonitorInputPorts);
	SourceSelector->SetMonitoredPortGuid(DMXEditorSettings->ChannelsMonitorSource.MonitoredPortGuid);

	ClearDisplay();
}

void SDMXActivityMonitor::SaveMonitorSettings() const
{
	check(SourceSelector.IsValid());

	// Create a new Source Descriptor from current values
	FDMXMonitorSourceDescriptor MonitorSourceDescriptor;
	MonitorSourceDescriptor.bMonitorAllPorts = SourceSelector->IsMonitorAllPorts();
	MonitorSourceDescriptor.bMonitorInputPorts = SourceSelector->IsMonitorInputPorts();
	MonitorSourceDescriptor.MonitoredPortGuid = SourceSelector->GetMonitoredPortGuid();

	// Write to config
	UDMXEditorSettings* DMXEditorSettings = GetMutableDefault<UDMXEditorSettings>();
	check(DMXEditorSettings);

	DMXEditorSettings->ChannelsMonitorSource = MonitorSourceDescriptor;
	DMXEditorSettings->ActivityMonitorMinUniverseID = MinUniverseID;
	DMXEditorSettings->ActivityMonitorMaxUniverseID = MaxUniverseID;
	DMXEditorSettings->SaveConfig();
}

TSharedRef<SDMXActivityInUniverse> SDMXActivityMonitor::GetOrCreateActivityWidget(int32 UniverseID)
{
	TSharedPtr<SDMXActivityInUniverse>* BufferViewPtr = UniverseListSource.FindByPredicate([&](const TSharedPtr<SDMXActivityInUniverse>& BufferViewCandidate) {
			return BufferViewCandidate->GetUniverseID() == UniverseID;
		});

	TSharedPtr<SDMXActivityInUniverse> BufferView = BufferViewPtr ? *BufferViewPtr : nullptr;
	if (!BufferView.IsValid())
	{
		BufferView =
			SNew(SDMXActivityInUniverse)
			.UniverseID(UniverseID);

		UniverseListSource.Add(BufferView);

		// Sort the universe list source ascending
		UniverseListSource.Sort([](const TSharedPtr<SDMXActivityInUniverse>& ViewA, const TSharedPtr<SDMXActivityInUniverse>& ViewB) {
			return ViewA->GetUniverseID() < ViewB->GetUniverseID();
			});

		UniverseList->RequestListRefresh();
	}

	check(BufferView.IsValid());
	return BufferView.ToSharedRef();
}

void SDMXActivityMonitor::OnMinUniverseIDValueCommitted(const FText& InNewText, ETextCommit::Type CommitType)
{
	check(MinUniverseIDEditableTextBox.IsValid());
	check(MaxUniverseIDEditableTextBox.IsValid());

	if (!InNewText.IsNumeric())
	{	
		// If the entered text isn't numeric, restore the previous Value
		MinUniverseIDEditableTextBox->SetText(FText::FromString(FString::FromInt(MinUniverseID)));
	}
	else
	{
		int32 NewValue;
		if (LexTryParseString<int32>(NewValue, *InNewText.ToString()))
		{
			if (NewValue != MinUniverseID)
			{
				if (NewValue >= 0 && NewValue <= MaxUniverseID)
				{
					MinUniverseID = NewValue;
				}
				else if (NewValue < 0)
				{
					// Set to zero if smaller than zero
					MinUniverseIDEditableTextBox->SetText(FText::FromString(FString::FromInt(0)));
				}
				else if (NewValue > MaxUniverseID)
				{
					// Set to max universe ID if bigger than MaxUniverseID
					MinUniverseIDEditableTextBox->SetText(FText::FromString(FString::FromInt(MaxUniverseID)));
				}

				ClearDisplay();
				SaveMonitorSettings();
			}
		}
	}

	ResizeDataMapToUniverseRange();
}

void SDMXActivityMonitor::OnMaxUniverseIDValueCommitted(const FText& InNewText, ETextCommit::Type CommitType)
{
	check(MinUniverseIDEditableTextBox.IsValid());
	check(MaxUniverseIDEditableTextBox.IsValid());

	if (!InNewText.IsNumeric())
	{	
		// If the entered text isn't numeric, restore the previous Value
		MaxUniverseIDEditableTextBox->SetText(FText::FromString(FString::FromInt(MaxUniverseID)));
	}
	else
	{
		int32 NewValue = FCString::Atoi(*InNewText.ToString());

		if (NewValue != MaxUniverseID)
		{
			if (NewValue >= 0 && NewValue >= MinUniverseID)
			{
				MaxUniverseID = NewValue;
			}
			else if (NewValue < MinUniverseID)
			{
				// Set to MinUniverseID
				MinUniverseIDEditableTextBox->SetText(FText::FromString(FString::FromInt(MinUniverseID)));
			}

			ClearDisplay();
			SaveMonitorSettings();
		}
	}

	ResizeDataMapToUniverseRange();
}

void SDMXActivityMonitor::OnSourceSelected()
{
	ClearDisplay();

	SaveMonitorSettings();

	UpdateListenerRegistration();
}

void SDMXActivityMonitor::OnPortsChanged()
{
	ClearDisplay();

	UpdateListenerRegistration();
}

void SDMXActivityMonitor::OnPortBuffersCleared()
{
	ClearDisplay();
}

FReply SDMXActivityMonitor::OnClearButtonClicked()
{
	UDMXSubsystem::ClearDMXBuffers();

	return FReply::Handled();
}

void SDMXActivityMonitor::ClearDisplay()
{
	check(UniverseList.IsValid());

	UniverseToDataMap.Reset();

	UniverseListSource.Reset();
	UniverseList->RequestListRefresh();
}

void SDMXActivityMonitor::UpdateListenerRegistration()
{
	check(SourceSelector.IsValid());

	// Stop listening to previous ports
	for (const TSharedRef<FDMXRawListener>& Input : DMXListeners)
	{
		Input->Stop();
	}
	DMXListeners.Reset();

	if (SourceSelector->IsMonitorInputPorts())
	{
		// Listen to selected ports
		for (const FDMXInputPortSharedRef& InputPort : SourceSelector->GetSelectedInputPorts())
		{
			TSharedRef<FDMXRawListener> NewListener = MakeShared<FDMXRawListener>(InputPort);
			DMXListeners.Add(NewListener);

			NewListener->Start();
		}
	}
	else
	{
		for (const FDMXOutputPortSharedRef& OutputPort : SourceSelector->GetSelectedOutputPorts())
		{
			// Monitor outputs 
			TSharedRef<FDMXRawListener> NewListener = MakeShared<FDMXRawListener>(OutputPort);
			DMXListeners.Add(NewListener);

			NewListener->Start();
		}
	}
}

#undef LOCTEXT_NAMESPACE
