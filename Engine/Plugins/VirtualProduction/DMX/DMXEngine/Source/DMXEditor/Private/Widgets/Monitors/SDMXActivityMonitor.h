// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Analytics/DMXEditorToolAnalyticsProvider.h"
#include "DMXProtocolConstants.h"
#include "DMXProtocolCommon.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FDMXRawListener;
class FDMXSignal;
class ITableRow;
class SDMXActivityInUniverse;
class SDMXMonitorSourceSelector;
class SEditableTextBox;
template <typename ItemType> class SListView;
class STableViewBase;
	

/**
 * A Monitor for DMX activity in a range of DMX Universes
 */
class SDMXActivityMonitor
	: public SCompoundWidget
{
public:
	SDMXActivityMonitor();

	virtual ~SDMXActivityMonitor();

	SLATE_BEGIN_ARGS(SDMXActivityMonitor)
	{}

	SLATE_END_ARGS()

	/** Constructs the widget */
	void Construct(const FArguments& InArgs);

private:
	/** Clears all DMX Buffers */
	void ClearAllDMXBuffers();

	/** Resizes Latest Data to the Min and Max Universe Range */
	void ResizeDataMapToUniverseRange();

	/** Map of latest data */
	TMap<int32, TArray<uint8, TFixedAllocator<DMX_UNIVERSE_SIZE>>> UniverseToDataMap;

protected:
	//~ Begin SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~End of SWidget interface

private:
	/** Initialize values from DMX Editor Settings */
	void LoadMonitorSettings();

	/** Saves settings of the monitor in config */
	void SaveMonitorSettings() const;

	/** Returns a view for specified universe ID, creates one if it doesn't exist yet */
	TSharedRef<SDMXActivityInUniverse> GetOrCreateActivityWidget(int32 UniverseID);

	/** Called when a Min Universe ID value was commited */
	void OnMinUniverseIDValueCommitted(const FText& InNewText, ETextCommit::Type CommitType);

	/** Called when a Max Universe ID value was commited */
	void OnMaxUniverseIDValueCommitted(const FText& InNewText, ETextCommit::Type CommitType);

	/** Called when a Source was selected in the Monitor Source Selector */
	void OnSourceSelected();

	/** Called when ports changed */
	void OnPortsChanged();

	/** Called when port buffers were cleared */
	void OnPortBuffersCleared();

	/** Called when the clear ui values button was clicked */
	FReply OnClearButtonClicked();

	/** Remove all displayed  values on the universes monitor */
	void ClearDisplay();

	/** Reregisters self as Listener to the ports active */
	void UpdateListenerRegistration();

	/** The min universe ID to recieve */
	int32 MinUniverseID;

	/** The max universe ID to recieve */
	int32 MaxUniverseID;

private:
	/** Called by SListView to generate custom list table row */
	TSharedRef<ITableRow> OnGenerateUniverseRow(TSharedPtr<SDMXActivityInUniverse> Item, const TSharedRef<STableViewBase>& OwnerTable);

	/** The DMX inputs of the activity monitor */
	TSet<TSharedRef<FDMXRawListener>> DMXListeners;

	/** ListView for universe counts */
	TSharedPtr<SListView<TSharedPtr<SDMXActivityInUniverse>>> UniverseList;

	/** Universe Monitors being displayed */
	TArray<TSharedPtr<SDMXActivityInUniverse>> UniverseListSource;

	/** Selector for the source to monitor */
	TSharedPtr<SDMXMonitorSourceSelector> SourceSelector;

	/** Text block to edit the Min Universe ID */
	TSharedPtr<SEditableTextBox> MinUniverseIDEditableTextBox;

	/** Text block to edit the Max Universe ID */
	TSharedPtr<SEditableTextBox> MaxUniverseIDEditableTextBox;

	/** The analytics provider for this tool */
	UE::DMX::FDMXEditorToolAnalyticsProvider AnalyticsProvider;
};
