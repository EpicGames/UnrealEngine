// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Monitors/SDMXChannelsMonitor.h"

#include "Containers/UnrealString.h"
#include "DMXEditorSettings.h"
#include "DMXProtocolCommon.h"
#include "DMXSubsystem.h"
#include "IO/DMXInputPort.h"
#include "IO/DMXOutputPort.h"
#include "IO/DMXPortManager.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h" 
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Monitors/SDMXMonitorSourceSelector.h"
#include "Widgets/SDMXChannel.h"

#define LOCTEXT_NAMESPACE "SDMXChannelsMonitor"

SDMXChannelsMonitor::SDMXChannelsMonitor()
	: UniverseID(1)
	, AnalyticsProvider("ChannelsMonitor")
{}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SDMXChannelsMonitor::Construct(const FArguments& InArgs)
{
	SetCanTick(true);

	ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(15.0f)
			.HAlign(HAlign_Fill)
			.AutoHeight()
			[
				SNew(SWrapBox)
				.InnerSlotPadding(FVector2D(36.0f, 10.0f))
				.UseAllottedSize(true)
		
				// Source Selector
				+ SWrapBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SAssignNew(SourceSelector, SDMXMonitorSourceSelector)
					.OnSourceSelected(this, &SDMXChannelsMonitor::OnSourceSelected)
				]

				// Universe ID
				+ SWrapBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
						.Text(LOCTEXT("UniverseIDLabel", "Local Universe"))
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(10.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(SBox)
						.MinDesiredWidth(40.f)
						[
							SAssignNew(UniverseIDEditableTextBox, SEditableTextBox)
							.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
							.Text(FText::FromString(LexToString(UniverseID)))
							.OnTextCommitted(this, &SDMXChannelsMonitor::OnUniverseIDValueCommitted)
						]
					]
				]

				// Clear button
				+ SWrapBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.OnClicked(this, &SDMXChannelsMonitor::OnClearButtonClicked)
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
						.Text(LOCTEXT("ClearTextLabel", "Clear DMX Buffers"))
					]
				]
			]

			// Separator
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(3.0f)
			[
				SNew(SSeparator)
				.Orientation(Orient_Horizontal)
			]

			// Channel Widgets
			+ SVerticalBox::Slot()
			[
				SNew(SScrollBox)
				.Orientation(Orient_Vertical)
				.ScrollBarAlwaysVisible(false)

				+ SScrollBox::Slot()
				.HAlign(HAlign_Fill)
				[
					SAssignNew(ChannelValuesBox, SWrapBox)
					.UseAllottedSize(true)
					.InnerSlotPadding(FVector2D(1.0f, 1.0f))
				]
			]
		];

	// Init from config
	LoadMonitorSettings();

	// Init UI
	CreateChannelValueWidgets();
	ZeroChannelValues();

	FDMXPortManager::Get().GetOnBuffersCleared().AddSP(this, &SDMXChannelsMonitor::OnPortBuffersCleared);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SDMXChannelsMonitor::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	FDMXSignalSharedPtr Signal;

	if (SourceSelector->IsMonitorInputPorts())
	{
		// The widget returns the right arrays, also output ports that loopback when input ports are selected.
		for (const TSharedRef<FDMXInputPort>& InputPort : SourceSelector->GetSelectedInputPorts())
		{
			if (InputPort->GameThreadGetDMXSignal(UniverseID, Signal))
			{
				break;
			}
		}
	}
	else
	{
		for (const TSharedRef<FDMXOutputPort>& OutputPort : SourceSelector->GetSelectedOutputPorts())
		{
			constexpr bool bEvenIfNotLoopbackToEngine = true;
			if (OutputPort->GameThreadGetDMXSignal(UniverseID, Signal, bEvenIfNotLoopbackToEngine))
			{
				break;
			}
		}
	}

	if (Signal.IsValid())
	{
		SetChannelValues(Signal->ChannelData);
	}
}

void SDMXChannelsMonitor::CreateChannelValueWidgets()
{
	if (!ChannelValuesBox.IsValid())
	{
		return;
	}

	for (uint32 ChannelID = 1; ChannelID <= DMX_UNIVERSE_SIZE; ++ChannelID)
	{
		TSharedPtr<SDMXChannel> ChannelValueWidget;

		ChannelValuesBox->AddSlot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SAssignNew(ChannelValueWidget, SDMXChannel)
				.ChannelID(ChannelID)
				.Value(0)
			];

		ChannelValueWidgets.Add(ChannelValueWidget);
	}
}

void SDMXChannelsMonitor::LoadMonitorSettings()
{
	// Restore from config
	UDMXEditorSettings* DMXEditorSettings = GetMutableDefault<UDMXEditorSettings>();

	FText UniverseIDText = FText::FromString(FString::FromInt(DMXEditorSettings->ChannelsMonitorUniverseID));
	UniverseIDEditableTextBox->SetText(UniverseIDText);
	OnUniverseIDValueCommitted(UniverseIDText, ETextCommit::Default);

	// Setting the values on the source selector widget will trigger updates of the actual values
	SourceSelector->SetMonitorAllPorts(DMXEditorSettings->ChannelsMonitorSource.bMonitorAllPorts);
	SourceSelector->SetMonitorInputPorts(DMXEditorSettings->ChannelsMonitorSource.bMonitorInputPorts);
	SourceSelector->SetMonitoredPortGuid(DMXEditorSettings->ChannelsMonitorSource.MonitoredPortGuid);
}

void SDMXChannelsMonitor::SaveMonitorSettings() const
{
	// Ignore calls during initalization
	if (!SourceSelector.IsValid())
	{
		return;
	}

	// Create a new Source Descriptor from current values
	FDMXMonitorSourceDescriptor MonitorSourceDescriptor;
	MonitorSourceDescriptor.bMonitorAllPorts = SourceSelector->IsMonitorAllPorts();
	MonitorSourceDescriptor.bMonitorInputPorts = SourceSelector->IsMonitorInputPorts();
	MonitorSourceDescriptor.MonitoredPortGuid = SourceSelector->GetMonitoredPortGuid();

	// Write to config
	UDMXEditorSettings* DMXEditorSettings = GetMutableDefault<UDMXEditorSettings>();
	check(DMXEditorSettings);

	DMXEditorSettings->ChannelsMonitorSource = MonitorSourceDescriptor;
	DMXEditorSettings->ChannelsMonitorUniverseID = UniverseID;
	DMXEditorSettings->SaveConfig();
}

void SDMXChannelsMonitor::OnSourceSelected()
{
	ZeroChannelValues();
	SaveMonitorSettings();
}

void SDMXChannelsMonitor::ZeroChannelValues()
{
	for (int32 ChannelID = 0; ChannelID < ChannelValueWidgets.Num(); ++ChannelID)
	{
		ChannelValueWidgets[ChannelID]->SetValue(0);
	}
}

void SDMXChannelsMonitor::SetChannelValues(const TArray<uint8>& Buffer)
{
	check(Buffer.Num() <= ChannelValueWidgets.Num());

	for (int32 ChannelID = 0; ChannelID < Buffer.Num(); ++ChannelID)
	{
		ChannelValueWidgets[ChannelID]->SetValue(Buffer[ChannelID]);
	}
}

void SDMXChannelsMonitor::OnUniverseIDValueCommitted(const FText& InNewText, ETextCommit::Type CommitType)
{
	// If the entered text isn't numeric, restore the previous Value
	if (!InNewText.IsNumeric())
	{
		UniverseIDEditableTextBox->SetText(FText::FromString(LexToString(UniverseID)));
	}
	else
	{
		int32 NewValue = FCString::Atoi(*InNewText.ToString());
		 
		if (NewValue != UniverseID)
		{
			// Only accept values >= 0
			if (NewValue >= 0)
			{
				UniverseID = NewValue;

				SaveMonitorSettings();
			}
			else
			{
				// Fix the Text and Set anew
				UniverseIDEditableTextBox->SetText(FText::FromString(FString::FromInt(0)));
			}
		}

		ZeroChannelValues();
	}
}

FReply SDMXChannelsMonitor::OnClearButtonClicked()
{
	UDMXSubsystem::ClearDMXBuffers();

	return FReply::Handled();
}

void SDMXChannelsMonitor::OnPortBuffersCleared()
{
	ZeroChannelValues();
}

#undef LOCTEXT_NAMESPACE
