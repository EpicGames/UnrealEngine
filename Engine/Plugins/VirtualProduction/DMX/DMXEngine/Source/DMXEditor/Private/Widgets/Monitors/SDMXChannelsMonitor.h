// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Analytics/DMXEditorToolAnalyticsProvider.h"
#include "DMXProtocolCommon.h"
#include "HAL/CriticalSection.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SCheckBox;
class SDMXMonitorSourceSelector;
class SDMXPortSelector;
class SEditableTextBox;
class SHorizontalBox;


/**
 * DMX Widget to monitor all the channels in a DMX Universe
 */
class SDMXChannelsMonitor
	: public SCompoundWidget
{
public:
	SDMXChannelsMonitor();

	SLATE_BEGIN_ARGS(SDMXChannelsMonitor)
	{}

	SLATE_END_ARGS()

	/** Constructs the widget */
	void Construct(const FArguments& InArgs);

protected:
	// ~Begin SWidget Interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	// ~End SWidget Interface

private:
	/** Creates the channel value widgets */
	void CreateChannelValueWidgets();

	/** Initialize values from DMX Editor Settings */
	void LoadMonitorSettings();

	/** Saves settings of the monitor in config */
	void SaveMonitorSettings() const;

	/** Set the channel widgets to 0 */
	void ZeroChannelValues();

	/** Set the channel widgets to the values of the buffer */
	void SetChannelValues(const TArray<uint8>& Buffer);

	/** Called when a source was selected in the Source Selector */
	void OnSourceSelected();

	/** Called when the universe ID was committed */
	void OnUniverseIDValueCommitted(const FText& InNewText, ETextCommit::Type CommitType);
	
	/** Called when the clear bButton was clicked */
	FReply OnClearButtonClicked();

	/** Called when port buffers were cleared */
	void OnPortBuffersCleared();

	/** Horizontal box that contains the Source Selector and its Label */
	TSharedPtr<SHorizontalBox> SourceSelectorBox;

	/** Selector for the source to monitor */
	TSharedPtr<SDMXMonitorSourceSelector> SourceSelector;

	/** Container widget for all the channels' values */
	TSharedPtr<class SWrapBox> ChannelValuesBox;
	
	/** Widgets for individual channels, length should be same as number of channels in a universe */
	TArray<TSharedPtr<class SDMXChannel>, TFixedAllocator<DMX_UNIVERSE_SIZE>> ChannelValueWidgets;

	/** Text block to edit the Universe ID */
	TSharedPtr<SEditableTextBox> UniverseIDEditableTextBox;

	/** Universe ID value computed using Net, Subnet and Universe values */
	int32 UniverseID;

	/** The analytics provider for this tool */
	UE::DMX::FDMXEditorToolAnalyticsProvider AnalyticsProvider;
};
