// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXConflictMonitor.h"

#include "Algo/AnyOf.h"
#include "Algo/Find.h"
#include "Algo/Transform.h"
#include "Analytics/DMXEditorToolAnalyticsProvider.h"
#include "Commands/DMXConflictMonitorCommands.h"
#include "DMXConflictMonitorActiveObjectItem.h"
#include "DMXConflictMonitorConflictModel.h"
#include "DMXEditorLog.h"
#include "DMXEditorSettings.h"
#include "DMXEditorStyle.h"
#include "DMXStats.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandList.h"
#include "Internationalization/Regex.h"
#include "IO/DMXConflictMonitor.h"
#include "IO/DMXPortManager.h"
#include "SDMXConflictMonitorActiveObjectRow.h"
#include "SDMXConflictMonitorToolbar.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Text/SRichTextBlock.h"

#define LOCTEXT_NAMESPACE "SDMXConflictMonitor"

DECLARE_CYCLE_STAT(TEXT("DMX Conflict Monitor User Interface"), STAT_DMXConflictMonitorUI, STATGROUP_DMX);

namespace UE::DMX
{
	const FName SDMXConflictMonitor::FActiveObjectCollumnID::ObjectName = TEXT("ObjectName");
	const FName SDMXConflictMonitor::FActiveObjectCollumnID::OpenAsset = TEXT("OpenAsset");
	const FName SDMXConflictMonitor::FActiveObjectCollumnID::ShowInContentBrowser = TEXT("ShowInContentBrowser");

	SDMXConflictMonitor::SDMXConflictMonitor()
		: StatusInfo(EDMXConflictMonitorStatusInfo::Idle)
		, AnalyticsProvider("ConflictMonitor")
	{}

	void SDMXConflictMonitor::Construct(const FArguments& InArgs)
	{
		SetupCommandList();
		SetCanTick(false);

		ChildSlot
		[
			SNew(SVerticalBox)

			// Toolbar
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4.f)
			[
				SNew(SDMXConflictMonitorToolbar, CommandList.ToSharedRef())
				.StatusInfo_Lambda([this]()
					{
						return StatusInfo;
					})
				.TimeGameThread_Lambda([this]()
					{
						return TimeGameThread;
					})
				.OnDepthChanged_Lambda([this]()
					{
						Refresh();
					})
			]

			+ SVerticalBox::Slot()
			.FillHeight(1.f)
			.Padding(16.f)
			[
				SNew(SSplitter)
				.Orientation(EOrientation::Orient_Horizontal)

				// Log
				+ SSplitter::Slot()
				.Value(0.62f)
				.MinSize(10.f)
				[
					SNew(SScrollBox)
					.Orientation(EOrientation::Orient_Vertical)
					
					+ SScrollBox::Slot()
					.AutoSize()
					[
						SNew(SBorder)
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Fill)
						.BorderImage(FAppStyle::GetBrush("NoBorder"))
						[
							SAssignNew(LogTextBlock, SRichTextBlock)
							.Visibility(EVisibility::HitTestInvisible)
							.AutoWrapText(true)
							.TextStyle(FAppStyle::Get(), "MessageLog")
							.DecoratorStyleSet(&FDMXEditorStyle::Get())
						]
					]
				]

				// Active objects
				+ SSplitter::Slot()
				.Value(0.38f)
				.MinSize(10.f)
				[
					SNew(SScrollBox)
					.Orientation(EOrientation::Orient_Vertical)
					
					+ SScrollBox::Slot()
					.FillSize(1.f)
					[
						SNew(SBorder)
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Fill)
						.BorderImage(FAppStyle::GetBrush("NoBorder"))
						[
							SAssignNew(ActiveObjectList, SListView<TSharedPtr<FDMXConflictMonitorActiveObjectItem>>)
							.HeaderRow(GenerateActiveObjectHeaderRow())
							.ListItemsSource(&ActiveObjectListSource)
							.OnGenerateRow(this, &SDMXConflictMonitor::OnGenerateActiveObjectRow)
						]
					]
				]
			]
		];

		Refresh();

		const UDMXEditorSettings* EditorSettings = GetDefault<UDMXEditorSettings>();
		if (EditorSettings->ConflictMonitorSettings.bRunWhenOpened)
		{
			Play();
		}
	}

	void SDMXConflictMonitor::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		SCOPE_CYCLE_COUNTER(STAT_DMXConflictMonitorUI);

		const double StartTime = FPlatformTime::Seconds();

		if (FSlateApplication::Get().AnyMenusVisible())
		{
			return;
		}

		const FDMXConflictMonitor* ConflictMonitor = FDMXConflictMonitor::Get();
		if (!ConflictMonitor)
		{
			return;
		}

		// Only refresh when data or conflicts changed. This is more performant and leaves the widgets interactable.
		const  TArray<TSharedRef<FDMXMonitoredOutboundDMXData>> NewOutboundData = ConflictMonitor->GetMonitoredOutboundData();
		const bool bDataChanged =
			NewOutboundData.Num() != CachedOutboundData.Num() ||
			Algo::AnyOf(NewOutboundData, [this](const TSharedRef<FDMXMonitoredOutboundDMXData>& Data)
				{
					return Algo::NoneOf(CachedOutboundData, [&Data](const TSharedRef<FDMXMonitoredOutboundDMXData>& Other)
						{
							return Other->Trace == Data->Trace;
						});
				});

		const TMap<FName, TArray<TSharedRef<FDMXMonitoredOutboundDMXData>>> NewOutboundConflicts = ConflictMonitor->GetOutboundConflictsSynchronous();
		const bool bConflictsChanged = !CachedOutboundConflicts.OrderIndependentCompareEqual(NewOutboundConflicts);

		const bool bLeftMouseButtonDown = FSlateApplication::Get().GetPressedMouseButtons().Contains(EKeys::LeftMouseButton);

		if ((bDataChanged || bConflictsChanged) && !bLeftMouseButtonDown)
		{
			CachedOutboundData = NewOutboundData;
			CachedOutboundConflicts = NewOutboundConflicts;
			Refresh();
		}

		UpdateStatusInfo();

		const double ConflictMonitorTimeGameThread = ConflictMonitor->GetTimeGameThread();

		const double EndTime = FPlatformTime::Seconds();
		TimeGameThread = (EndTime - StartTime) * 1000.0 + ConflictMonitorTimeGameThread;
	}

	FReply SDMXConflictMonitor::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		return FReply::Handled();
	}

	TSharedRef<SHeaderRow> SDMXConflictMonitor::GenerateActiveObjectHeaderRow()
	{
		const TSharedRef<SHeaderRow> HeaderRow =
			SNew(SHeaderRow)
			.Visibility_Lambda([this]()
				{
					const bool bIsActive =
						StatusInfo == EDMXConflictMonitorStatusInfo::OK ||
						StatusInfo == EDMXConflictMonitorStatusInfo::Conflict;

					return bIsActive ? EVisibility::Visible : EVisibility::Collapsed;
				});

		HeaderRow->AddColumn(
			SHeaderRow::FColumn::FArguments()
			.ColumnId(FActiveObjectCollumnID::ObjectName)
			.DefaultLabel(LOCTEXT("ActiveObjectLabel", "Objects sending DMX"))
			.FillWidth(1.f)
		);

		const FText AssetActionText = LOCTEXT("AssetActionLabel", "Asset Action");
		const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		const float AssetActionLabelSize = FontMeasureService->Measure(AssetActionText, FAppStyle::GetFontStyle("NormalText")).X + 8.f;

		HeaderRow->AddColumn(
			SHeaderRow::FColumn::FArguments()
			.ColumnId(FActiveObjectCollumnID::OpenAsset)
			.DefaultLabel(AssetActionText)
			.ManualWidth(AssetActionLabelSize)
		);

		const FText BrowseToText = LOCTEXT("BrowseToAssetLabel", "Browse To");
		const float BrowseToTextLabelSize = FontMeasureService->Measure(BrowseToText, FAppStyle::GetFontStyle("NormalText")).X + 16.f;

		HeaderRow->AddColumn(
			SHeaderRow::FColumn::FArguments()
			.ColumnId(FActiveObjectCollumnID::ShowInContentBrowser)
			.DefaultLabel(BrowseToText)
			.ManualWidth(BrowseToTextLabelSize)
		);

		return HeaderRow;
	}

	TSharedRef<ITableRow> SDMXConflictMonitor::OnGenerateActiveObjectRow(TSharedPtr<FDMXConflictMonitorActiveObjectItem> InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return 
			SNew(SDMXConflictMonitorActiveObjectRow, OwnerTable, InItem.ToSharedRef())
			.Visibility(EVisibility::SelfHitTestInvisible);
	}

	void SDMXConflictMonitor::Refresh()
	{
		const bool bStopped = !UserSession.IsValid();
		if (bStopped)
		{
			return;
		}

		// Fetch conflicts text
		TArray<TSharedPtr<FDMXConflictMonitorConflictModel>> NewModels;
		Algo::Transform(CachedOutboundConflicts, NewModels, [](const TPair<FName, TArray<TSharedRef<FDMXMonitoredOutboundDMXData>>>& Conflicts)
			{
				return MakeShared<FDMXConflictMonitorConflictModel>(Conflicts.Value);
			});

		FString NewLogText;
		for (const TSharedPtr<FDMXConflictMonitorConflictModel>& Model : NewModels)
		{
			constexpr bool bWithMarkup = true;
			NewLogText.Append(Model->GetConflictAsString(bWithMarkup));
			NewLogText.Append(TEXT("\n"));
		}

		// Update active DMX Objects
		TMap<FName, TSharedPtr<FDMXConflictMonitorActiveObjectItem>> ObjectNameToItemMap;
		for (const TSharedRef<FDMXMonitoredOutboundDMXData>& Data : CachedOutboundData)
		{
			const FString Trace = Data->Trace.ToString();
			
			const FRegexPattern ObjectPattern(TEXT("\\/([^\\/,]+)(?=(?:,|$))"));
			FRegexMatcher ObjectMatcher(ObjectPattern, Trace);

			const FRegexPattern ObjectPathPattern(TEXT("^([^,]+)"));
			FRegexMatcher ObjectPathMatcher(ObjectPathPattern, Trace);

			const FName ObjectName = ObjectMatcher.FindNext() ? *ObjectMatcher.GetCaptureGroup(1) : *Trace;
			const FName ObjectPath = ObjectPathMatcher.FindNext() ? *ObjectPathMatcher.GetCaptureGroup(1) : FName();
			const FSoftObjectPath SoftObjectPath(ObjectPath.ToString());

			ObjectNameToItemMap.FindOrAdd(ObjectName, MakeShared<FDMXConflictMonitorActiveObjectItem>(ObjectName, SoftObjectPath));
		}

		// Update texts
		Models = NewModels;
		LogTextBlock->SetText(FText::FromString(NewLogText));

		ActiveObjectListSource.Reset();
		ObjectNameToItemMap.GenerateValueArray(ActiveObjectListSource);
		ActiveObjectList->RequestListRefresh();

		// Auto-pause even if when data hasn't changed
		const UDMXEditorSettings* EditorSettings = GetDefault<UDMXEditorSettings>();
		if (!NewModels.IsEmpty() && IsScanning() && EditorSettings->ConflictMonitorSettings.bAutoPause)
		{
			Pause();
		}

		// Log conflicts (without markup)
		if (IsPrintingToLog())
		{
			for (const TSharedPtr<FDMXConflictMonitorConflictModel>& Model : NewModels)
			{
				UE_LOGF(LogDMXEditor, Log, "%ls", *Model->GetConflictAsString());
			}
		}
	}

	void SDMXConflictMonitor::SetupCommandList()
	{
		CommandList = MakeShared<FUICommandList>();

		CommandList->MapAction(FDMXConflictMonitorCommands::Get().StartScan,
			FExecuteAction::CreateSP(this, &SDMXConflictMonitor::Play),
			FCanExecuteAction::CreateLambda([this]
				{
					return !GetCanTick() && !bIsPaused;
				}),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateLambda([this]
				{
					return !GetCanTick() && !bIsPaused;
				})
		);

		CommandList->MapAction(FDMXConflictMonitorCommands::Get().PauseScan,
			FExecuteAction::CreateSP(this, &SDMXConflictMonitor::Pause),
			FCanExecuteAction::CreateLambda([this]
				{
					return GetCanTick();
				}),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateLambda([this]
				{
					return GetCanTick();
				})
		);

		CommandList->MapAction(FDMXConflictMonitorCommands::Get().ResumeScan,
			FExecuteAction::CreateSP(this, &SDMXConflictMonitor::Play),
			FCanExecuteAction::CreateLambda([this]
				{
					return !GetCanTick() && bIsPaused;
				}),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateLambda([this]
				{
					return !GetCanTick() && bIsPaused;
				})
		);

		CommandList->MapAction(FDMXConflictMonitorCommands::Get().StopScan,
			FExecuteAction::CreateSP(this, &SDMXConflictMonitor::Stop),
			FCanExecuteAction::CreateLambda([this]
				{
					return GetCanTick() || bIsPaused;
				})
		);

		CommandList->MapAction(FDMXConflictMonitorCommands::Get().ToggleAutoPause,
			FExecuteAction::CreateSP(this, &SDMXConflictMonitor::ToggleAutoPause),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SDMXConflictMonitor::IsAutoPause)
		);

		CommandList->MapAction(FDMXConflictMonitorCommands::Get().TogglePrintToLog,
			FExecuteAction::CreateSP(this, &SDMXConflictMonitor::TogglePrintToLog),
			FCanExecuteAction::CreateLambda([this]
				{
					return IsAutoPause();
				}),
			FIsActionChecked::CreateSP(this, &SDMXConflictMonitor::IsPrintingToLog)
		);

		CommandList->MapAction(FDMXConflictMonitorCommands::Get().ToggleRunWhenOpened,
			FExecuteAction::CreateSP(this, &SDMXConflictMonitor::ToggleRunWhenOpened),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SDMXConflictMonitor::IsRunWhenOpened)		
		);
	}

	void SDMXConflictMonitor::Play()
	{
		UserSession = FDMXConflictMonitor::Join("SDMXConflictMonitor");

		bIsPaused = false;
		SetCanTick(true);

		UpdateStatusInfo();
	}

	void SDMXConflictMonitor::Pause()
	{
		UserSession.Reset();

		bIsPaused = true;
		SetCanTick(false);

		UpdateStatusInfo();
	}

	void SDMXConflictMonitor::Stop()
	{
		UserSession.Reset();

		bIsPaused = false;
		SetCanTick(false);

		CachedOutboundData.Reset();
		CachedOutboundConflicts.Reset();
		Models.Reset();
		Refresh();

		UpdateStatusInfo();

		// Empty the active object list when stopped, so it's clear that it is no longer updated
		ActiveObjectListSource.Reset();
		ActiveObjectList->RequestListRefresh();
	}

	void SDMXConflictMonitor::SetAutoPause(bool bEnabled)
	{
		UDMXEditorSettings* EditorSettings = GetMutableDefault<UDMXEditorSettings>();
		EditorSettings->ConflictMonitorSettings.bAutoPause = bEnabled;

		EditorSettings->SaveConfig();
	}

	void SDMXConflictMonitor::ToggleAutoPause()
	{
		SetAutoPause(!IsAutoPause());
	}

	bool SDMXConflictMonitor::IsAutoPause() const
	{
		const UDMXEditorSettings* EditorSettings = GetDefault<UDMXEditorSettings>();
		return EditorSettings->ConflictMonitorSettings.bAutoPause;
	}

	void SDMXConflictMonitor::SetPrintToLog(bool bEnabled)
	{
		UDMXEditorSettings* EditorSettings = GetMutableDefault<UDMXEditorSettings>();
		EditorSettings->ConflictMonitorSettings.bPrintToLog = bEnabled;

		EditorSettings->SaveConfig();
	}

	void SDMXConflictMonitor::TogglePrintToLog()
	{
		SetPrintToLog(!IsPrintingToLog());
	}

	bool SDMXConflictMonitor::IsPrintingToLog() const
	{
		// Only available when auto-pause
		const UDMXEditorSettings* EditorSettings = GetDefault<UDMXEditorSettings>();
		return EditorSettings->ConflictMonitorSettings.bPrintToLog && IsAutoPause();
	}

	void SDMXConflictMonitor::SetRunWhenOpened(bool bEnabled)
	{
		UDMXEditorSettings* EditorSettings = GetMutableDefault<UDMXEditorSettings>();
		if (EditorSettings->ConflictMonitorSettings.bRunWhenOpened != bEnabled)
		{
			EditorSettings->ConflictMonitorSettings.bRunWhenOpened = bEnabled;
			EditorSettings->SaveConfig();
		}
	}
	
	void SDMXConflictMonitor::ToggleRunWhenOpened()
	{
		SetRunWhenOpened(!IsRunWhenOpened());
	}
	
	bool SDMXConflictMonitor::IsRunWhenOpened() const
	{
		const UDMXEditorSettings* EditorSettings = GetDefault<UDMXEditorSettings>();
		return EditorSettings->ConflictMonitorSettings.bRunWhenOpened;
	}

	bool SDMXConflictMonitor::IsScanning() const
	{
		return GetCanTick() && !bIsPaused;
	}

	void SDMXConflictMonitor::UpdateStatusInfo()
	{
		if (bIsPaused)
		{
			StatusInfo = EDMXConflictMonitorStatusInfo::Paused;
		}
		else if (!GetCanTick())
		{
			StatusInfo = EDMXConflictMonitorStatusInfo::Idle;
		}
		else if (Models.IsEmpty())
		{
			StatusInfo = EDMXConflictMonitorStatusInfo::OK;
		}
		else
		{
			StatusInfo = EDMXConflictMonitorStatusInfo::Conflict;
		}
	}
}

#undef LOCTEXT_NAMESPACE
