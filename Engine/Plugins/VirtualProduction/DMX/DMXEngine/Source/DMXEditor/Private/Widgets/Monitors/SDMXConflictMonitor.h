// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Analytics/DMXEditorToolAnalyticsProvider.h"
#include "UObject/NameTypes.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class FUICommandList;
class ITableRow;
class SHeaderRow;
template <typename ItemType> class SListView;
class SRichTextBlock;
class STableViewBase;

namespace UE::DMX
{
	class FDMXConflictMonitorActiveObjectItem;
	class FDMXConflictMonitorConflictModel;
	class FDMXConflictMonitorUserSession;
	struct FDMXMonitoredOutboundDMXData;
	enum class EDMXConflictMonitorStatusInfo : uint8;

	/** Monitors conflicts. */
	class SDMXConflictMonitor
		: public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SDMXConflictMonitor)
			{}

		SLATE_END_ARGS()

		SDMXConflictMonitor();

		/** Constructs the widget */
		void Construct(const FArguments& InArgs);


		struct FActiveObjectCollumnID
		{
			static const FName ObjectName;
			static const FName OpenAsset;
			static const FName ShowInContentBrowser;
		};

	protected:
		//~ Begin SWidget interface
		virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
		virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		//~ End SWidget interface

	private:
		/** Generates the header row of the active objects list */
		TSharedRef<SHeaderRow> GenerateActiveObjectHeaderRow();

		/** Generates a row for an object that is actively sending DMX */
		TSharedRef<ITableRow> OnGenerateActiveObjectRow(TSharedPtr<FDMXConflictMonitorActiveObjectItem> InItem, const TSharedRef<STableViewBase>& OwnerTable);

		/** Refreshes the widget */
		void Refresh();

		/** Initializes the command list for this widget */
		void SetupCommandList();

		/** Updates the status info for this monitor */
		void UpdateStatusInfo();

		/** Returns true if the monitor is scanning */
		bool IsScanning() const;

		void Play();
		void Pause();
		void Stop();

		void SetAutoPause(bool bEnabled);
		void ToggleAutoPause();
		bool IsAutoPause() const;

		void SetPrintToLog(bool bEnabled);
		void TogglePrintToLog();
		bool IsPrintingToLog() const;

		void SetRunWhenOpened(bool bEnabled);
		void ToggleRunWhenOpened();
		bool IsRunWhenOpened() const;

		/** Cached outbound data */
		TArray<TSharedRef<FDMXMonitoredOutboundDMXData>> CachedOutboundData;

		/** Cached outbound conflicts */
		TMap<FName, TArray<TSharedRef<FDMXMonitoredOutboundDMXData>>> CachedOutboundConflicts;

		/** Items displayed in the list */
		TArray<TSharedPtr<FDMXConflictMonitorConflictModel>> Models;

		/** Text block displaying outbound conflicts, one conflict per row */
		TSharedPtr<SRichTextBlock> LogTextBlock;

		/** Timer to refresh at refresh period */
		double Timer = 0.0;

		/** True if paused */
		bool bIsPaused = false;

		/** Source for the Active Object List */
		TArray<TSharedPtr<FDMXConflictMonitorActiveObjectItem>> ActiveObjectListSource;

		/** The Active Object List */
		TSharedPtr<SListView<TSharedPtr<FDMXConflictMonitorActiveObjectItem>>> ActiveObjectList;

		/** The status of the monitor. Note status info is ment for UI purposes, and not the state of the monitor. */
		EDMXConflictMonitorStatusInfo StatusInfo;

		/** The conflict montitor user session used by this widget */
		TSharedPtr<FDMXConflictMonitorUserSession> UserSession;

		/** Commandlist specific to this widget (only one can ever be displayed) */
		TSharedPtr<FUICommandList> CommandList;

		/** The analytics provider for this tool */
		FDMXEditorToolAnalyticsProvider AnalyticsProvider;

		/** Time on the game thread */
		double TimeGameThread = 0.0;

		// Slate args
		TAttribute<double> UpdateInterval;
	};
}
