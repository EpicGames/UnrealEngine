// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXConflictMonitorActiveObjectRow.h"

#include "ContentBrowserModule.h"
#include "DMXConflictMonitorActiveObjectItem.h"
#include "DMXEditorStyle.h"
#include "Editor.h"
#include "IContentBrowserSingleton.h"
#include "SDMXConflictMonitor.h"
#include "Styling/AppStyle.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Text/SRichTextBlock.h"

#define LOCTEXT_NAMESPACE "SDMXConflictMonitorActiveObjectRow"

namespace UE::DMX
{
	void SDMXConflictMonitorActiveObjectRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, const TSharedRef<FDMXConflictMonitorActiveObjectItem>& InActiveObjectItem)
	{
		ActiveObjectItem = InActiveObjectItem;

		SMultiColumnTableRow<TSharedPtr<FDMXConflictMonitorActiveObjectItem>>::Construct(
			FSuperRowType::FArguments(),
			OwnerTable
		);
	}

	TSharedRef<SWidget> SDMXConflictMonitorActiveObjectRow::GenerateWidgetForColumn(const FName& ColumnName)
	{
		const FMargin Padding(4.f, 2.f, 4.f, 2.f);
		if (ColumnName == SDMXConflictMonitor::FActiveObjectCollumnID::ObjectName)
		{
			return
				SNew(SBorder)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.Padding(Padding)
				[
					SNew(STextBlock)
					.Visibility(EVisibility::HitTestInvisible)
					.TextStyle(FAppStyle::Get(), "MessageLog")
					.Text_Lambda([this, SharedThis = AsShared()]()
						{
							return FText::FromString(ActiveObjectItem->ObjectName.ToString());
						})
				];
		}
		else if (ColumnName == SDMXConflictMonitor::FActiveObjectCollumnID::OpenAsset)
		{
			return 
				SNew(SBorder)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.Padding(Padding)
				[
					SNew(SWidgetSwitcher)
					.WidgetIndex_Lambda([this, SharedThis = AsShared()]
						{
							return ActiveObjectItem->ObjectPath.IsNull() ? 0 : 1;
						})

					+ SWidgetSwitcher::Slot()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("UnavailableAssetHyperlink", "unknown"))
						.TextStyle(FAppStyle::Get(), "NormalFont")
					]

					+ SWidgetSwitcher::Slot()
					[
						SNew(SHyperlink)
						.Text(LOCTEXT("OpenAssetHyperlink", "Open Asset"))
						.Style(FAppStyle::Get(), "Common.GotoNativeCodeHyperlink")
						.OnNavigate(this, &SDMXConflictMonitorActiveObjectRow::OnOpenAssetClicked)
					]
				];
		}
		else if (ColumnName == SDMXConflictMonitor::FActiveObjectCollumnID::ShowInContentBrowser)
		{
			return 
				SNew(SBorder)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.Padding(Padding)
				[
					SNew(SWidgetSwitcher)
					.WidgetIndex_Lambda([this, SharedThis = AsShared()]
					{
						return ActiveObjectItem->ObjectPath.IsNull() ? 0 : 1;
					})

					+ SWidgetSwitcher::Slot()
					[
						SNew(STextBlock)
						.Text(FText())
						.TextStyle(FAppStyle::Get(), "NormalFont")
					]

					+ SWidgetSwitcher::Slot()
					[
						SNew(SButton)
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.Search"))
							.OnMouseButtonDown(this, &SDMXConflictMonitorActiveObjectRow::OnShowInContentBrowserClicked)
						]
					]
				];
		}
		else
		{
			checkf(0, TEXT("Unhandled column ID"));
		}

		return SNullWidget::NullWidget;
	}

	void SDMXConflictMonitorActiveObjectRow::OnOpenAssetClicked()
	{
		if (UObject* AssetObject = ActiveObjectItem->ObjectPath.TryLoad())
		{
			TArray<UObject*> Assets;
			Assets.Add(AssetObject);
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets(Assets);
		}
	}

	FReply SDMXConflictMonitorActiveObjectRow::OnShowInContentBrowserClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (UObject* AssetObject = ActiveObjectItem->ObjectPath.TryLoad())
		{
			FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
			TArray<FAssetData> Assets;
			Assets.Add(AssetObject);
			ContentBrowserModule.Get().SyncBrowserToAssets(Assets);
		}

		return FReply::Handled();
	}
}

#undef LOCTEXT_NAMESPACE
