// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class SRichTextBlock;

namespace UE::DMX
{
	class FDMXConflictMonitorActiveObjectItem;

	/** A row in the Conflict Monitor's Active Object list */
	class SDMXConflictMonitorActiveObjectRow
		: public SMultiColumnTableRow<TSharedPtr<FDMXConflictMonitorActiveObjectItem>>
	{
	public:
		SLATE_BEGIN_ARGS(SDMXConflictMonitorActiveObjectRow)
			{}

		SLATE_END_ARGS()

		/** Constructs this widget */
		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, const TSharedRef<FDMXConflictMonitorActiveObjectItem>& InActiveObjectItem);

	protected:
		//~ Begin SMultiColumnTableRow interface
		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
		//~ End SMultiColumnTableRow interface

	private:
		/** Called when the open asset hyperlink was clicked */
		void OnOpenAssetClicked();

		/** Called when the show in content browser button was clicked */
		FReply OnShowInContentBrowserClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

		/** The active object item */
		TSharedPtr<FDMXConflictMonitorActiveObjectItem> ActiveObjectItem;
	};
}
