// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXConflictMonitorToolbar.h"

#include "Commands/DMXConflictMonitorCommands.h"
#include "DMXEditorSettings.h"
#include "DMXEditorStyle.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Internationalization/Text.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SDMXConflictMonitorToolbar"

namespace UE::DMX
{
	namespace Private
	{
		/** Widget to display the scanning status and a throbber */
		class SDMXConflictMonitorScanInfo
			: public SCompoundWidget
		{
		public:
			SLATE_BEGIN_ARGS(SDMXConflictMonitorScanInfo)
				: _IsScanning(false)
				{}

				/** Sets if the toolbar displays a scanning status */
				SLATE_ATTRIBUTE(bool, IsScanning)

			SLATE_END_ARGS()

			/** Constructs the widget */
			void Construct(const FArguments& InArgs)
			{
				IsScanning = InArgs._IsScanning;

				SetVisibility(TAttribute<EVisibility>::CreateSP(this, &SDMXConflictMonitorScanInfo::GetWidgetVisibility));

				ChildSlot
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				[
					SNew(SHorizontalBox)

					// Throbber 
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(2.f, 0.f, 6.f, 0.f)
					[
						SNew(SCircularThrobber)
						.Radius(8.5f)
					]

					// Scanning Info 
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.FillWidth(1.f)
					.Padding(2.f, 0.f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ScanningText", "Scanning.."))
					]
				];
			}

		private:
			EVisibility GetWidgetVisibility() const
			{
				return IsScanning.Get() ?
					EVisibility::Visible :
					EVisibility::Collapsed;
			}

			// Slate args
			TAttribute<bool> IsScanning;
		};
	}

	void SDMXConflictMonitorToolbar::Construct(const FArguments& InArgs, const TSharedRef<FUICommandList>& InCommandList)
	{
		CommandList = InCommandList;
		OnDepthChanged = InArgs._OnDepthChanged;

		CachedTimeGameThread = InArgs._TimeGameThread.Get();

		ChildSlot
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				CreateToolbarMenu(InArgs)
			]
		];
	}

	TSharedRef<SWidget> SDMXConflictMonitorToolbar::CreateToolbarMenu(const FArguments& InArgs)
	{
		StatusInfo = InArgs._StatusInfo;
		OnDepthChanged = InArgs._OnDepthChanged;

		FToolBarBuilder ToolbarBuilder(CommandList, FMultiBoxCustomization::None);
		const FName NoExtender = NAME_None;

		ToolbarBuilder.BeginSection("PlaySection");
		{
			// Play
			ToolbarBuilder.BeginStyleOverride("Toolbar.BackplateLeftPlay");
			ToolbarBuilder.AddToolBarButton(FDMXConflictMonitorCommands::Get().StartScan,
				NoExtender,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.PlayDMX"));
			ToolbarBuilder.EndStyleOverride();

			// Pause
			ToolbarBuilder.BeginStyleOverride("Toolbar.BackplateLeft");
			ToolbarBuilder.AddToolBarButton(FDMXConflictMonitorCommands::Get().PauseScan,
				NoExtender,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.PauseDMX"));
			ToolbarBuilder.EndStyleOverride();

			// Resume
			ToolbarBuilder.BeginStyleOverride("Toolbar.BackplateLeftPlay");
			ToolbarBuilder.AddToolBarButton(FDMXConflictMonitorCommands::Get().ResumeScan,
				NoExtender,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.ResumeDMX"));
			ToolbarBuilder.EndStyleOverride();

			// Stop
			ToolbarBuilder.BeginStyleOverride("Toolbar.BackplateCenterStop");
			ToolbarBuilder.AddToolBarButton(FDMXConflictMonitorCommands::Get().StopScan,
				NoExtender,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.StopDMX"));
			ToolbarBuilder.EndStyleOverride();
		
			ToolbarBuilder.BeginStyleOverride("Toolbar.BackplateRightCombo");
			ToolbarBuilder.AddComboButton(
				FUIAction(),
				FOnGetContent::CreateSP(this, &SDMXConflictMonitorToolbar::CreateSettingsMenu),
				LOCTEXT("SettingsLabel", "Settings"),
				LOCTEXT("SettingsToolTip", "Settings for the conflict monitor."),
				TAttribute<FSlateIcon>(),
				true);

			ToolbarBuilder.EndSection();
			ToolbarBuilder.EndStyleOverride();

			ToolbarBuilder.BeginStyleOverride("Toolbar.BackplateRight");
			ToolbarBuilder.AddSeparator();
			ToolbarBuilder.EndStyleOverride();
		}
		ToolbarBuilder.EndSection();

		ToolbarBuilder.BeginSection("ScanSection");
		{
			// Status
			ToolbarBuilder.AddWidget(
				SNew(Private::SDMXConflictMonitorScanInfo)
				.IsScanning_Lambda([this]()
					{
						return
							StatusInfo.Get() == EDMXConflictMonitorStatusInfo::OK ||
							StatusInfo.Get() == EDMXConflictMonitorStatusInfo::Conflict;
					})
			);
		}
		ToolbarBuilder.EndSection();

		ToolbarBuilder.BeginStyleOverride("AssetEditorToolbar");
		ToolbarBuilder.BeginSection("StatusSection");
		{
			// Status Label
			ToolbarBuilder.AddWidget(
				SNew(SBorder)
				.VAlign(VAlign_Center)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("StatusLabel", "Status: "))
				]
			);

			// Status Info
			ToolbarBuilder.AddWidget(
				SNew(SBorder)
				.VAlign(VAlign_Center)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				[
					SNew(STextBlock)
					.Text(this, &SDMXConflictMonitorToolbar::GetStatusText)
					.ColorAndOpacity(this, &SDMXConflictMonitorToolbar::GetStatusTextColor)
				]
			);
		}
		ToolbarBuilder.EndSection();

		ToolbarBuilder.BeginSection("Stats");
		{
			const FName TutorialHighlight = NAME_None;
			constexpr bool bSearchable = true;

			ToolbarBuilder.AddWidget(
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SNew(SBorder)
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.BorderImage(FAppStyle::GetBrush("NoBorder"))
					[
						SNew(STextBlock)
						.MinDesiredWidth(100.f)
						.Text_Lambda([TimeGameThreadAttribute = InArgs._TimeGameThread, this]()
							{			
								const double DeltaTime = FSlateApplication::Get().GetDeltaTime();

								const double TimeGameThread = TimeGameThreadAttribute.Get();

								CachedTimeGameThread = 	CachedTimeGameThread + (TimeGameThread - CachedTimeGameThread) * (DeltaTime / 1.0);

								FNumberFormattingOptions Options;
								Options.SetMaximumFractionalDigits(2);
								Options.SetMinimumFractionalDigits(2);

								const FText ValueText = FText::AsNumber(CachedTimeGameThread, &Options);
								return FText::Format(LOCTEXT("PerformanceText", "CPU Usage (GameThread): {0}ms"), ValueText);
							})
					]
				],
				TutorialHighlight,
				bSearchable,
				HAlign_Right);
		}
		ToolbarBuilder.EndSection();

		return ToolbarBuilder.MakeWidget();
	}

	FText SDMXConflictMonitorToolbar::GetStatusText() const
	{
		switch (StatusInfo.Get())
		{
		case EDMXConflictMonitorStatusInfo::Idle:
			return LOCTEXT("StatusIdle", "Idle");
		case EDMXConflictMonitorStatusInfo::Paused:
			return LOCTEXT("StatusPaused", "Paused");
		case EDMXConflictMonitorStatusInfo::OK:
			return LOCTEXT("StatusOK", "OK");
		case EDMXConflictMonitorStatusInfo::Conflict:
			return LOCTEXT("StatusConflict", "Conflict");
		default:
			checkNoEntry();
		}
		return FText::GetEmpty();
	}

	FSlateColor SDMXConflictMonitorToolbar::GetStatusTextColor() const
	{
		switch (StatusInfo.Get())
		{
		case EDMXConflictMonitorStatusInfo::Idle:
		case EDMXConflictMonitorStatusInfo::Paused:
			return FLinearColor::White.CopyWithNewOpacity(0.9f);
		case EDMXConflictMonitorStatusInfo::OK:
			return FLinearColor::Green.CopyWithNewOpacity(0.9f);
		case EDMXConflictMonitorStatusInfo::Conflict:
			return FLinearColor::Red.CopyWithNewOpacity(0.9f);
		default:
			checkNoEntry();
		}

		return FSlateColor();
	}

	TSharedRef<SWidget> SDMXConflictMonitorToolbar::CreateSettingsMenu()
	{
		constexpr bool bShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

		MenuBuilder.BeginSection("OptionsSection", LOCTEXT("OptionsSectionTitle", "Options"));
		{
			MenuBuilder.AddMenuEntry(FDMXConflictMonitorCommands::Get().ToggleAutoPause);
			MenuBuilder.AddMenuEntry(FDMXConflictMonitorCommands::Get().TogglePrintToLog);
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("MonitorSection", LOCTEXT("MonitorSectionTitle", "Monitor"));
		{
			MenuBuilder.AddMenuEntry(FDMXConflictMonitorCommands::Get().ToggleRunWhenOpened);

			const TSharedRef<SWidget> EditDepthWidget = 
				SNew(SNumericEntryBox<uint8>)
				.MinDesiredValueWidth(40.f)
				.MinValue(1)
				.MinSliderValue(0)
				.MaxSliderValue(8)
				.AllowSpin(true)
				.ToolTipText(LOCTEXT("DepthTooltip", "The displayed depth of the traces. If 1, only shows the top level traces."))
				.Value_Lambda([]()
					{
						const UDMXEditorSettings* EditorSettings = GetDefault<UDMXEditorSettings>();
						return EditorSettings->ConflictMonitorSettings.Depth;
					})
				.OnValueChanged(this, &SDMXConflictMonitorToolbar::SetDepth);
			
			MenuBuilder.AddWidget(EditDepthWidget, 
				LOCTEXT("DepthLabel", "Depth"));
		}
		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}

	void SDMXConflictMonitorToolbar::SetDepth(uint8 InDepth)
	{
		UDMXEditorSettings* EditorSettings = GetMutableDefault<UDMXEditorSettings>();
		if (EditorSettings->ConflictMonitorSettings.Depth != InDepth)
		{
			EditorSettings->ConflictMonitorSettings.Depth = InDepth;
			EditorSettings->SaveConfig();

			OnDepthChanged.ExecuteIfBound();
		}
	}
} // namespace UE::DMX

#undef LOCTEXT_NAMESPACE
