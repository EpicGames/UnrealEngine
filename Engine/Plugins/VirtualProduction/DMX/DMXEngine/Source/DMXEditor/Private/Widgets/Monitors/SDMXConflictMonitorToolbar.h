// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class FUICommandList;


namespace UE::DMX
{	
	/** Possible statuses */
	enum class EDMXConflictMonitorStatusInfo : uint8
	{
		Idle,
		Paused,
		OK,
		Conflict
	};

	/** Monitors conflicts. */
	class SDMXConflictMonitorToolbar
		: public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SDMXConflictMonitorToolbar)
			: _StatusInfo(EDMXConflictMonitorStatusInfo::Idle)
			, _TimeGameThread(0.0)
			{}

			/** The status of the monitor */
			SLATE_ATTRIBUTE(EDMXConflictMonitorStatusInfo, StatusInfo)

			/** The game thread time */
			SLATE_ATTRIBUTE(double, TimeGameThread)

			/** Broadcast when the depth changed */
			SLATE_EVENT(FSimpleDelegate, OnDepthChanged)

		SLATE_END_ARGS()

		/** Constructs the widget */
		void Construct(const FArguments& InArgs, const TSharedRef<FUICommandList>& InCommandList);

	private:
		/** Creates the toolbar menu */
		TSharedRef<SWidget> CreateToolbarMenu(const FArguments& InArgs);

		/** Returns the current status text */
		FText GetStatusText() const;

		/** Returns the current status text color */
		FSlateColor GetStatusTextColor() const;

		/** Generates the settings menu */
		TSharedRef<SWidget> CreateSettingsMenu();

		/** Sets a new depth */
		void SetDepth(uint8 InDepth);

		/** The command list for this widget */
		TSharedPtr<FUICommandList> CommandList;

		/** The current depth */
		uint8 Depth = 3;

		/** Cached time on game thread */
		double CachedTimeGameThread = 0.0;

		// Slate args
		TAttribute<EDMXConflictMonitorStatusInfo> StatusInfo;
		FSimpleDelegate OnDepthChanged;
	};
} // namespace UE::DMX
