// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXMonitorSourceSelector.h"

#include "IO/DMXInputPort.h"
#include "IO/DMXOutputPort.h"
#include "IO/DMXPortManager.h"

#include "Styling/AppStyle.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SWrapBox.h"


#define LOCTEXT_NAMESPACE "SDMXMonitorSourceSelector"

const TArray<TSharedPtr<FText>> SDMXMonitorSourceSelector::IODirectionNames =  
	{
		MakeShared<FText>(LOCTEXT("InputsLabel", "Inputs")),
		MakeShared<FText>(LOCTEXT("OutputsLabel", "Outputs"))
	};

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SDMXMonitorSourceSelector::Construct(const FArguments& InArgs)
{
	OnSourceSelected = InArgs._OnSourceSelected;

	check(IODirectionNames.Num() == 2);

	TSharedPtr<FText> InitialDirection = InArgs._bSelectInput ? IODirectionNames[0] : IODirectionNames[1];
	check(InitialDirection.IsValid());

	ChildSlot
	[
		SNew(SHorizontalBox)

		// Monitor All Ports Checkbox
		+ SHorizontalBox::Slot()
		.Padding(FMargin(36.f, 0.f, 0.f, 0.f))
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.Text(LOCTEXT("MonitorAllLabel", "Monitor All Ports"))
			]

			+ SHorizontalBox::Slot()
			.Padding(10.f, 0.f, 0.f, 0.f)
			.AutoWidth()
			[
				SAssignNew(MonitorAllPortsCheckBox, SCheckBox)
				.OnCheckStateChanged(this, &SDMXMonitorSourceSelector::OnMonitorAllPortsCheckBoxChanged)
				.IsChecked(InArgs._bMonitorAllPorts)
			]
		]

		// IO Direction ComboBox 
		+ SHorizontalBox::Slot()
		.Padding(FMargin(36.f, 0.f, 0.f, 0.f))
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SAssignNew(MonitoredIODirectionWrapper, SHorizontalBox)

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(40.f)
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.Text(LOCTEXT("SourceLabel", "Source"))
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(10.f, 0.f, 0.f, 0.f)
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(80.0f)
				[
					SAssignNew(MonitoredIODirectionComboBox, SComboBox<TSharedPtr<FText>>)
					.OptionsSource(&IODirectionNames)
					.InitiallySelectedItem(InitialDirection)
					.OnGenerateWidget(this, &SDMXMonitorSourceSelector::GenerateIODirectionEntry)
					.OnSelectionChanged(this, &SDMXMonitorSourceSelector::OnMonitoredIODirectionChanged)
					[
						SAssignNew(MonitoredIODirectionTextBlock, STextBlock)
						.Text(*InitialDirection)
						.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					]
				]
			]
		]
		
		// Port Selector
		+ SHorizontalBox::Slot()
		.Padding(FMargin(36.f, 0.f, 0.f, 0.f))
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SAssignNew(PortSelectorWrapper, SHorizontalBox)

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(40.f)
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.Text(LOCTEXT("PortLabel", "Port"))	
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(10.f, 0.f, 0.f, 0.f)
			.AutoWidth()
			[
				SAssignNew(PortSelector, SDMXPortSelector)
				.Mode(InArgs._PortSelectorMode)
				.InitialSelection(InArgs._InitiallySelectedGuid)
				.OnPortSelected(this, &SDMXMonitorSourceSelector::OnPortSelected)
			]
		]
	];

	UpdateMonitoredPorts();

	FDMXPortManager::Get().OnPortsChanged.AddSP(this, &SDMXMonitorSourceSelector::UpdateMonitoredPorts);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

bool SDMXMonitorSourceSelector::IsMonitorAllPorts() const
{
	check(MonitorAllPortsCheckBox.IsValid());

	return MonitorAllPortsCheckBox->GetCheckedState() == ECheckBoxState::Checked;
}

void SDMXMonitorSourceSelector::SetMonitorAllPorts(bool bMonitorAllPorts)
{
	check(MonitorAllPortsCheckBox.IsValid());

	MonitorAllPortsCheckBox->SetIsChecked(bMonitorAllPorts);
	UpdateMonitoredPorts();
}

bool SDMXMonitorSourceSelector::IsMonitorInputPorts() const
{
	check(IODirectionNames.Num() == 2);
	check(MonitoredIODirectionComboBox.IsValid());

	if (IsMonitorAllPorts())
	{
		TSharedPtr<FText> SelectedDirectionItem = MonitoredIODirectionComboBox->GetSelectedItem();
		return SelectedDirectionItem == IODirectionNames[0];
	}
	else if(PortSelector->GetSelectedInputPort().IsValid())
	{
		return true;
	}

	return false;
}

void SDMXMonitorSourceSelector::SetMonitorInputPorts(bool bMonitorInputPorts)
{
	check(IODirectionNames.Num() == 2);
	check(MonitoredIODirectionComboBox.IsValid());

	if (bMonitorInputPorts)
	{
		MonitoredIODirectionComboBox->SetSelectedItem(IODirectionNames[0]);
	}
	else
	{
		MonitoredIODirectionComboBox->SetSelectedItem(IODirectionNames[1]);
	}

	UpdateMonitoredPorts();
}

FGuid SDMXMonitorSourceSelector::GetMonitoredPortGuid() const
{
	check(PortSelector.IsValid());

	if (FDMXInputPortSharedPtr SelectedInputPort = PortSelector->GetSelectedInputPort())
	{
		return SelectedInputPort->GetPortGuid();
	}
	
	if (FDMXOutputPortSharedPtr SelectedOutputPort = PortSelector->GetSelectedOutputPort())
	{
		return SelectedOutputPort->GetPortGuid();
	}

	return FGuid();
}

void SDMXMonitorSourceSelector::SetMonitoredPortGuid(const FGuid& PortGuid)
{
	if (PortGuid.IsValid())
	{
		PortSelector->SelectPort(PortGuid);		
		UpdateMonitoredPorts();
	}
}

void SDMXMonitorSourceSelector::UpdateMonitoredPorts()
{
	check(MonitorAllPortsCheckBox.IsValid());
	check(MonitoredIODirectionComboBox.IsValid());
	check(PortSelector.IsValid());
	check(MonitoredIODirectionWrapper.IsValid());
	check(PortSelectorWrapper.IsValid());

	SelectedInputPorts.Reset();
	SelectedOutputPorts.Reset();

	const bool bMonitorAllPorts = MonitorAllPortsCheckBox->GetCheckedState() == ECheckBoxState::Checked;
	if (bMonitorAllPorts)
	{
		MonitoredIODirectionWrapper->SetVisibility(EVisibility::Visible);
		PortSelectorWrapper->SetVisibility(EVisibility::Collapsed);

		if (IsMonitorInputPorts())
		{
			SelectedInputPorts = FDMXPortManager::Get().GetInputPorts();
		}
		else
		{
			SelectedOutputPorts = FDMXPortManager::Get().GetOutputPorts();
		}
	}
	else
	{
		MonitoredIODirectionWrapper->SetVisibility(EVisibility::Collapsed);
		PortSelectorWrapper->SetVisibility(EVisibility::Visible);

		if (IsMonitorInputPorts())
		{
			if (FDMXInputPortSharedPtr InputPort = PortSelector->GetSelectedInputPort())
			{
				if (InputPort.IsValid())
				{
					SelectedInputPorts.Add(InputPort.ToSharedRef());
				}
			}

			// Make a selection if nothing was already selected
			if (SelectedInputPorts.IsEmpty() && FDMXPortManager::Get().GetInputPorts().Num() > 0)
			{
				const FDMXInputPortSharedRef& NewSelection = FDMXPortManager::Get().GetInputPorts()[0];
				PortSelector->SelectPort(NewSelection->GetPortGuid());
			}
		}
		else
		{
			if (FDMXOutputPortSharedPtr OutputPort = PortSelector->GetSelectedOutputPort())
			{
				if (OutputPort.IsValid())
				{
					SelectedOutputPorts.Add(OutputPort.ToSharedRef());
				}
			}

			// Make a selection if nothing was already selected
			if (SelectedOutputPorts.IsEmpty() && FDMXPortManager::Get().GetOutputPorts().Num() > 0)
			{
				const FDMXOutputPortSharedRef& NewSelection = FDMXPortManager::Get().GetOutputPorts()[0];
				PortSelector->SelectPort(NewSelection->GetPortGuid());
			}
		}
	}
}

TSharedRef<SWidget> SDMXMonitorSourceSelector::GenerateIODirectionEntry(TSharedPtr<FText> IODirectionNameToAdd)
{
	return
		SNew(STextBlock)
		.Text(*IODirectionNameToAdd)
		.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));
}

void SDMXMonitorSourceSelector::OnMonitorAllPortsCheckBoxChanged(const ECheckBoxState NewCheckState)
{
	UpdateMonitoredPorts();
	OnSourceSelected.ExecuteIfBound();
}

void SDMXMonitorSourceSelector::OnMonitoredIODirectionChanged(TSharedPtr<FText> SelectedIODirectionName, ESelectInfo::Type SelectInfo)
{
	check(MonitoredIODirectionTextBlock.IsValid());

	MonitoredIODirectionTextBlock->SetText(*SelectedIODirectionName);

	UpdateMonitoredPorts();
	OnSourceSelected.ExecuteIfBound();
}

void SDMXMonitorSourceSelector::OnPortSelected()
{
	UpdateMonitoredPorts();
	OnSourceSelected.ExecuteIfBound();
}

#undef LOCTEXT_NAMESPACE
