// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolCommon.h"
#include "Widgets/SDMXPortSelector.h"

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SComboBox.h"

enum class EDMXPortSelectorMode : uint8;
class SDMXPortSelector;

class SCheckBox;
class SHorizontalBox;
class STextBlock;


/** Widget to select Ports to monitor. Note, when inputs are selected, loop back outputs are selected too. */
class SDMXMonitorSourceSelector
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMXMonitorSourceSelector)
		: _bMonitorAllPorts(true)
		, _bSelectInput(true)
		, _PortSelectorMode(EDMXPortSelectorMode::SelectFromAvailableInputsAndOutputs)
		, _InitiallySelectedGuid(FGuid())
	{}

		SLATE_ARGUMENT(bool, bMonitorAllPorts)
	
		SLATE_ARGUMENT(bool, bSelectInput)

		SLATE_ARGUMENT(EDMXPortSelectorMode, PortSelectorMode)

		SLATE_ARGUMENT(FGuid, InitiallySelectedGuid)

		SLATE_EVENT(FSimpleDelegate, OnSourceSelected)

	SLATE_END_ARGS()

	/** Constructs the widget */
	void Construct(const FArguments& InArgs);

	/** Returns true if the Monitor All Ports CheckBox is checked */
	bool IsMonitorAllPorts() const;

	/** Sets the monitor all ports checkbox to checked or unchecked depending on the bMonitorAllPorts Argument */
	void SetMonitorAllPorts(bool bMonitorAllPorts);

	/** Returns true if Input is selected in the IODirection ComboBox */
	bool IsMonitorInputPorts() const;

	/** Set the IODirection to Input if bMonitorInputPorts argument is true, else sets it to Output*/
	void SetMonitorInputPorts(bool bMonitorInputPorts);

	/** Returns the Monitored Port Guid from the Port Selector, or an invalid Guid if the Port Selector holds no selection */
	FGuid GetMonitoredPortGuid() const;

	/** Sets the Port Guid to select specified port, if the Guid is valid, else selects the first port if a port is available */
	void SetMonitoredPortGuid(const FGuid& PortGuid);

	const TArray<FDMXInputPortSharedRef>& GetSelectedInputPorts() const { return SelectedInputPorts; }
	const TArray<FDMXOutputPortSharedRef>& GetSelectedOutputPorts() const { return SelectedOutputPorts; }

private:
	/** Updates the monitored ports */
	void UpdateMonitoredPorts();

	/** Generates an entry in the monitor source combo box */
	TSharedRef<SWidget> GenerateIODirectionEntry(TSharedPtr<FText> SourceNameToAdd);

	/** Called when the Monitor All Ports Check Box state changed */
	void OnMonitorAllPortsCheckBoxChanged(const ECheckBoxState NewCheckState);

	/** Called when an IO Direction to be monitored was selected in related combo box */
	void OnMonitoredIODirectionChanged(TSharedPtr<FText> SelectedIODirectionName, ESelectInfo::Type SelectInfo);

	/** Called when a port was selected in the Port Selector */
	void OnPortSelected();

	/** The input ports that are currently being monitored */
	TArray<FDMXInputPortSharedRef> SelectedInputPorts;

	/** The output ports that are currently being monitored */
	TArray<FDMXOutputPortSharedRef> SelectedOutputPorts;

	/** CheckBox to toggle if all ports should be monitored */
	TSharedPtr<SCheckBox> MonitorAllPortsCheckBox;

	/** Horizontal box that contains the Port Selector and its Label */
	TSharedPtr<SHorizontalBox> PortSelectorWrapper;

	/** Selector for the port to monitor */
	TSharedPtr<SDMXPortSelector> PortSelector;

	/** Horizontal box that contains the Monitored IO Direction and its Label */
	TSharedPtr<SHorizontalBox> MonitoredIODirectionWrapper;

	/** ComboBox to select where monitored DMX signals originate */
	TSharedPtr<SComboBox<TSharedPtr<FText>>> MonitoredIODirectionComboBox;

	/** Text block that is shown on top of the combo box */
	TSharedPtr<STextBlock> MonitoredIODirectionTextBlock;

	/** IODirection for the combo box to select where the signals monitored originate */
	static const TArray<TSharedPtr<FText>> IODirectionNames;

	FSimpleDelegate OnSourceSelected;
};
