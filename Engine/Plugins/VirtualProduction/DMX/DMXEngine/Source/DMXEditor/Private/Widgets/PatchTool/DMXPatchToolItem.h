// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/SoftObjectPtr.h"

class UDMXLibrary;

namespace UE::DMX
{
	class FDMXPatchToolItem
		: public TSharedFromThis<FDMXPatchToolItem>
	{
	public:
		FDMXPatchToolItem(TSoftObjectPtr<UDMXLibrary> InDMXLibrary)
			: SoftDMXLibrary(InDMXLibrary)
		{}

		const TSoftObjectPtr<UDMXLibrary> SoftDMXLibrary;
	};
}
