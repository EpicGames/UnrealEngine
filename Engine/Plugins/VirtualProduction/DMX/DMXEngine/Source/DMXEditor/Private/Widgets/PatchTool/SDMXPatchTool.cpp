// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXPatchTool.h"

#include "Algo/Transform.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "DMXPatchToolItem.h"
#include "DMXSubsystem.h"
#include "Editor.h"
#include "Engine/Selection.h"
#include "Game/DMXComponent.h"
#include "GameFramework/Actor.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXLibrary.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SGridPanel.h"

#define LOCTEXT_NAMESPACE "SDMXPatchTool"

namespace UE::DMX
{
	SDMXPatchTool::SDMXPatchTool()
		: AnalyticsProvider("PatchTool")
	{}

	SDMXPatchTool::~SDMXPatchTool()
	{}

	void SDMXPatchTool::Construct(const FArguments& InArgs)
	{
		Refresh();

		// Listen to assets being added or removed
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		AssetRegistryModule.Get().OnAssetAdded().AddSP(this, &SDMXPatchTool::OnAssetAddedOrRemoved);
		AssetRegistryModule.Get().OnAssetRemoved().AddSP(this, &SDMXPatchTool::OnAssetAddedOrRemoved);

		// Listen to DMX Library changes
		UDMXLibrary::GetOnEntitiesAdded().AddSP(this, &SDMXPatchTool::OnEntitiesAddedOrRemoved);
		UDMXLibrary::GetOnEntitiesRemoved().AddSP(this, &SDMXPatchTool::OnEntitiesAddedOrRemoved);
	}

	void SDMXPatchTool::AddReferencedObjects(FReferenceCollector& Collector)
	{
		Collector.AddReferencedObject(DMXLibrary);
		Collector.AddReferencedObject(FixturePatch);
		Collector.AddReferencedObjects(FixturePatchSource);
	}

	FString SDMXPatchTool::GetReferencerName() const
	{
		return TEXT("SDMXPatchTool");
	}

	void SDMXPatchTool::RequestRefresh()
	{
		if (!RefreshTimerHandle.IsValid())
		{
			RefreshTimerHandle = GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateSP(this, &SDMXPatchTool::Refresh));
		}
	}

	void SDMXPatchTool::Refresh()
	{
		RefreshTimerHandle.Invalidate();

		// Update the library source and make a valid selection
		LibrarySource.Reset();

		UDMXSubsystem* Subsystem = UDMXSubsystem::GetDMXSubsystem_Pure();
		check(Subsystem);

		Algo::Transform(Subsystem->GetDMXLibraries(), LibrarySource,
			[](const TSoftObjectPtr<UDMXLibrary>& SoftDMXLibrary)
			{
				return MakeShared<FDMXPatchToolItem>(SoftDMXLibrary);
			});

		const bool bCanSelectDMXLibrary = DMXLibrary && Algo::FindBy(LibrarySource, DMXLibrary.Get(), &FDMXPatchToolItem::SoftDMXLibrary) != nullptr;
		if (!bCanSelectDMXLibrary)
		{
			DMXLibrary = !LibrarySource.IsEmpty() ? LibrarySource[0]->SoftDMXLibrary.LoadSynchronous() : nullptr;
		}

		const TSharedPtr<FDMXPatchToolItem>* SelectDMXLibraryItemPtr = Algo::FindBy(LibrarySource, DMXLibrary.Get(), &FDMXPatchToolItem::SoftDMXLibrary);
		const TSharedPtr<FDMXPatchToolItem> SelectDMXLibraryItem = SelectDMXLibraryItemPtr ? *SelectDMXLibraryItemPtr : nullptr;


		// Update the fixture patch source and make a valid selection
		FixturePatchSource.Reset();
		if (DMXLibrary)
		{
			FixturePatchSource = ObjectPtrWrap(DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>());
		}

		const bool bCanSelectFixturePatch = FixturePatch && FixturePatchSource.Contains(FixturePatch);
		if (!bCanSelectFixturePatch)
		{
			FixturePatch = !FixturePatchSource.IsEmpty() ? FixturePatchSource[0] : nullptr;
		}


		// Rebuild the widget
		ChildSlot
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			.AutoHeight()
			[
				SNew(SGridPanel)

				// Library Selection Label
				+ SGridPanel::Slot(0, 0)
				.Padding(4.f)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.MinDesiredWidth(160.f)
					.MaxDesiredWidth(160.f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("DMXLibraryComboboxLabel", "DMX Library"))
						.Font(FCoreStyle::GetDefaultFontStyle("Regular", 14))
					]
				]

				// Library Selection Combo Box
				+ SGridPanel::Slot(1, 0)
				.Padding(4.f)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SAssignNew(LibraryComboBox, SComboBox<TSharedPtr<FDMXPatchToolItem>>)
						.OnGenerateWidget(this, &SDMXPatchTool::GenerateLibraryComboBoxEntry)
						.OnSelectionChanged(this, &SDMXPatchTool::OnLibrarySelected)
						.OptionsSource(&ObjectPtrDecay(LibrarySource))
						.InitiallySelectedItem(SelectDMXLibraryItem)
						[
							SNew(STextBlock)
							.Text_Lambda([this]()
								{
									return DMXLibrary ?
										FText::FromString(DMXLibrary->GetName()) :
										LOCTEXT("NoDMXLibraryAvailableInfo", "No DMX Library available");
								})
						]
					]
				]

				// Patch Selection Label
				+ SGridPanel::Slot(0, 1)
				.Padding(4.f)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.MinDesiredWidth(160.f)
					.MaxDesiredWidth(160.f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("DMXFixturePatchComboboxLabel", "Fixture Patch"))
						.Font(FCoreStyle::GetDefaultFontStyle("Regular", 14))
					]
				]

				// Patch Selection Combo Box
				+ SGridPanel::Slot(1, 1)
				.Padding(4.f)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SAssignNew(FixturePatchComboBox, SComboBox<UDMXEntityFixturePatch*>)
						.OnGenerateWidget(this, &SDMXPatchTool::GenerateFixturePatchComboBoxEntry)
						.OnSelectionChanged(this, &SDMXPatchTool::OnFixturePatchSelected)
						.OptionsSource(&ObjectPtrDecay(FixturePatchSource))
						.InitiallySelectedItem(FixturePatch)
						[
							SNew(STextBlock)
							.Text_Lambda([this]()
								{
									return FixturePatch ?
										FText::FromString(FixturePatch->Name) :
										LOCTEXT("NoPatchAvailableInfo", "No DMX Library selected");
								})
						]
					]
				]
			]

			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Top)
				.Padding(8.f)
				[
					SNew(SButton)
					.OnClicked(this, &SDMXPatchTool::OnAddressIncrementalClicked)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("AddressIncrementalButtonText", "Address incremental"))
						.Font(FCoreStyle::GetDefaultFontStyle("Regular", 14))
					]
				]

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Top)
				.Padding(8.f)
				[
					SNew(SButton)
					.OnClicked(this, &SDMXPatchTool::OnAddressSameClicked)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("AddressSameButtonText", "Address same"))
						.Font(FCoreStyle::GetDefaultFontStyle("Regular", 14))
					]
				]
			]

			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			.Padding(8.f)
			[
				SNew(SButton)
				.OnClicked(this, &SDMXPatchTool::OnAddressAndRenameClicked)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AddressAndRenameButtonText", "Address and Rename"))
					.Font(FCoreStyle::GetDefaultFontStyle("Regular", 14))
				]
			]
		];
	}

	FReply SDMXPatchTool::OnAddressIncrementalClicked()
	{
		UDMXEntityFixturePatch* SelectedFixturePatch = FixturePatchComboBox->GetSelectedItem();
		if (SelectedFixturePatch)
		{
			int32 IndexOfPatch = FixturePatchSource.IndexOfByKey(SelectedFixturePatch);

			for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
			{
				if (AActor* Actor = Cast<AActor>(*It))
				{
					for (UDMXComponent* Component : TInlineComponentArray<UDMXComponent*>(Actor))
					{
						if (FixturePatchSource.IsValidIndex(IndexOfPatch))
						{
							UDMXEntityFixturePatch* NextFixturePatch = FixturePatchSource[IndexOfPatch];
							Component->SetFixturePatch(NextFixturePatch);

							IndexOfPatch++;
						}
						else
						{
							return FReply::Handled();
						}
					}
				}
			}
		}

		return FReply::Handled();
	}

	FReply SDMXPatchTool::OnAddressSameClicked()
	{
		UDMXEntityFixturePatch* SelectedFixturePatch = FixturePatchComboBox->GetSelectedItem();
		if (SelectedFixturePatch)
		{
			for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
			{
				if (AActor* Actor = Cast<AActor>(*It))
				{
					for (UDMXComponent* Component : TInlineComponentArray<UDMXComponent*>(Actor))
					{
						Component->SetFixturePatch(SelectedFixturePatch);
					}
				}
			}
		}

		return FReply::Handled();
	}

	FReply SDMXPatchTool::OnAddressAndRenameClicked()
	{
		UDMXEntityFixturePatch* SelectedFixturePatch = FixturePatchComboBox->GetSelectedItem();
		if (SelectedFixturePatch)
		{
			int32 IndexOfPatch = FixturePatchSource.IndexOfByKey(SelectedFixturePatch);

			for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
			{
				if (AActor* Actor = Cast<AActor>(*It))
				{
					for (UDMXComponent* Component : TInlineComponentArray<UDMXComponent*>(Actor))
					{
						if (FixturePatchSource.IsValidIndex(IndexOfPatch))
						{
							UDMXEntityFixturePatch* NextFixturePatch = FixturePatchSource[IndexOfPatch];
							Component->SetFixturePatch(NextFixturePatch);

							// Rename
							Actor->SetActorLabel(NextFixturePatch->Name);

							IndexOfPatch++;
						}
						else
						{
							return FReply::Handled();
						}
					}
				}
			}
		}

		return FReply::Handled();
	}

	TSharedRef<SWidget> SDMXPatchTool::GenerateLibraryComboBoxEntry(TSharedPtr<FDMXPatchToolItem> ItemToAdd)
	{
		const FText LibraryName = FText::FromString(ItemToAdd->SoftDMXLibrary.GetAssetName());

		return
			SNew(STextBlock)
			.Text(LibraryName);
	}

	void SDMXPatchTool::OnLibrarySelected(TSharedPtr<FDMXPatchToolItem> SelectedItem, ESelectInfo::Type SelectInfo)
	{
		if (DMXLibrary.Get() != SelectedItem->SoftDMXLibrary)
		{
			DMXLibrary = SelectedItem->SoftDMXLibrary.LoadSynchronous();

			RequestRefresh();
		}
	}

	TSharedRef<SWidget> SDMXPatchTool::GenerateFixturePatchComboBoxEntry(UDMXEntityFixturePatch* FixturePatchToAdd)
	{
		const FText FixturePatchName = FText::FromString(FixturePatchToAdd->Name);

		return
			SNew(STextBlock)
			.Text(FixturePatchName);
	}

	void SDMXPatchTool::OnFixturePatchSelected(UDMXEntityFixturePatch* SelectedFixturePatch, ESelectInfo::Type SelectInfo)
	{
		FixturePatch = SelectedFixturePatch;
	}

	void SDMXPatchTool::OnAssetAddedOrRemoved(const FAssetData& AssetData)
	{
		RequestRefresh();
	}

	void SDMXPatchTool::OnEntitiesAddedOrRemoved(UDMXLibrary* ChangedDMXLibrary, TArray<UDMXEntity*> Entities)
	{
		RequestRefresh();
	}
}

#undef LOCTEXT_NAMESPACE
