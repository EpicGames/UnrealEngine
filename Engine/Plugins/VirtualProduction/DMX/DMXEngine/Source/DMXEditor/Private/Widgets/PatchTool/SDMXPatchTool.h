// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Analytics/DMXEditorToolAnalyticsProvider.h"
#include "Engine/TimerHandle.h"
#include "UObject/GCObject.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class STextBlock;
class UDMXEntity;
class UDMXEntityFixturePatch;
class UDMXLibrary;
struct FAssetData;
template<typename OptionType> class SComboBox;

namespace UE::DMX
{
	class FDMXPatchToolItem;

	/**
	 * A Monitor for DMX activity in a range of DMX Universes
	 */
	class SDMXPatchTool
		: public SCompoundWidget
		, public FGCObject
	{
	public:
		SLATE_BEGIN_ARGS(SDMXPatchTool)
			{}

		SLATE_END_ARGS()

		SDMXPatchTool();
		virtual ~SDMXPatchTool();

		/** Constructs the widget */
		void Construct(const FArguments& InArgs);

	private:
		// ~Begin FGCObject interface
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override;
		// ~End FGCObject interface

		/** Refreshes the widget on the next tick */
		void RequestRefresh();

		/** Refreshes the widget */
		void Refresh();

		/** Called when the Address Incremental Button was clicked */
		FReply OnAddressIncrementalClicked();

		/** Called when the Address Same Button was clicked */
		FReply OnAddressSameClicked();

		/** Called when the Address And Rename Button was clicked */
		FReply OnAddressAndRenameClicked();

		/** Generates an entry in the library combo box */
		TSharedRef<SWidget> GenerateLibraryComboBoxEntry(TSharedPtr<FDMXPatchToolItem> LibraryToAdd);

		/** Called when a dmx library was slected */
		void OnLibrarySelected(TSharedPtr<FDMXPatchToolItem> SelectedLibrary, ESelectInfo::Type SelectInfo);

		/** Combobox to select a library */
		TSharedPtr<SComboBox<TSharedPtr<FDMXPatchToolItem>>> LibraryComboBox;

		/** Generates an entry in the library combo box */
		TSharedRef<SWidget> GenerateFixturePatchComboBoxEntry(UDMXEntityFixturePatch* FixturePatchToAdd);

		/** Called when a fixture patch was slected */
		void OnFixturePatchSelected(UDMXEntityFixturePatch* SelectedFixturePatch, ESelectInfo::Type SelectInfo);

		/** Called when an asset was added or removed */
		void OnAssetAddedOrRemoved(const FAssetData& AssetData);

		/** Called when the library was edited */
		void OnEntitiesAddedOrRemoved(UDMXLibrary* Library, TArray<UDMXEntity*> Entities);

		/** Source for the library combo box */
		TArray<TSharedPtr<FDMXPatchToolItem>> LibrarySource;

		/** Combobox to select a patch within the library */
		TSharedPtr<SComboBox<UDMXEntityFixturePatch*>> FixturePatchComboBox;

		/** Source for the fixture patch combo box */
		TArray<TObjectPtr<UDMXEntityFixturePatch>> FixturePatchSource;

		/** The currently selected DMX Library. Useful to GC */
		TObjectPtr<UDMXLibrary> DMXLibrary;

		/** The currently selected fixture patch */
		TObjectPtr<UDMXEntityFixturePatch> FixturePatch;

		/** Timer handle for the request refresh method */
		FTimerHandle RefreshTimerHandle;

		/** The analytics provider for this tool */
		FDMXEditorToolAnalyticsProvider AnalyticsProvider;
	};
}
