// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SAssetPickerButton.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Editor.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "ScopedTransaction.h"
#include "Engine/Selection.h"
#include "AssetRegistry/AssetRegistryModule.h"

#define LOCTEXT_NAMESPACE "SAssetPickerButton"

namespace AssetPickerButtonDefs
{
	// Active Combo pin alpha
	static const float ActiveComboAlpha = 1.f;
	// InActive Combo pin alpha
	static const float InActiveComboAlpha = 0.6f;
	// Active foreground pin alpha
	static const float ActivePinForegroundAlpha = 1.f;
	// InActive foreground pin alpha
	static const float InactivePinForegroundAlpha = 0.15f;
	// Active background pin alpha
	static const float ActivePinBackgroundAlpha = 0.8f;
	// InActive background pin alpha
	static const float InactivePinBackgroundAlpha = 0.4f;
};

void SAssetPickerButton::Construct(const FArguments& InArgs)
{
	CurrentAssetValue = InArgs._CurrentAssetValue;
	OnParentIsHovered = InArgs._OnParentIsHovered;
	OnAssetSelected = InArgs._OnAssetSelected;

	AssetClass = InArgs._AssetClass;
	AllowedClasses = InArgs._AllowedClasses;
	DisallowedClasses = InArgs._DisallowedClasses;

	ChildSlot
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2,0)
		.MaxWidth(100.0f)
		[
			SAssignNew(AssetPickerAnchor, SComboButton)
			.ButtonStyle( FAppStyle::Get(), "PropertyEditor.AssetComboStyle" )
			.ForegroundColor( this, &SAssetPickerButton::OnGetComboForeground)
			.ContentPadding( FMargin(2,2,2,1) )
			.ButtonColorAndOpacity( this, &SAssetPickerButton::OnGetWidgetBackground )
			.MenuPlacement(MenuPlacement_BelowAnchor)
			.ButtonContent()
			[
				SNew(STextBlock)
				.ColorAndOpacity( this, &SAssetPickerButton::OnGetComboForeground )
				.TextStyle( FAppStyle::Get(), "PropertyEditor.AssetClass" )
				.Font( FAppStyle::GetFontStyle( "PropertyWindow.NormalFont" ) )
				.Text( this, &SAssetPickerButton::OnGetComboTextValue )
				.ToolTipText( this, &SAssetPickerButton::GetObjectToolTip )
			]
			.OnGetMenuContent(this, &SAssetPickerButton::GenerateAssetPicker)
		]
		// Use button
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(1,0)
		.VAlign(VAlign_Center)
		[
			SAssignNew(UseButton, SButton)
			.ButtonStyle( FAppStyle::Get(), "NoBorder" )
			.ButtonColorAndOpacity( this, &SAssetPickerButton::OnGetWidgetBackground )
			.OnClicked(this, &SAssetPickerButton::OnClickUse)
			.ContentPadding(1.f)
			.ToolTipText(LOCTEXT("UseSelectionTooltip", "Use asset browser selection"))
			[
				SNew(SImage)
				.ColorAndOpacity( this, &SAssetPickerButton::OnGetWidgetForeground )
				.Image( FAppStyle::GetBrush(TEXT("Icons.CircleArrowLeft")) )
			]
		]
		// Browse button
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(1,0)
		.VAlign(VAlign_Center)
		[
			SAssignNew(BrowseButton, SButton)
			.ButtonStyle( FAppStyle::Get(), "NoBorder" )
			.ButtonColorAndOpacity( this, &SAssetPickerButton::OnGetWidgetBackground )
			.OnClicked(this, &SAssetPickerButton::OnClickBrowse)
			.ContentPadding(0)
			.ToolTipText(LOCTEXT("BrowseTooltip", "Browse"))
			[
				SNew(SImage)
				.ColorAndOpacity( this, &SAssetPickerButton::OnGetWidgetForeground )
				.Image( FAppStyle::GetBrush(TEXT("Icons.Search")) )
			]
		]
	];
}

FText SAssetPickerButton::GetObjectToolTip() const
{
	const FAssetData& CurrentAssetData = GetAssetData();
	FText Value;
	if (CurrentAssetData.IsValid())
	{
		Value = FText::FromString(CurrentAssetData.GetFullName());
	}
	else
	{
		Value = FText::GetEmpty();
	}
	return Value;
}

FReply SAssetPickerButton::OnClickUse()
{
	FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();

	if (AssetClass != nullptr)
	{
		UObject* SelectedObject = GEditor->GetSelectedObjects()->GetTop(AssetClass.Get());
		if (SelectedObject != nullptr && OnAssetSelected.IsBound())
		{
			OnAssetSelected.Execute(FAssetData(SelectedObject, true));
		}
	}

	return FReply::Handled();
}

FReply SAssetPickerButton::OnClickBrowse()
{
	const FAssetData& AssetData = GetAssetData();
	if (AssetData.IsValid())
	{
		TArray<FAssetData> Objects;
		Objects.Add(AssetData);

		GEditor->SyncBrowserToObjects(Objects);
	}
	return FReply::Handled();
}

TSharedRef<SWidget> SAssetPickerButton::GenerateAssetPicker()
{
	// This class and its children are the classes that we can show objects for
	UClass* AllowedClass = AssetClass != nullptr ? AssetClass.Get() : UObject::StaticClass();

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.Filter.ClassPaths.Add(AllowedClass->GetClassPathName());
	AssetPickerConfig.bAllowNullSelection = true;
	AssetPickerConfig.Filter.bRecursiveClasses = true;
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SAssetPickerButton::OnAssetSelectedFromPicker);
	AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateSP(this, &SAssetPickerButton::OnAssetEnterPressedInPicker);
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
	AssetPickerConfig.bAllowDragging = false;

	if (AllowedClasses.Num() > 0)
	{
		// Clear out the allowed class names and have the pin's metadata override.
		AssetPickerConfig.Filter.ClassPaths.Empty();
		AssetPickerConfig.Filter.ClassPaths = AllowedClasses;
	}

	if (DisallowedClasses.Num() > 0)
	{
		AssetPickerConfig.Filter.RecursiveClassPathsExclusionSet.Append(DisallowedClasses);
	}

	return
		SNew(SBox)
		.HeightOverride(300)
		.WidthOverride(300)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
			[
				ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
			]
		];
}

void SAssetPickerButton::OnAssetSelectedFromPicker(const struct FAssetData& AssetData)
{
	const FAssetData& CurrentAssetData = GetAssetData();
	if (CurrentAssetData != AssetData)
	{
		OnAssetSelected.Execute(AssetData);
		// Close the asset picker
		AssetPickerAnchor->SetIsOpen(false);

		if (OnAssetSelected.IsBound())
		{
			OnAssetSelected.Execute(AssetData);
		}
	}
}

void SAssetPickerButton::OnAssetEnterPressedInPicker(const TArray<FAssetData>& InSelectedAssets)
{
	if (InSelectedAssets.Num() > 0)
	{
		OnAssetSelectedFromPicker(InSelectedAssets[0]);
	}
}

FText SAssetPickerButton::GetDefaultComboText() const
{
	return LOCTEXT("DefaultComboText", "Select Asset");
}

FText SAssetPickerButton::OnGetComboTextValue() const
{
	FText Value = GetDefaultComboText();

	const FAssetData& CurrentAssetData = GetAssetData();

	if (UField* Field = Cast<UField>(CurrentAssetValue.Get().Get()))
	{
		Value = Field->GetDisplayNameText();
	}
	else if (CurrentAssetData.IsValid())
	{
		Value = FText::FromString(CurrentAssetData.AssetName.ToString());
	}

	return Value;
}

bool SAssetPickerButton::GetIsParentHovered() const
{
	if (OnParentIsHovered.IsBound())
	{
		return OnParentIsHovered.Execute();
	}
	return false;
}

FSlateColor SAssetPickerButton::OnGetComboForeground() const
{
	float Alpha = IsHovered() || GetIsParentHovered() ? AssetPickerButtonDefs::ActiveComboAlpha : AssetPickerButtonDefs::InActiveComboAlpha;
	return FSlateColor(FLinearColor(1.f, 1.f, 1.f, Alpha));
}

FSlateColor SAssetPickerButton::OnGetWidgetForeground() const
{
	float Alpha = IsHovered() || GetIsParentHovered() ? AssetPickerButtonDefs::ActivePinForegroundAlpha : AssetPickerButtonDefs::InactivePinForegroundAlpha;
	return FSlateColor(FLinearColor(1.f, 1.f, 1.f, Alpha));
}

FSlateColor SAssetPickerButton::OnGetWidgetBackground() const
{
	float Alpha = IsHovered() || GetIsParentHovered() ? AssetPickerButtonDefs::ActivePinBackgroundAlpha : AssetPickerButtonDefs::InactivePinBackgroundAlpha;
	return FSlateColor(FLinearColor(1.f, 1.f, 1.f, Alpha));
}

const FAssetData& SAssetPickerButton::GetAssetData() const
{
	if (UObject* Object = CurrentAssetValue.Get().Get())
	{
		if (FSoftObjectPath(Object) != CachedAssetData.GetSoftObjectPath())
		{
			// This always uses the exact object pointed at
			CachedAssetData = FAssetData(Object, true);
		}
	}
	else
	{
		if (CachedAssetData.IsValid())
		{
			CachedAssetData = FAssetData();
		}
	}

	return CachedAssetData;
}

#undef LOCTEXT_NAMESPACE
