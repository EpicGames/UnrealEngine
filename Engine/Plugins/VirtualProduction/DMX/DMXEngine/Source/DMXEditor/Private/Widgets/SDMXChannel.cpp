// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDMXChannel.h"

#include "DMXEditorLog.h"
#include "DMXProtocolConstants.h"
#include "DMXEditorStyle.h"

#include "Brushes/SlateColorBrush.h"
#include "Engine/Font.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "UObject/ConstructorHelpers.h"


#define LOCTEXT_NAMESPACE "SDMXChannel"

const float SDMXChannel::NewValueChangedAnimDuration = 0.8f;

void SDMXChannel::Construct(const FArguments& InArgs)
{
	SetVisibility(EVisibility::SelfHitTestInvisible);
	SetCanTick(false);

	ChannelID = InArgs._ChannelID;
	Value = InArgs._Value;
	NewValueFreshness = 0.0f;

	constexpr float PaddingInfo = 3.0f;


	ChannelIDTextBlock =
		SNew(STextBlock)
		.Text(GetChannelIDText())
		.ColorAndOpacity(InArgs._ChannelIDTextColor)
		.MinDesiredWidth(24.0f)
		.Justification(ETextJustify::Center)
		.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));

	ChannelValueTextBlock =
		SNew(STextBlock)
		.Text(GetValueText())
		.ColorAndOpacity(InArgs._ValueTextColor)
		.MinDesiredWidth(24.0f)
		.Justification(ETextJustify::Center)
		.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));

	// Arrange widgets depending on bInvertChannelDisplay argument
	TSharedPtr<STextBlock> UpperWidget;
	TSharedPtr<STextBlock> LowerWidget;
	
	if (!InArgs._bShowChannelIDBottom)
	{
		UpperWidget = ChannelIDTextBlock;
		LowerWidget = ChannelValueTextBlock;
	}
	else
	{
		UpperWidget = ChannelValueTextBlock;
		LowerWidget = ChannelIDTextBlock;
	}

	ChildSlot
	[
		SNew(SBox)
		.WidthOverride(36.f)
		.HeightOverride(36.f)
		[
			SNew(SOverlay)

			// Background color image
			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SAssignNew(BarColorBorder, SImage)
				.Image(FDMXEditorStyle::Get().GetBrush(TEXT("DMXEditor.WhiteBrush")))
				.ColorAndOpacity(GetBackgroundColor())
			]

			// Info
			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding(PaddingInfo)
			[
				SNew(SVerticalBox)

				// ID Label
				+ SVerticalBox::Slot()
				.FillHeight(0.5f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					UpperWidget.ToSharedRef()
				]

				// Value Label
				+ SVerticalBox::Slot()
				.FillHeight(0.5f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					LowerWidget.ToSharedRef()
				]
			]
		]
	];
}

void SDMXChannel::SetChannelID(uint32 NewChannelID)
{
	ChannelID = NewChannelID;
}

void SDMXChannel::SetValue(uint8 NewValue)
{
	check(ChannelValueTextBlock.IsValid());

	// is NewValue a different value from current one?
	if (NewValue != Value)
	{
		Value = NewValue;

		ChannelValueTextBlock->SetText(GetValueText());

		// Activate timer to animate value bar color
		if (!AnimationTimerHandle.IsValid())
		{
			AnimationTimerHandle = RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateSP(this, &SDMXChannel::UpdateValueChangedAnim));
		}

		// restart value change animation
		NewValueFreshness = 1.0f;
	}
}

EActiveTimerReturnType SDMXChannel::UpdateValueChangedAnim(double InCurrentTime, float InDeltaTime)
{
	NewValueFreshness = FMath::Max(NewValueFreshness - InDeltaTime / NewValueChangedAnimDuration, 0.0f);
	
	BarColorBorder->SetColorAndOpacity(GetBackgroundColor());

	// disable timer when the value bar color animation ends
	if (NewValueFreshness <= 0.0f)
	{
		TSharedPtr<FActiveTimerHandle> PinnedTimerHandle = AnimationTimerHandle.Pin();
		if (PinnedTimerHandle.IsValid())
		{
			UnRegisterActiveTimer(PinnedTimerHandle.ToSharedRef());
		}
	}
	return EActiveTimerReturnType::Continue;
}

FText SDMXChannel::GetChannelIDText() const
{
	return FText::AsNumber(ChannelID);
}

FText SDMXChannel::GetValueText() const
{
	return FText::AsNumber(Value);
}

FSlateColor SDMXChannel::GetBackgroundColor() const
{
	const float CurrentPercent = static_cast<float>(Value) / static_cast<float>(DMX_MAX_CHANNEL_VALUE);

	// totally transparent when 0
	if (CurrentPercent <= 0.0f)
	{
		return FLinearColor(0, 0, 0, 0);
	}

	// Intensities to be animated when a new value is set and then multiplied by the background color
	static const float NormalIntensity = 0.3f;
	static const float FreshValueIntensity = 0.7f;
	// lerp intensity depending on NewValueFreshness^2 to make it pop for a while when it has just been updated
	const float ValueFreshnessIntensity = FMath::Lerp(NormalIntensity, FreshValueIntensity, NewValueFreshness * NewValueFreshness);

	// color variations for low and high channel values
	static const FVector LowValueColor = FVector(0, 0.045f, 0.15f);
	static const FVector HighValueColor = FVector(0, 0.3f, 1.0f);
	const FVector ColorFromChannelValue = FMath::Lerp(LowValueColor, HighValueColor, CurrentPercent);

	// returning a FVector, a new FSlateColor will be created from it with (RGB = vector, Alpha = 1.0)
	FVector Result = ColorFromChannelValue * ValueFreshnessIntensity;
	return FSlateColor(FLinearColor(Result.X, Result.Y, Result.Z));
}

#undef LOCTEXT_NAMESPACE
