// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FActiveTimerHandle;
class SImage;
class STextBlock;


/** 
 *Shows a channel in a box with the channel value top, the channel ID bottom
 *
 *	*-------*
 *	| Value |
 *	| ----- |
 *	|  ID   |
 *	*-------*
 */
class SDMXChannel
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMXChannel)
		: _ChannelID(0)
		, _Value(0)
		, _bShowChannelIDBottom(false)
		, _ChannelIDTextColor(FLinearColor::White.CopyWithNewOpacity(.6f))
		, _ValueTextColor(FLinearColor::White.CopyWithNewOpacity(.9f))
		{}

		/** The channel ID this widget represents */
		SLATE_ARGUMENT(uint32, ChannelID)

		/** The current value from the channel */
		SLATE_ARGUMENT(uint8, Value)

		/** If true, draws the channel number below the value */
		SLATE_ARGUMENT(bool, bShowChannelIDBottom)

		/** Color of the Channel ID Text */
		SLATE_ATTRIBUTE(FSlateColor, ChannelIDTextColor)

		/** Color of the Value Text */
		SLATE_ATTRIBUTE(FSlateColor, ValueTextColor)

	SLATE_END_ARGS()

	/** Constructs the widget */
	void Construct(const FArguments& InArgs);

	/** Sets the channel ID this widget represents */
	void SetChannelID(uint32 NewChannelID);

	/** Gets the channel ID this widget represents */
	uint32 GetChannelID() const
	{
		return ChannelID;
	}

	/** Sets the current value from the channel */
	void SetValue(uint8 NewValue);

	/** Gets the current value from the channel */
	uint8 GetValue() const
	{
		return Value;
	}

	/**
	 * Updates the variable that controls the color animation progress for the Value Bar.
	 * This is called by a timer.
	 */
	EActiveTimerReturnType UpdateValueChangedAnim(double InCurrentTime, float InDeltaTime);

private:
	/** The channel ID this widget represents */
	uint32 ChannelID;

	/** The current value from the channel */
	uint8 Value;

	/** The ProgressBar widget to display the channel value graphically */
	TSharedPtr<SImage> BarColorBorder;

	/**
	 * Used to animate the color when the value changes.
	 * 0..1 range: 1 = value has just changed, 0 = standard color
	 */
	float NewValueFreshness;

	/** How long it takes to become standard color again after a new value is set */
	static const float NewValueChangedAnimDuration;

	/** Used to stop the animation timer once the animation is completed */
	TWeakPtr<FActiveTimerHandle> AnimationTimerHandle;

private:
	/** Returns the channel ID in Text form to display it in the UI */
	FText GetChannelIDText() const;

	/** Returns the channel value in Text form to display it in the UI */
	FText GetValueText() const;

	/** Returns the fill color for the ValueBar */
	FSlateColor GetBackgroundColor() const;

	/** Textblock that shows the Channel ID */
	TSharedPtr<STextBlock> ChannelIDTextBlock;

	/** Textblock that shows the Value */
	TSharedPtr<STextBlock> ChannelValueTextBlock;
};
