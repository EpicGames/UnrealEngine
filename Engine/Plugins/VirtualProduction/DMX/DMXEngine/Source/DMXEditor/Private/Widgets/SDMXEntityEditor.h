// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class UDMXEntity;


/** Base class for widgets that edit a specific entity */
class SDMXEntityEditor
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMXEntityEditor)
	{}

	SLATE_END_ARGS()

	/** Constructs the widget */
	void Construct(const FArguments& InArgs)
	{};

	/**
	 * Refreshes the list of entities to display any added entities, select the new entity by name
	 * and initiates a rename on the selected Entity node
	 */
	virtual void RequestRenameOnNewEntity(const UDMXEntity* InEntity, ESelectInfo::Type SelectionType) = 0;

	/** Selects an entity in this editor tab's list */
	virtual void SelectEntity(UDMXEntity* InEntity, ESelectInfo::Type InSelectionType = ESelectInfo::Type::Direct) = 0;

	/** Selects Entities in this editor tab's list */
	virtual void SelectEntities(const TArray<UDMXEntity*>& InEntities, ESelectInfo::Type SelectionType = ESelectInfo::Type::Direct) = 0;

	/** Returns the selected entities on this editor tab */
	virtual TArray<UDMXEntity*> GetSelectedEntities() const = 0;
};
