// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDMXEntityTreeViewBase.h"

#include "DMXEditor.h"
#include "DragDrop/DMXEntityDragDropOp.h"
#include "Library/DMXLibrary.h"

#include "Styling/AppStyle.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Images/SImage.h"


#define LOCTEXT_NAMESPACE "SDMXEntityTreeViewBase"

void SDMXEntityTreeViewBase::Construct(const FArguments& InArgs)
{
	// Initialize Widget input variables
	DMXEditor = InArgs._DMXEditor;

	// listen to common editor shortcuts for copy/paste etc
	CommandList = MakeShared<FUICommandList>();
	CommandList->MapAction(FGenericCommands::Get().Cut,
		FUIAction(FExecuteAction::CreateSP(this, &SDMXEntityTreeViewBase::OnCutSelectedNodes),
			FCanExecuteAction::CreateSP(this, &SDMXEntityTreeViewBase::CanCutNodes))
	);
	CommandList->MapAction(FGenericCommands::Get().Copy,
		FUIAction(FExecuteAction::CreateSP(this, &SDMXEntityTreeViewBase::OnCopySelectedNodes),
			FCanExecuteAction::CreateSP(this, &SDMXEntityTreeViewBase::CanCopyNodes))
	);
	CommandList->MapAction(FGenericCommands::Get().Paste,
		FUIAction(FExecuteAction::CreateSP(this, &SDMXEntityTreeViewBase::OnPasteNodes),
			FCanExecuteAction::CreateSP(this, &SDMXEntityTreeViewBase::CanPasteNodes))
	);
	CommandList->MapAction(FGenericCommands::Get().Duplicate,
		FUIAction(FExecuteAction::CreateSP(this, &SDMXEntityTreeViewBase::OnDuplicateNodes),
			FCanExecuteAction::CreateSP(this, &SDMXEntityTreeViewBase::CanDuplicateNodes))
	);
	CommandList->MapAction(FGenericCommands::Get().Delete,
		FUIAction(FExecuteAction::CreateSP(this, &SDMXEntityTreeViewBase::OnDeleteNodes),
			FCanExecuteAction::CreateSP(this, &SDMXEntityTreeViewBase::CanDeleteNodes))
	);
	CommandList->MapAction(FGenericCommands::Get().Rename,
		FUIAction(FExecuteAction::CreateSP(this, &SDMXEntityTreeViewBase::OnRenameNode),
			FCanExecuteAction::CreateSP(this, &SDMXEntityTreeViewBase::CanRenameNode))
	);

	GEditor->RegisterForUndo(this);

	TSharedPtr<SBorder> HeaderBox = SNew(SBorder)
		.Padding(0)
		.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
		.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f))
		[
			SNew(SHorizontalBox)

			// [+ Add New] button
			+ SHorizontalBox::Slot()
			.Padding(3.0f)
			.AutoWidth()
			.HAlign(HAlign_Left)
			[
				GenerateAddNewEntityButton()
			]

			// Filter box
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.Padding(3.0f)
			[
				SAssignNew(FilterBox, SSearchBox)
				.HintText(LOCTEXT("SearchEntitiesHint", "Search entities"))
				.OnTextChanged(this, &SDMXEntityTreeViewBase::OnFilterTextChanged)
			]

		];

	// Tree widget which displays the entities in their categories (e.g. protocol),
	// and also controls selection and drag/drop
	RootNode = MakeShared<FDMXEntityTreeRootNode>();
	EntitiesTreeWidget = SNew(STreeView<TSharedPtr<FDMXEntityTreeNodeBase>>)
		.TreeItemsSource(&RootNode->GetChildren())
		.SelectionMode(ESelectionMode::Multi)
		.OnGenerateRow(this, &SDMXEntityTreeViewBase::OnGenerateRow)
		.OnGetChildren(this, &SDMXEntityTreeViewBase::OnGetChildren)
		.OnExpansionChanged(this, &SDMXEntityTreeViewBase::OnExpansionChanged)
		.OnSelectionChanged(this, &SDMXEntityTreeViewBase::OnSelectionChanged)
		.OnContextMenuOpening(this, &SDMXEntityTreeViewBase::OnContextMenuOpen)
		.HighlightParentNodesForSelection(false);

	ChildSlot
	[
		SNew(SVerticalBox)
		
		+ SVerticalBox::Slot()
		.Padding(0)
		.AutoHeight()
		.HAlign(HAlign_Fill)
		[
			HeaderBox.ToSharedRef()
		]

		+ SVerticalBox::Slot()
		.Padding(0)
		[
			SNew(SBorder)
			.Padding(2.0f)
			.BorderImage(FAppStyle::GetBrush("SCSEditor.TreePanel"))
			[
				EntitiesTreeWidget.ToSharedRef()
			]
		]
	];

	UpdateTree();

	// Make sure we know when tabs become active to update details tab
	OnActiveTabChangedDelegateHandle = FGlobalTabmanager::Get()->OnActiveTabChanged_Subscribe(FOnActiveTabChanged::FDelegate::CreateSP(this, &SDMXEntityTreeViewBase::OnActiveTabChanged));
}

SDMXEntityTreeViewBase::~SDMXEntityTreeViewBase()
{
	FGlobalTabmanager::Get()->OnActiveTabChanged_Unsubscribe(OnActiveTabChangedDelegateHandle);
	GEditor->UnregisterForUndo(this);
}

void SDMXEntityTreeViewBase::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (RefreshTreeViewState != EDMXRefreshTreeViewState::NoRefreshRequested)
	{
		const bool bRegenerateNodes = RefreshTreeViewState == EDMXRefreshTreeViewState::RegenerateNodes;
		UpdateTreeInternal(bRegenerateNodes);

		RefreshTreeViewState = EDMXRefreshTreeViewState::NoRefreshRequested;
	}

	if (PendingSelection.Num() > 0)
	{
		// Clear search filter
		FilterBox->SetText(FText::GetEmpty());

		EntitiesTreeWidget->ClearSelection();
		EntitiesTreeWidget->SetItemSelection(PendingSelection, true, ESelectInfo::OnMouseClick);

		PendingSelection.Reset();
	}
	
	if (PendingExpandItemMap.Num() > 0)
	{
		for (const TTuple<TSharedRef<FDMXEntityTreeNodeBase>, bool>& NodeToExpansionStatePair : PendingExpandItemMap)
		{
			EntitiesTreeWidget->SetItemExpansion(NodeToExpansionStatePair.Key, NodeToExpansionStatePair.Value);
		}

		PendingExpandItemMap.Reset();
	}
}

void SDMXEntityTreeViewBase::UpdateTreeInternal(bool bRegenerateTreeNodes /*= true*/)
{
	// the DMXEditor may have been closed, no need to update the tree
	if (DMXEditor.IsValid() && EntitiesTreeWidget.IsValid())
	{
		if (bRegenerateTreeNodes)
		{
			// Obtain the set of expandable tree nodes that are currently collapsed
			TSet<TSharedPtr<FDMXEntityTreeNodeBase>> CollapsedTreeNodes;
			GetCollapsedNodes(CollapsedTreeNodes);

			// Obtain the list of selected items
			TArray<TSharedPtr<FDMXEntityTreeNodeBase>> SelectedTreeNodes = EntitiesTreeWidget->GetSelectedItems();

			RootNode->ClearChildren();
			RebuildNodes(RootNode);

			// Restore the previous expansion state on the new tree nodes
			TArray<TSharedPtr<FDMXEntityTreeNodeBase>> CollapsedTreeNodeArray = CollapsedTreeNodes.Array();
			for (int32 i = 0; i < CollapsedTreeNodeArray.Num(); ++i)
			{
				// Look for a category match in the new hierarchy; if found, mark it as collapsed to match the previous setting
				TSharedPtr<FDMXEntityTreeNodeBase> NodeToExpandPtr = FindNodeByName(CollapsedTreeNodeArray[i]->GetDisplayNameText());
				if (NodeToExpandPtr.IsValid())
				{
					EntitiesTreeWidget->SetItemExpansion(NodeToExpandPtr, false);
				}
				else
				{
					EntitiesTreeWidget->SetItemExpansion(NodeToExpandPtr, true);
				}
			}
		}

		// Refresh the Tree Widget
		EntitiesTreeWidget->RequestTreeRefresh();
	}
}

void SDMXEntityTreeViewBase::UpdateTree(bool bRegenerateTreeNodes)
{
	if (bRegenerateTreeNodes)
	{
		RefreshTreeViewState = EDMXRefreshTreeViewState::RegenerateNodes;
	}
	else
	{
		RefreshTreeViewState = EDMXRefreshTreeViewState::UpdateNodes;
	}
}

TArray<TSharedPtr<FDMXEntityTreeEntityNode>> SDMXEntityTreeViewBase::GetEntityNodes(TSharedPtr<FDMXEntityTreeNodeBase> ParentNode) const
{
	TArray<TSharedPtr<FDMXEntityTreeEntityNode>> Result;

	ParentNode = ParentNode.IsValid() ? ParentNode : GetRootNode();
	for (const TSharedPtr<FDMXEntityTreeNodeBase>& Child : ParentNode->GetChildren())
	{
		if (Child.IsValid())
		{
			if (Child->GetNodeType() == FDMXEntityTreeNodeBase::ENodeType::EntityNode)
			{
				Result.Add(StaticCastSharedPtr<FDMXEntityTreeEntityNode>(Child));
			}

			// Recursive for all childs
			Result.Append(GetEntityNodes(Child));
		}
	}

	return Result;
}

TSharedPtr<FDMXEntityTreeCategoryNode> SDMXEntityTreeViewBase::FindCategoryNodeOfEntity(UDMXEntity* Entity) const
{
	TSharedPtr<FDMXEntityTreeEntityNode> EntityNode = FindNodeByEntity(Entity);
	if (EntityNode.IsValid())
	{
		TSharedPtr<FDMXEntityTreeNodeBase> CategoryNode = EntityNode->GetParent().Pin();
		if (CategoryNode.IsValid())
		{
			check(CategoryNode->GetNodeType() == FDMXEntityTreeNodeBase::ENodeType::CategoryNode);
			return StaticCastSharedPtr<FDMXEntityTreeCategoryNode>(CategoryNode);
		}
	}

	return nullptr;
}

TSharedPtr<FDMXEntityTreeEntityNode> SDMXEntityTreeViewBase::FindNodeByEntity(const UDMXEntity* Entity, TSharedPtr<FDMXEntityTreeNodeBase> StartNode /*= nullptr*/) const
{
	if (Entity)
	{
		// Start at root node if none was provided
		if (!StartNode.IsValid())
		{
			StartNode = RootNode;
		}

		// Test the StartNode
		if (StartNode.IsValid() && StartNode->GetNodeType() == FDMXEntityTreeNodeBase::ENodeType::EntityNode)
		{
			TSharedPtr<FDMXEntityTreeEntityNode> EntityNode = StaticCastSharedPtr<FDMXEntityTreeEntityNode>(StartNode);
			if (EntityNode->GetEntity() == Entity)
			{
				return EntityNode;
			}
		}

		// Test children 
		for (const TSharedPtr<FDMXEntityTreeNodeBase>& ChildNode : StartNode->GetChildren())
		{
			TSharedPtr<FDMXEntityTreeEntityNode> EntityNode = FindNodeByEntity(Entity, ChildNode);
			if (EntityNode.IsValid())
			{
				return EntityNode;
			}
		}
	}

	return nullptr;
}

TSharedPtr<FDMXEntityTreeNodeBase> SDMXEntityTreeViewBase::FindNodeByName(const FText& InName, TSharedPtr<FDMXEntityTreeNodeBase> StartNode /*= nullptr*/) const
{
	if (!InName.IsEmpty())
	{
		// Start at root node if none was provided
		if (!StartNode.IsValid())
		{
			StartNode = RootNode;
		}

		// Test the StartNode
		if (StartNode.IsValid())
		{
			// Check to see if the given entity matches the given tree node
			if (StartNode->GetDisplayNameText().CompareTo(InName) == 0)
			{
				return StartNode;
			}
		}

		// Test children recursive
		for (const TSharedPtr<FDMXEntityTreeNodeBase>& ChildNode : StartNode->GetChildren())
		{
			TSharedPtr<FDMXEntityTreeNodeBase> Node = FindNodeByName(InName, ChildNode);
			if (Node.IsValid())
			{
				return Node;
			}
		}
	}

	return nullptr;
}

void SDMXEntityTreeViewBase::SelectItemByNode(const TSharedRef<FDMXEntityTreeNodeBase>& Node, ESelectInfo::Type SelectInfo)
{
	PendingSelection = { Node };
}

void SDMXEntityTreeViewBase::SelectItemByEntity(const UDMXEntity* Entity, ESelectInfo::Type SelectInfo /*= ESelectInfo::Direct*/)
{
	// Check if the tree is being told to clear
	if (!Entity)
	{
		EntitiesTreeWidget->ClearSelection();
	}
	else
	{
		const TSharedPtr<FDMXEntityTreeNodeBase>& ItemNode = FindNodeByEntity(Entity);
		if (ItemNode.IsValid())
		{
			PendingSelection = { ItemNode };
		}
	}
}

void SDMXEntityTreeViewBase::SelectItemsByEntities(const TArray<UDMXEntity*>& InEntities, ESelectInfo::Type SelectInfo /*= ESelectInfo::Direct*/)
{
	// Check if the tree is being told to clear
	if (InEntities.Num() == 0)
	{
		EntitiesTreeWidget->ClearSelection();
	}
	else
	{
		TArray<TSharedPtr<FDMXEntityTreeNodeBase>> SelectedNodes;
		for (UDMXEntity* Entity : InEntities)
		{
			if (TSharedPtr<FDMXEntityTreeNodeBase> EntityNode = FindNodeByEntity(Entity))
			{
				const TSharedPtr<FDMXEntityTreeNodeBase>& ItemNode = FindNodeByEntity(Entity);
				SelectedNodes.Add(EntityNode);
			}
		}
		PendingSelection = SelectedNodes;
	}
}

void SDMXEntityTreeViewBase::SelectItemByName(const FString& ItemName, ESelectInfo::Type SelectInfo /*= ESelectInfo::Direct*/)
{
	// Check if the tree is being told to clear
	if (ItemName.IsEmpty())
	{
		EntitiesTreeWidget->ClearSelection();
	}
	else
	{
		const TSharedPtr<FDMXEntityTreeNodeBase>& ItemNode = FindNodeByName(FText::FromString(ItemName));
		if (ItemNode.IsValid())
		{
			PendingSelection = { ItemNode };
		}
	}
}

TArray<UDMXEntity*> SDMXEntityTreeViewBase::GetSelectedEntities() const
{
	TArray<UDMXEntity*> SelectedEntities;

	if (EntitiesTreeWidget.IsValid())
	{
		const TArray<TSharedPtr<FDMXEntityTreeNodeBase>>& SelectedItems = EntitiesTreeWidget->GetSelectedItems();
		for (const TSharedPtr<FDMXEntityTreeNodeBase>& Node : SelectedItems)
		{
			if (Node.IsValid() && Node->GetNodeType() == FDMXEntityTreeNodeBase::ENodeType::EntityNode)
			{
				TSharedPtr<FDMXEntityTreeEntityNode> EntityNode = StaticCastSharedPtr<FDMXEntityTreeEntityNode>(Node);
				SelectedEntities.Add(EntityNode->GetEntity());
			}
		}
	}

	return SelectedEntities;
}


bool SDMXEntityTreeViewBase::IsNodeSelected(const TSharedPtr<FDMXEntityTreeNodeBase>& Node) const
{
	return EntitiesTreeWidget->IsItemSelected(Node);
}

TArray<TSharedPtr<FDMXEntityTreeNodeBase>> SDMXEntityTreeViewBase::GetSelectedNodes() const
{
	return EntitiesTreeWidget->GetSelectedItems();
}

int32 SDMXEntityTreeViewBase::GetNumExpandedNodes() const
{
	TSet<TSharedPtr<FDMXEntityTreeNodeBase>> ExpandedTreeNodes;
	EntitiesTreeWidget->GetExpandedItems(ExpandedTreeNodes);

	return ExpandedTreeNodes.Num();
}

void SDMXEntityTreeViewBase::SetNodeExpansion(TSharedPtr<FDMXEntityTreeNodeBase> Node, bool bExpandItem)
{
	if (Node.IsValid())
	{
		PendingExpandItemMap.Add(Node.ToSharedRef(), bExpandItem);
	}
}

void SDMXEntityTreeViewBase::RequestScrollIntoView(TSharedPtr<FDMXEntityTreeNodeBase> Node)
{
	EntitiesTreeWidget->RequestScrollIntoView(Node);
}


FText SDMXEntityTreeViewBase::GetFilterText() const
{
	return FilterBox->GetText();
}

UDMXLibrary* SDMXEntityTreeViewBase::GetDMXLibrary() const
{
	if (DMXEditor.IsValid())
	{
		TSharedPtr<FDMXEditor> PinnedEditor(DMXEditor.Pin());
		if (PinnedEditor.IsValid())
		{
			return PinnedEditor->GetDMXLibrary();
		}
	}
	return nullptr;
}

FReply SDMXEntityTreeViewBase::OnEntitiesDragged(TSharedPtr<FDMXEntityTreeNodeBase> Node, const FPointerEvent& MouseEvent)
{
	if (Node.IsValid() && Node->GetNodeType() == FDMXEntityTreeNodeBase::ENodeType::EntityNode)
	{
		TArray<TSharedPtr<FDMXEntityTreeNodeBase>> SelectedItems = EntitiesTreeWidget->GetSelectedItems();
		TArray<TWeakObjectPtr<UDMXEntity>> DraggedEntities;

		for (TSharedPtr<FDMXEntityTreeNodeBase> SelectedItem : SelectedItems)
		{
			if (ensureMsgf(SelectedItem->GetNodeType() == FDMXEntityTreeNodeBase::ENodeType::EntityNode, TEXT("Unexpected non-entity node is selected")))
			{
				TSharedPtr<FDMXEntityTreeEntityNode> EntityNode = StaticCastSharedPtr<FDMXEntityTreeEntityNode>(SelectedItem);
				DraggedEntities.Add(EntityNode->GetEntity());
			}
		}

		// If no entities are selected, use the dragged entity instead
		if (DraggedEntities.Num() == 0)
		{
			TSharedPtr<FDMXEntityTreeEntityNode> EntityNode = StaticCastSharedPtr<FDMXEntityTreeEntityNode>(Node);
			DraggedEntities.Add(EntityNode->GetEntity());
		}

		TSharedRef<FDMXEntityDragDropOperation> DragOperation = MakeShared<FDMXEntityDragDropOperation>(GetDMXLibrary(), DraggedEntities);
		return FReply::Handled().BeginDragDrop(DragOperation);
	}

	return FReply::Unhandled();
}

FReply SDMXEntityTreeViewBase::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SDMXEntityTreeViewBase::PostUndo(bool bSuccess)
{
	if (UDMXLibrary* Library = GetDMXLibrary())
	{
		Library->Modify();
	}

	UpdateTree();
}

void SDMXEntityTreeViewBase::PostRedo(bool bSuccess)
{
	if (UDMXLibrary* Library = GetDMXLibrary())
	{
		Library->Modify();
	}

	UpdateTree();
}

void SDMXEntityTreeViewBase::OnGetChildren(TSharedPtr<FDMXEntityTreeNodeBase> InNode, TArray<TSharedPtr<FDMXEntityTreeNodeBase>>& OutChildren)
{
	if (InNode.IsValid())
	{
		const TArray<TSharedPtr<FDMXEntityTreeNodeBase>>& Children = InNode->GetChildren();

		if (!GetFilterText().IsEmpty())
		{
			OutChildren.Reserve(Children.Num());

			for (const TSharedPtr<FDMXEntityTreeNodeBase>& Child : Children)
			{
				if (!Child->IsFlaggedForFiltration())
				{
					OutChildren.Add(Child);
				}
			}
		}
		else
		{
			OutChildren = Children;
		}
	}
	else
	{
		OutChildren.Empty();
	}
}

void SDMXEntityTreeViewBase::OnExpansionChanged(TSharedPtr<FDMXEntityTreeNodeBase> Node, bool bInExpansionState)
{
	// Only applies when there's no filtering
	if (Node.IsValid() && GetFilterText().IsEmpty())
	{
		Node->SetExpansionState(bInExpansionState);
	}
}	

bool SDMXEntityTreeViewBase::RefreshFilteredState(TSharedPtr<FDMXEntityTreeNodeBase> Node, bool bRecursive)
{
	FString FilterText = FText::TrimPrecedingAndTrailing(GetFilterText()).ToString();
	TArray<FString> FilterTerms;
	FilterText.ParseIntoArray(FilterTerms, TEXT(" "), /*CullEmpty =*/true);

	struct RefreshFilteredState_Inner
	{
		static void RefreshFilteredState(TSharedPtr<FDMXEntityTreeNodeBase> InNode, const TArray<FString>& InFilterTerms, bool bInRecursive)
		{
			if (bInRecursive)
			{
				for (TSharedPtr<FDMXEntityTreeNodeBase> Child : InNode->GetChildren())
				{
					RefreshFilteredState(Child, InFilterTerms, bInRecursive);
				}
			}

			FString DisplayStr = InNode->GetDisplayNameString();

			bool bIsFilteredOut = false;
			for (const FString& FilterTerm : InFilterTerms)
			{
				if (!DisplayStr.Contains(FilterTerm))
				{
					bIsFilteredOut = true;
				}
			}
			// if we're not recursing, then assume this is for a new node and we need to update the parent
			// otherwise, assume the parent was hit as part of the recursion
			InNode->UpdateCachedFilterState(!bIsFilteredOut, /*bUpdateParent =*/!bInRecursive);
		}
	};

	RefreshFilteredState_Inner::RefreshFilteredState(Node, FilterTerms, bRecursive);
	return Node->IsFlaggedForFiltration();
}

void SDMXEntityTreeViewBase::OnFilterTextChanged(const FText& InFilterText)
{
	for (TSharedPtr<FDMXEntityTreeNodeBase> Node : RootNode->GetChildren())
	{
		RefreshFilteredState(Node, true);
	}

	// Clears selection to make UpdateTree automatically select the first visible node
	EntitiesTreeWidget->ClearSelection();

	constexpr bool bRegenerateTreeNodes = false;
	UpdateTree(bRegenerateTreeNodes);
}

void SDMXEntityTreeViewBase::GetCollapsedNodes(TSet<TSharedPtr<FDMXEntityTreeNodeBase>>& OutCollapsedNodes, TSharedPtr<FDMXEntityTreeNodeBase> InParentNodePtr /*= nullptr*/) const
{
	if (!InParentNodePtr.IsValid())
	{
		InParentNodePtr = RootNode;
	}

	for (const TSharedPtr<FDMXEntityTreeNodeBase>& Node : InParentNodePtr->GetChildren())
	{
		if (Node->GetChildren().Num() > 0)
		{
			if (!EntitiesTreeWidget->IsItemExpanded(Node))
			{
				OutCollapsedNodes.Add(Node);
			}
			else // not collapsed. Check children
			{
				GetCollapsedNodes(OutCollapsedNodes, Node);
			}
		}
	}
}

void SDMXEntityTreeViewBase::OnActiveTabChanged(TSharedPtr<SDockTab> PreviouslyActive, TSharedPtr<SDockTab> NewlyActivated)
{
	if (IsInTab(NewlyActivated))
	{
		UpdateTree();
	}
}

bool SDMXEntityTreeViewBase::IsInTab(TSharedPtr<SDockTab> InDockTab) const
{
	// Too many hierarchy levels to do it with a recursive function. Crashes with Stack Overflow.
	// Using loop instead.

	if (InDockTab.IsValid())
	{
		// Tab content that should be a parent of this widget on some level
		TSharedPtr<SWidget> TabContent = InDockTab->GetContent();
		// Current parent being checked against
		TSharedPtr<SWidget> CurrentParent = GetParentWidget();

		while (CurrentParent.IsValid())
		{
			if (CurrentParent == TabContent)
			{
				return true;
			}
			CurrentParent = CurrentParent->GetParentWidget();
		}

		// reached top widget (parent is invalid) and none was the tab
		return false;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
