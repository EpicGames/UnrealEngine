// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DMXEntityTreeNode.h"

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

class FDMXEditor;
class UDMXLibrary;

class FMenuBuilder;
class FUICommandList;
class ITableRow;
class SComboButton;
class SDockTab;
class SSearchBox;
class STableViewBase;
template<typename ItemType> class STreeView;


/** Base class for a tree view of entities, using DMXEntityTreeNode as its Nodes. */
class SDMXEntityTreeViewBase
	: public SCompoundWidget
	, public FEditorUndoClient
{
public:
	DECLARE_DELEGATE_OneParam(FDMXOnSelectionChanged, const TArray<UDMXEntity*>&);

	SLATE_BEGIN_ARGS(SDMXEntityTreeViewBase)
	{}
		/** The DMX Editor that owns this widget */
		SLATE_ARGUMENT(TWeakPtr<FDMXEditor>, DMXEditor)

	SLATE_END_ARGS()

	/** Constructs the widget */
	void Construct(const FArguments& InArgs);

	/** Destructor */
	virtual ~SDMXEntityTreeViewBase();

protected:
	//~ Begin SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget interface

	/** Internally refreshes the the tree widget, actually carrying out the update */
	void UpdateTreeInternal(bool bRegenerateTreeNodes = true);

public:
	/** Updates the Tree Widget */
	void UpdateTree(bool bRegenerateTreeNodes = true);

	/**
	 * Creates a new category node directly under the passed parent or just retrieves it if existent.
	 * If InParentNodePtr is null, a root category node is created/retrieved.
	 * Stores a value, so when something is dropped on the category the value can be retrieved.
	 */
	template <typename ValueType>
	TSharedRef<FDMXEntityTreeCategoryNode> GetOrCreateCategoryNode(const FDMXEntityTreeCategoryNode::ECategoryType InCategoryType, const FText InCategoryName, ValueType Value, TSharedPtr<FDMXEntityTreeNodeBase> InParentNode = nullptr, const FText& InToolTip = FText::GetEmpty())
	{
		for (const TSharedPtr<FDMXEntityTreeNodeBase>& Node : InParentNode.IsValid() ? InParentNode->GetChildren() : RootNode->GetChildren())
		{
			if (Node.IsValid() && Node->GetNodeType() == FDMXEntityTreeNodeBase::ENodeType::CategoryNode)
			{
				TSharedPtr<FDMXEntityTreeCategoryNode> ExistingCategoryNode = StaticCastSharedPtr<FDMXEntityTreeCategoryNode>(Node);
				if (ExistingCategoryNode->GetCategoryType() == InCategoryType && ExistingCategoryNode->GetDisplayNameText().CompareTo(InCategoryName) == 0)
				{
					return ExistingCategoryNode.ToSharedRef();
				}
			}
		}

		// Didn't find an existing node. Add one.
		TSharedRef<FDMXEntityTreeCategoryNode> NewNode = MakeShared<FDMXEntityTreeCategoryNode>(InCategoryType, InCategoryName, Value, InToolTip);
		if (InParentNode.IsValid())
		{
			InParentNode->AddChild(NewNode);
		}
		else
		{
			RootNode->AddChild(NewNode);
		}

		constexpr bool bRefreshFilteredStateRecursive = false;
		RefreshFilteredState(NewNode, bRefreshFilteredStateRecursive);

		constexpr bool bExpandItem = false;
		EntitiesTreeWidget->SetItemExpansion(NewNode, bExpandItem);
		NewNode->SetExpansionState(bExpandItem);

		return NewNode;
	}

	/** Returns the Entity Nodes in the Tree */
	TArray<TSharedPtr<FDMXEntityTreeEntityNode>> GetEntityNodes(TSharedPtr<FDMXEntityTreeNodeBase> ParentNode = nullptr) const;

	/** Returns the category node of the entity, or null if is not in the list */
	TSharedPtr<FDMXEntityTreeCategoryNode> FindCategoryNodeOfEntity(UDMXEntity* Entity) const;

	/** Helper method to recursively find a tree node for the given DMX Entity starting at the given tree node */
	TSharedPtr<FDMXEntityTreeEntityNode> FindNodeByEntity(const UDMXEntity* Entity, TSharedPtr<FDMXEntityTreeNodeBase> StartNode = nullptr) const;

	/** Helper method to recursively find a tree node with the given name starting at the given tree node */
	TSharedPtr<FDMXEntityTreeNodeBase> FindNodeByName(const FText& Name, TSharedPtr<FDMXEntityTreeNodeBase> StartNode = nullptr) const;

	/** Selects an item */
	void SelectItemByNode(const TSharedRef<FDMXEntityTreeNodeBase>& Node, ESelectInfo::Type SelectInfo = ESelectInfo::Direct);

	/** Selects an item by Entity */
	void SelectItemByEntity(const UDMXEntity* InEntity, ESelectInfo::Type SelectInfo = ESelectInfo::Direct);

	/** Selects items by an Array of Entites */
	void SelectItemsByEntities(const TArray<UDMXEntity*>& InEntities, ESelectInfo::Type SelectInfo = ESelectInfo::Direct);

	/** Selects an item by name */
	void SelectItemByName(const FString& ItemName, ESelectInfo::Type SelectInfo = ESelectInfo::Direct);

	/** Get only the valid selected entities */
	TArray<UDMXEntity*> GetSelectedEntities() const;

	/** Returns true if the node is selected */
	bool IsNodeSelected(const TSharedPtr<FDMXEntityTreeNodeBase>& Node) const;

	/** Returns the selcted nodes */
	TArray<TSharedPtr<FDMXEntityTreeNodeBase>> GetSelectedNodes() const;

	/** Returns the number of expanded nodes */
	int32 GetNumExpandedNodes() const;

	/** Sets the node to be expaned or collapsed */
	void SetNodeExpansion(TSharedPtr<FDMXEntityTreeNodeBase> Node, bool bExpandItem);

	/** Requests to scroll the node into view */
	void RequestScrollIntoView(TSharedPtr<FDMXEntityTreeNodeBase> Node);

	/** Gets current filter from the FilterBox */
	FText GetFilterText() const;

	/** Gets the DMX Library object being edited */
	UDMXLibrary* GetDMXLibrary() const;

	/** Returns the root node of the tree */
	const TSharedPtr<FDMXEntityTreeRootNode>& GetRootNode() const { return RootNode; }

	/** Handler for when an entity from the list is dragged */
	virtual FReply OnEntitiesDragged(TSharedPtr<FDMXEntityTreeNodeBase> InNodePtr, const FPointerEvent& MouseEvent);

protected:
	//~ Begin SWidget interface
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);
	//~ End SWidget Interface

	//~ Begin FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	// End of FEditorUndoClient
	
	/** Generates the Add New Entity Button that is displayed above the Tree View */
	virtual TSharedRef<SWidget> GenerateAddNewEntityButton() = 0;

	/** Rebuils of the root node */
	virtual void RebuildNodes(const TSharedPtr<FDMXEntityTreeRootNode>& RootNode) = 0;

	/** Generates a row in the tree view */
	virtual TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FDMXEntityTreeNodeBase> Node, const TSharedRef<STableViewBase>& OwnerTable) = 0;

	/** Gets the children of specified Node */
	virtual void OnGetChildren(TSharedPtr<FDMXEntityTreeNodeBase> InNode, TArray<TSharedPtr<FDMXEntityTreeNodeBase>>& OutChildren);

	/** Called when the expansion of a Node changed */
	virtual void OnExpansionChanged(TSharedPtr<FDMXEntityTreeNodeBase> Node, bool bInExpansionState);

	/** Called to display a context menu when right clicking an Entity */
	virtual TSharedPtr<SWidget> OnContextMenuOpen() = 0;

	/** Called when Entities were selected in the Tree View */
	virtual void OnSelectionChanged(TSharedPtr<FDMXEntityTreeNodeBase> InSelectedNodePtr, ESelectInfo::Type SelectInfo) = 0;

	/** Cut selected node(s) */
	virtual void OnCutSelectedNodes() = 0;
	virtual bool CanCutNodes() const = 0;

	/** Copy selected node(s) */
	virtual void OnCopySelectedNodes() = 0;
	virtual bool CanCopyNodes() const = 0;

	/** Pastes previously copied node(s) */
	virtual void OnPasteNodes() = 0;
	virtual bool CanPasteNodes() const = 0;

	/** Callbacks to duplicate the selected component */
	virtual bool CanDuplicateNodes() const = 0;
	virtual void OnDuplicateNodes() = 0;

	/** Removes existing selected component nodes from the SCS */
	virtual void OnDeleteNodes() = 0;
	virtual bool CanDeleteNodes() const = 0;

	/** Requests a rename on the selected Entity. */
	virtual void OnRenameNode() = 0;
	virtual bool CanRenameNode() const = 0;

	/**
	 * Compares the filter bar's text with the item's component name. Use
	 * bRecursive to refresh the state of child nodes as well. Returns true if
	 * the node is set to be filtered out
	 */
	bool RefreshFilteredState(TSharedPtr<FDMXEntityTreeNodeBase> Node, bool bRecursive);

	/** Broadcast when the selection updated. */
	FDMXOnSelectionChanged OnSelectionChangedDelegate;

	/** Broadcast when the entity list added an entity to the library */
	FSimpleDelegate OnEntitiesAddedDelegate;

	/** Broadcast when the entity list changed order of the library's entity array */
	FSimpleDelegate OnEntityOrderChangedDelegate;

	/** Broadcast when the entity list deleted an entity from the library */
	FSimpleDelegate OnEntitiesRemovedDelegate;

	/** Command list for handling actions such as copy and paste */
	TSharedPtr<FUICommandList> CommandList;

	/** Pointer back to the DMXEditor tool that owns us */
	TWeakPtr<FDMXEditor> DMXEditor;

private:
	/** Callback when the filter is changed, forces the action tree(s) to filter */
	void OnFilterTextChanged(const FText& InFilterText);

	/** Returns the set of expandable nodes that are currently collapsed in the UI */
	void GetCollapsedNodes(TSet<TSharedPtr<FDMXEntityTreeNodeBase>>& OutCollapsedNodes, TSharedPtr<FDMXEntityTreeNodeBase> InParentNode = nullptr) const;

	/** Called when the active tab in the editor changes */
	void OnActiveTabChanged(TSharedPtr<SDockTab> PreviouslyActive, TSharedPtr<SDockTab> NewlyActivated);

	/** Searches this widget's parents to see if it's a child of InDockTab */
	bool IsInTab(TSharedPtr<SDockTab> InDockTab) const;

	/**
	 * The root node.
	 * It's not added to the tree, but the main categories and all their children
	 * (entity and sub-category nodes) belong to it to make recursive searching algorithms nicer.
	 */
	TSharedPtr<FDMXEntityTreeRootNode> RootNode;

	/** The filter box that handles filtering entities */
	TSharedPtr<SSearchBox> FilterBox;

	/** Map of items that should be expanded or collapsed next tick */
	TMap<TSharedRef<FDMXEntityTreeNodeBase>, bool> PendingExpandItemMap;

	/** Gate to prevent changing the selection while selection change is being broadcast. */
	bool bUpdatingSelection = false;

	/** Enum for states of refreshing required */
	enum class EDMXRefreshTreeViewState : uint8
	{
		RegenerateNodes,
		UpdateNodes,
		NoRefreshRequested
	};

	/** True when a tree refresh was requested */
	EDMXRefreshTreeViewState RefreshTreeViewState = EDMXRefreshTreeViewState::NoRefreshRequested;

	/** Object to select on the next tick */
	TArray<TSharedPtr<FDMXEntityTreeNodeBase>> PendingSelection;

	/** Delegate handle bound to the FGlobalTabmanager::OnActiveTabChanged delegate */
	FDelegateHandle OnActiveTabChangedDelegateHandle;

	/** The actual Tree widget */
	TSharedPtr<STreeView<TSharedPtr<FDMXEntityTreeNodeBase>>> EntitiesTreeWidget;
};
