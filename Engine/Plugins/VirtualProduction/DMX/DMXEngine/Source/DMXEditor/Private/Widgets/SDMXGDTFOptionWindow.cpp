// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDMXGDTFOptionWindow.h"
#include "Factories/DMXGDTFImportUI.h"

#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"

#define LOCTEXT_NAMESPACE "SDMXGDTFOptionWindow"

namespace UE::DMX
{
	void SDMXGDTFOptionWindow::Construct(const FArguments& InArgs)
	{
		check(InArgs._ImportUI);

		ImportUI = InArgs._ImportUI;
		WidgetWindow = InArgs._WidgetWindow;

		TSharedPtr<SBox> ImportTypeDisplay;
		TSharedPtr<SHorizontalBox> HeaderButtons;
		TSharedPtr<SBox> InspectorBox;

		this->ChildSlot
			[
				SNew(SBox)
				.MaxDesiredHeight(InArgs._MaxWindowHeight)
				.MaxDesiredWidth(InArgs._MaxWindowWidth)
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2.f)
					[
						SAssignNew(ImportTypeDisplay, SBox)
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2.f)
					[
						SNew(SBorder)
						.Padding(FMargin(3))
						.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(STextBlock)
								.Font(FAppStyle::GetFontStyle("CurveEd.LabelFont"))
								.Text(LOCTEXT("Import_CurrentFileTitle", "Current Asset: "))
							]

							+ SHorizontalBox::Slot()
							.Padding(5.f, 0.f, 0.f, 0.f)
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Font(FAppStyle::GetFontStyle("CurveEd.InfoFont"))
								.Text(InArgs._FullPath)
								.ToolTipText(InArgs._FullPath)
							]
						]
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2.f)
					[
						SAssignNew(InspectorBox, SBox)
						.MaxDesiredHeight(650.0f)
						.WidthOverride(400.0f)
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Right)
					.Padding(2.f)
					[
						SNew(SUniformGridPanel)
						.SlotPadding(2.f)
						+ SUniformGridPanel::Slot(1, 0)
						[
							SNew(SButton)
							.HAlign(HAlign_Center)
							.Text(LOCTEXT("OptionWindow_ImportAll", "Import All"))
							.ToolTipText(LOCTEXT("OptionWindow_ImportAll_ToolTip", "Import all files with these same settings"))
							.OnClicked(this, &SDMXGDTFOptionWindow::OnImportAll)
						]

						+ SUniformGridPanel::Slot(2, 0)
						[
							SAssignNew(ImportButton, SButton)
							.HAlign(HAlign_Center)
							.Text(LOCTEXT("OptionWindow_Import", "Import"))
							.OnClicked(this, &SDMXGDTFOptionWindow::OnImport)
						]

						+ SUniformGridPanel::Slot(3, 0)
						[
							SNew(SButton)
							.HAlign(HAlign_Center)
							.Text(LOCTEXT("OptionWindow_Cancel", "Cancel"))
							.ToolTipText(LOCTEXT("OptionWindow_Cancel_ToolTip", "Cancels importing this file"))
							.OnClicked(this, &SDMXGDTFOptionWindow::OnCancel)
						]
					]
				]
			];

		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

		InspectorBox->SetContent(DetailsView->AsShared());

		ImportTypeDisplay->SetContent(
			SNew(SBorder)
			.Padding(FMargin(3))
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(this, &SDMXGDTFOptionWindow::GetImportTypeDisplayText)
				]

				+ SHorizontalBox::Slot()
				[
					SNew(SBox)
					.HAlign(HAlign_Right)
					[
						SAssignNew(HeaderButtons, SHorizontalBox)

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(FMargin(2.f, 0.f))
						[
							SNew(SButton)
							.Text(LOCTEXT("OptionWindow_ResetOptions", "Reset to Default"))
							.OnClicked(this, &SDMXGDTFOptionWindow::OnResetToDefaultClick)
						]
					]
				]
			]
		);

		if (UDMXGDTFImportUI* ImportUIPtr = ImportUI.Get())
		{
			DetailsView->SetObject(ImportUIPtr);
		}
	}

	FReply SDMXGDTFOptionWindow::OnImport()
	{
		bShouldImport = true;
		if (WidgetWindow.IsValid())
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	FReply SDMXGDTFOptionWindow::OnImportAll()
	{
		bShouldImportAll = true;
		return OnImport();
	}

	FReply SDMXGDTFOptionWindow::OnCancel()
	{
		bShouldImport = false;
		bShouldImportAll = false;
		if (WidgetWindow.IsValid())
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	FReply SDMXGDTFOptionWindow::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
	{
		if (InKeyEvent.GetKey() == EKeys::Escape)
		{
			return OnCancel();
		}

		return FReply::Unhandled();
	}

	FReply SDMXGDTFOptionWindow::OnResetToDefaultClick() const
	{
		if (UDMXGDTFImportUI* ImportUIPtr = ImportUI.Get())
		{
			ImportUIPtr->ResetToDefault();
			//Refresh the view to make sure the custom UI are updating correctly
			DetailsView->SetObject(ImportUIPtr, true);
		}
		return FReply::Handled();
	}

	FText SDMXGDTFOptionWindow::GetImportTypeDisplayText() const
	{
		return FText::FromString(TEXT("Import GDTF"));
	}

}

#undef LOCTEXT_NAMESPACE
