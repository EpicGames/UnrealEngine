// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"

class SButton;
class UDMXGDTFImportUI;
class IDetailsView;

namespace UE::DMX
{

	class SDMXGDTFOptionWindow : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SDMXGDTFOptionWindow)
			: _ImportUI(nullptr)
			, _WidgetWindow()
			, _FullPath()
			, _MaxWindowHeight(0.0f)
			, _MaxWindowWidth(0.0f)
			{}

			SLATE_ARGUMENT(UDMXGDTFImportUI*, ImportUI)
			SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
			SLATE_ARGUMENT(FText, FullPath)
			SLATE_ARGUMENT(float, MaxWindowHeight)
			SLATE_ARGUMENT(float, MaxWindowWidth)

		SLATE_END_ARGS()

	public:
		void Construct(const FArguments& InArgs);

		bool ShouldImport() const { return bShouldImport;  }

		bool ShouldImportAll() const { return bShouldImportAll; }

	private:
		//~ Begin SWidget interface
		virtual bool SupportsKeyboardFocus() const override { return true; }
		virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
		//~ End SWidget interface

		FReply OnImport();
		FReply OnImportAll();
		FReply OnCancel();
		FReply OnResetToDefaultClick() const;
		FText GetImportTypeDisplayText() const;

		TWeakObjectPtr<UDMXGDTFImportUI> ImportUI;
		TSharedPtr<IDetailsView> DetailsView;
		TWeakPtr< SWindow > WidgetWindow;
		TSharedPtr< SButton > ImportButton;
		bool bShouldImport = false;
		bool bShouldImportAll = false;
	};
}
