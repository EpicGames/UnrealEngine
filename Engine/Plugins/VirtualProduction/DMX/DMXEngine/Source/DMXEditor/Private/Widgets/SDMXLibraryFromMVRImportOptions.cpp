// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDMXLibraryFromMVRImportOptions.h"

#include "Customizations/DMXLibraryFromMVRImportOptionsDetails.h"
#include "Factories/DMXLibraryFromMVRImportOptions.h"

#include "IDetailsView.h"
#include "PropertyEditorModule.h"
#include "SPrimaryButton.h"
#include "Framework/Application/SlateApplication.h"
#include "Modules/ModuleManager.h"


#define LOCTEXT_NAMESPACE "SDMXLibraryFromMVRFactoryOptions"

void SDMXLibraryFromMVRImportOptions::Construct(const FArguments& InArgs, const TSharedRef<SWindow>& ParentWindow, UDMXLibraryFromMVRImportOptions* DMXLibraryFromMVRImportOptions)
{
	if (!ensureAlwaysMsgf(DMXLibraryFromMVRImportOptions, TEXT("Invalid Options object for DMX Library from MVR Import Options widget.")))
	{
		return;
	}

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	const TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	DetailsView->RegisterInstancedCustomPropertyLayout(UDMXLibraryFromMVRImportOptions::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(FDMXLibraryFromMVRImportOptionsDetails::MakeInstance));

	DetailsView->SetObject(DMXLibraryFromMVRImportOptions);

	// Default to Cancelled, only clicking the 'Import' button should proceed.
	DMXLibraryFromMVRImportOptions->bCancelled = true;

	ChildSlot
		[
			SNew(SVerticalBox)
	
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.FillHeight(1.f)
			[
				DetailsView
			]
				
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			.AutoHeight()
			[
				SNew(SHorizontalBox)
					
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(4.f, 2.f)
				[
					SAssignNew(ImportButton, SPrimaryButton)
					.Text(LOCTEXT("OptionWindow_Import", "Import"))
					.OnClicked_Lambda([ParentWindow, DMXLibraryFromMVRImportOptions]()
						{
							DMXLibraryFromMVRImportOptions->bCancelled = false;
							ParentWindow->RequestDestroyWindow();
							return FReply::Handled();
						})
				]

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Bottom)
				.AutoWidth()
				.Padding(4.f, 2.f)
				[
					SNew(SButton)
					.Text(LOCTEXT("OptionWindow_Cancel", "Cancel"))
					.ToolTipText(LOCTEXT("OptionWindow_Cancel_ToolTip", "Cancels importing this file"))
					.OnClicked_Lambda([ParentWindow]()
						{
							ParentWindow->RequestDestroyWindow();
							return FReply::Handled();
						})
				]
			]
		];

	RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SDMXLibraryFromMVRImportOptions::SetFocusPostConstruct));
}

EActiveTimerReturnType SDMXLibraryFromMVRImportOptions::SetFocusPostConstruct(double InCurrentTime, float InDeltaTime)
{
	if (ImportButton.IsValid())
	{
		FSlateApplication::Get().SetKeyboardFocus(ImportButton, EFocusCause::SetDirectly);
	}

	return EActiveTimerReturnType::Stop;
}

FReply SDMXLibraryFromMVRImportOptions::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		if (UDMXLibraryFromMVRImportOptions* ImportOptions = WeakImportOptions.Get())
		{
			ImportOptions->bCancelled = true;
		}
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
