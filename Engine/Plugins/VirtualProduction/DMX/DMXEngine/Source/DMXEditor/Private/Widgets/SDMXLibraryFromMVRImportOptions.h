// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"

class UDMXLibraryFromMVRImportOptions;

class SPrimaryButton;


/** Widget that displays the DMX Library from MVR Import Options */
class SDMXLibraryFromMVRImportOptions
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMXLibraryFromMVRImportOptions)
	{}
		SLATE_ARGUMENT(FVector2D, MaxWidgetSize)

    SLATE_END_ARGS()

	/** Constructs this widget */
    void Construct(const FArguments& InArgs, const TSharedRef<SWindow>& ParentWindow, UDMXLibraryFromMVRImportOptions* DMXLibraryFromMVRImportOptions);

protected:
	//~ Begin SWidget interface
    virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget interface

private:
	/** Sets focus after the widget was constructed (next tick) */
	EActiveTimerReturnType SetFocusPostConstruct(double InCurrentTime, float InDeltaTime);

	/** The Import Button widget */
	TSharedPtr<SPrimaryButton> ImportButton;

	/** Import options being displayed */
	TWeakObjectPtr<UDMXLibraryFromMVRImportOptions> WeakImportOptions;
};
