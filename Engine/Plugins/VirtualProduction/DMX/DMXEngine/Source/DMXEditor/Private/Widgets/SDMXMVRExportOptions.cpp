// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXMVRExportOptions.h"

#include "Exporters/DMXMVRExportOptions.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SPrimaryButton.h"

#define LOCTEXT_NAMESPACE "SDMXMVRExportOptions"

namespace UE::DMX
{
	SDMXMVRExportOptions::~SDMXMVRExportOptions()
	{
		UDMXMVRExportOptions* ExportOptions = GetMutableDefault<UDMXMVRExportOptions>();
		ExportOptions->SaveConfig();
	}

	void SDMXMVRExportOptions::Construct(const FArguments& InArgs, const TSharedRef<SWindow>& ParentWindow)
	{		
		// Build an options details view
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		const TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

		DetailsView->SetObject(GetMutableDefault<UDMXMVRExportOptions>());

		ChildSlot
		[
			SNew(SVerticalBox)
	
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.FillHeight(1.f)
			[
				DetailsView
			]
				
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			.AutoHeight()
			[
				SNew(SHorizontalBox)
					
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(4.f, 2.f)
				[
					SNew(SPrimaryButton)
					.Text(LOCTEXT("OptionWindow_Export", "Export"))
					.OnClicked_Lambda([ParentWindow]()
						{
							UDMXMVRExportOptions* ExportOptions = GetMutableDefault<UDMXMVRExportOptions>();
							ExportOptions->bCanceled = false;

							ParentWindow->RequestDestroyWindow();
							return FReply::Handled();
						})
				]

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Bottom)
				.AutoWidth()
				.Padding(4.f, 2.f)
				[
					SNew(SButton)
					.Text(LOCTEXT("OptionWindow_Cancel", "Cancel"))
					.OnClicked_Lambda([ParentWindow]()
						{							
							UDMXMVRExportOptions* ExportOptions = GetMutableDefault<UDMXMVRExportOptions>();
							ExportOptions->bCanceled = true;

							ParentWindow->RequestDestroyWindow();
							return FReply::Handled();
						})
				]
			]
		];
	}
}

#undef LOCTEXT_NAMESPACE
