// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

namespace UE::DMX
{
	/** The UI presented when exporting MVR files */
	class SDMXMVRExportOptions final
		: public SCompoundWidget 
	{
	public:
		SLATE_BEGIN_ARGS(SDMXMVRExportOptions)
		{}

		SLATE_END_ARGS()

		~SDMXMVRExportOptions();

		/** Constructs the widget */
		void Construct(const FArguments& InArgs, const TSharedRef<SWindow>& ParentWindow);
	};
}
