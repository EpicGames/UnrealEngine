// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDMXNamedType.h"

#include "CoreMinimal.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Notifications/SPopUpErrorText.h"

class FDMXEditor;

class SInlineEditableTextBlock;
class SPopupErrorText;



/** editable text box for unique strings */
template <typename NamedType>
class SDMXNamedTypeRow
	: public STableRow<TSharedPtr<NamedType>>
{
public:
	SLATE_BEGIN_ARGS(SDMXNamedTypeRow)
	{}

		SLATE_ARGUMENT(FText, NameEmptyError)

		SLATE_ARGUMENT(FText, NameDuplicateError)

	SLATE_END_ARGS()

	/** Constructs the widget */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, TSharedPtr<NamedType> InItem)
	{
		STableRow<TSharedPtr<NamedType>>::Construct(typename STableRow<TSharedPtr<NamedType>>::FArguments(), OwnerTable);
		
		check(InItem.IsValid());

		Item = StaticCastSharedPtr<IDMXNamedType>(InItem);

		NameEmptyError = InArgs._NameEmptyError;
		NameDuplicateError = InArgs._NameDuplicateError;

		FString Name;
		Item->GetName(Name);

		this->ChildSlot
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				.FillWidth(1)
				[
					SAssignNew(InlineEditableTextBlock, SInlineEditableTextBlock)					
						.Text(FText::FromString(Name))
						.Font(FCoreStyle::GetDefaultFontStyle("Regular", 10))
						.IsReadOnly(false)
						.OnVerifyTextChanged(this, &SDMXNamedTypeRow::OnVerifyTextChanged)
						.OnTextCommitted(this, &SDMXNamedTypeRow::OnTextCommitted)
				]
				+ SHorizontalBox::Slot()					
				.AutoWidth()
				.Padding(3, 0)
				[
					SAssignNew(ErrorTextWidget, SPopupErrorText)
				]
			];

		ErrorTextWidget->SetError(FText::GetEmpty());
	}

	/** Returns the item */
	TSharedPtr<NamedType> GetItem() const { return StaticCastSharedPtr<NamedType>(Item); }

	/** Enters editing  */
	void EnterEditingMode()
	{
		FSlateApplication::Get().SetKeyboardFocus(InlineEditableTextBlock);
		InlineEditableTextBlock->EnterEditingMode();
	}

protected:
	//~ Begin SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		if (!InlineEditableTextBlock->IsInEditMode())
		{
			FString Name;
			Item->GetName(Name);
			InlineEditableTextBlock->SetText(FText::FromString(Name));
		}
	}

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override
	{
		InlineEditableTextBlock->EnterEditingMode();
		return FReply::Handled();
	}
	//~End of SWidget interface

	/** Called when text changed */
	bool OnVerifyTextChanged(const FText& InNewText, FText& OutErrorMessage)
	{
		if (FText::TrimPrecedingAndTrailing(InNewText).IsEmpty())
		{
			ErrorTextWidget->SetError(NameEmptyError);
			return true;
		}

		FString OldName;
		Item->GetName(OldName);

		const FString& NewName = InNewText.ToString();

		if (OldName == NewName)
		{
			ErrorTextWidget->SetError(FText::GetEmpty());
			return true;
		}

		if (Item->IsNameUnique(NewName))
		{
			ErrorTextWidget->SetError(FText::GetEmpty());
		}
		else
		{
			ErrorTextWidget->SetError(NameDuplicateError);
		}

		return true;
	}

	/** Called when text was comitted */
	void OnTextCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
	{
		FString OldName;
		Item->GetName(OldName);

		const FString& NewName = InNewText.ToString();

		if (OldName == NewName)
		{
			return;
		}

		FString UniqueNewName;
		Item->SetName(NewName, UniqueNewName);

		InlineEditableTextBlock->SetText(FText::FromString(UniqueNewName));

		ErrorTextWidget->SetError(FText::GetEmpty());

		// Release keyboard focus in case it was programmatically
		// set. Otherwise the row goes into edit mode when clicked anew
		FSlateApplication::Get().ClearKeyboardFocus();
	}

protected:
	/**  item shown in the row */
	TSharedPtr<IDMXNamedType> Item;

	/** Error displayed when the text box is empty */
	FText NameEmptyError;

	/** Error displayed when the the  name entered already exists */
	FText NameDuplicateError;
	
	/** Text box editing the Name */
	TSharedPtr<SInlineEditableTextBlock> InlineEditableTextBlock;

	/** Error reporting widget, required as SInlineEditableTextBlock error can't be cleared on commit */
	TSharedPtr<SPopupErrorText> ErrorTextWidget;
};
