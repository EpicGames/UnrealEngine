// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDMXSignalFormatSelector.h"

#include "DMXProtocolTypes.h"

#include "Styling/AppStyle.h"
#include "Widgets/Input/SComboBox.h"


#define LOCTEXT_NAMESPACE "SDMXSignalFormatSelector"

const TArray<TSharedPtr<FString>> SDMXSignalFormatSelector::AvailableSignalFormats =
{
	MakeShared<FString>(TEXT("8bit")),
	MakeShared<FString>(TEXT("16bit")),
	MakeShared<FString>(TEXT("24bit")),
	MakeShared<FString>(FText(LOCTEXT("32BitValueNotFullySupportedComboBoxEntry", "32bit (not fully supported)")).ToString())
};

void SDMXSignalFormatSelector::Construct(const FArguments& InArgs)
{
	HasMultipleValues = InArgs._HasMultipleValues;
	OnSignalFormatSelectedDelegate = InArgs._OnSignalFormatSelected;

	SignalFormatsSource = AvailableSignalFormats;

	// Only show the 32bit option if it is the inital selection
	if (InArgs._InitialSelection != EDMXFixtureSignalFormat::E32Bit)
	{
		SignalFormatsSource.RemoveAt(3);
	}

	const TSharedRef<FString>& InitialSelection = [InArgs, this]()
	{
		switch (InArgs._InitialSelection)
		{
			case EDMXFixtureSignalFormat::E8Bit:
				return SignalFormatsSource[0].ToSharedRef();
			case EDMXFixtureSignalFormat::E16Bit:
				return SignalFormatsSource[1].ToSharedRef();
			case EDMXFixtureSignalFormat::E24Bit:
				return SignalFormatsSource[2].ToSharedRef();
			case EDMXFixtureSignalFormat::E32Bit:
				return SignalFormatsSource[3].ToSharedRef();
			default:
				checkNoEntry(); // Unhahndled enum value
				return SignalFormatsSource[0].ToSharedRef();
		}
	}();

	ChildSlot
	[
		SAssignNew(ComboBox, SComboBox<TSharedPtr<FString>>)
		.OptionsSource(&SignalFormatsSource)
		.InitiallySelectedItem(InitialSelection)
		.OnSelectionChanged(this, &SDMXSignalFormatSelector::OnSignalFormatSelected)
		.OnGenerateWidget(this, &SDMXSignalFormatSelector::GenerateComboBoxWidget)
		[
			SAssignNew(SelectionTextBlock, STextBlock)
			.Text_Lambda([this, InArgs]()
				{
					if (HasMultipleValues.IsBound() && HasMultipleValues.Get())
					{
						return LOCTEXT("HasMultipleValues", "Multiple Values");
					}

					switch (GetSelectedSignalFormat())
					{
					case EDMXFixtureSignalFormat::E8Bit:
						return FText::FromString(*SignalFormatsSource[0]);
					case EDMXFixtureSignalFormat::E16Bit:
						return FText::FromString(*SignalFormatsSource[1]);
					case EDMXFixtureSignalFormat::E24Bit:
						return FText::FromString(*SignalFormatsSource[2]);
					case EDMXFixtureSignalFormat::E32Bit:
						return FText::FromString(*SignalFormatsSource[3]);
					default:
						return LOCTEXT("InvalidSelection", "Invalid Selection");
					}

				})
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		]
	];

}

EDMXFixtureSignalFormat SDMXSignalFormatSelector::GetSelectedSignalFormat() const
{
	const TSharedPtr<FString> SelectedItem = ComboBox->GetSelectedItem();

	if (SelectedItem == AvailableSignalFormats[0])
	{
		return EDMXFixtureSignalFormat::E8Bit;
	}
	else if (SelectedItem == AvailableSignalFormats[1])
	{
		return EDMXFixtureSignalFormat::E16Bit;
	}
	else if (SelectedItem == AvailableSignalFormats[2])
	{
		return EDMXFixtureSignalFormat::E24Bit;
	}
	else if (SelectedItem == AvailableSignalFormats[3])
	{
		return EDMXFixtureSignalFormat::E32Bit;
	}

	checkNoEntry(); // Unhandled option
	return EDMXFixtureSignalFormat::E8Bit;
}

TSharedRef<SWidget> SDMXSignalFormatSelector::GenerateComboBoxWidget(TSharedPtr<FString> Item)
{
	return
		SNew(STextBlock)
		.Text_Lambda([this, Item]()
			{
				return FText::FromString(*Item);
			})
		.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));
}

void SDMXSignalFormatSelector::OnSignalFormatSelected(TSharedPtr<FString> SelectedItem, ESelectInfo::Type SelectInfo)
{
	EDMXFixtureSignalFormat SelectedSignalFormat = GetSelectedSignalFormat();

	OnSignalFormatSelectedDelegate.ExecuteIfBound(SelectedSignalFormat);
}

#undef LOCTEXT_NAMESPACE
