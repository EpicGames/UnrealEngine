// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"

enum class EDMXFixtureSignalFormat : uint8;

template <typename OptionType> class SComboBox;
class STextBlock;

/** Widget to select a Fixture Function Signal Format */
class SDMXSignalFormatSelector
	: public SCompoundWidget
{
	DECLARE_DELEGATE_OneParam(FDMXOnSignalFormatSelected, EDMXFixtureSignalFormat);

public:
	SLATE_BEGIN_ARGS(SDMXSignalFormatSelector)
		: _InitialSelection()
	{}

		SLATE_ATTRIBUTE(bool, HasMultipleValues)

		/** The initially selected item in the selector */
		SLATE_ARGUMENT(EDMXFixtureSignalFormat, InitialSelection)

		/** Delegate executed when a Signal Format was selected */
		SLATE_EVENT(FDMXOnSignalFormatSelected, OnSignalFormatSelected)

	SLATE_END_ARGS()

	/** Constructs this widget */
	void Construct(const FArguments& InArgs);

	/** Returns the selected Signal Format */
	EDMXFixtureSignalFormat GetSelectedSignalFormat() const;

private:
	/** Generates the widget displayed on the combo box */
	TSharedRef<SWidget> GenerateComboBoxWidget(TSharedPtr<FString> Item);

	/** Called when a Signal Format was selected */
	void OnSignalFormatSelected(TSharedPtr<FString> SelectedItem, ESelectInfo::Type SelectInfo);

	/** The available Signal Format (incl. 32bit) */
	static const TArray<TSharedPtr<FString>> AvailableSignalFormats;

	/** The Signal Format available to this specific widget (with or without 32bit, depending on the slate arg) */
	TArray<TSharedPtr<FString>> SignalFormatsSource;

	/** The combo box that draws the options */
	TSharedPtr<SComboBox<TSharedPtr<FString>>> ComboBox;

	/** The Text Block that draws the selection */
	TSharedPtr<STextBlock> SelectionTextBlock;

	// Slate Arguments
	TAttribute<bool> HasMultipleValues;
	FDMXOnSignalFormatSelected OnSignalFormatSelectedDelegate;
};