// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SNameListPicker.h"

#include "DMXEditorLog.h"
#include "DMXNameListItem.h"
#include "DMXProtocolSettings.h"

#include "Styling/AppStyle.h"
#include "SListViewSelectorDropdownMenu.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "SDMXProtocolNamePicker"

const FText SNameListPicker::NoneLabel = LOCTEXT("NoneLabel", "<Select a Value>");

void SNameListPicker::Construct(const FArguments& InArgs)
{
	ValueAttribute = InArgs._Value;
	HasMultipleValuesAttribute = InArgs._HasMultipleValues;
	bDisplayWarningIcon = InArgs._bDisplayWarningIcon;
	OptionsSourceAttr = InArgs._OptionsSource;
	MaxVisibleItems = InArgs._MaxVisibleItems;
	OnValueChangedDelegate = InArgs._OnValueChanged;

	UpdateOptionsSource();

	EdititableTextBox = SNew(SEditableTextBox)
		.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		.Text(this, &SNameListPicker::GetCurrentNameLabel)
		.OnTextCommitted(this, &SNameListPicker::OnTextCommitted);

	// List of selectable names for the dropdown menu
	SAssignNew(OptionsListView, SListView< TSharedPtr<FName> >)
		.ListItemsSource(&FilteredOptions)
		.OnMouseButtonClick(this, &SNameListPicker::OnClickItem)
		.OnSelectionChanged(this, &SNameListPicker::OnSelectionChanged)
		.OnGenerateRow(this, &SNameListPicker::GenerateNameItemWidget)
		.SelectionMode(ESelectionMode::Single);
	UpdateFilteredOptions(TEXT(""));

	// Search box. Visible only when the list has 16+ names
	SAssignNew(SearchBox, SSearchBox)
		.HintText(LOCTEXT("ValueSearchHint", "Search Values"))
		.Visibility(this, &SNameListPicker::GetSearchBoxVisibility)
		.OnTextChanged(this, &SNameListPicker::OnSearchBoxTextChanged)
		.OnTextCommitted(this, &SNameListPicker::OnSearchBoxTextCommitted);

	// Dropdown menu
	SAssignNew(NamesListDropdown, SListViewSelectorDropdownMenu< TSharedPtr<FName> >, SearchBox, OptionsListView)
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		.Padding(2)
		[
			SNew(SBox)
			.WidthOverride(250)
			[				
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.Padding(1.f)
				.AutoHeight()
				[
					SearchBox.ToSharedRef()
				]

				+SVerticalBox::Slot()
				.MaxHeight(200.0f)
				[
					OptionsListView.ToSharedRef()
				]
			]
		]
	];

	// Combo button that summons the dropdown menu
	SAssignNew(PickerComboButton, SComboButton)
		.ButtonContent()
		[
			EdititableTextBox.ToSharedRef()
		]
		.MenuContent()
		[
			NamesListDropdown.ToSharedRef()
		]
		.IsFocusable(true)
		.ContentPadding(2.0f)
		.OnComboBoxOpened(this, &SNameListPicker::OnMenuOpened);
	PickerComboButton->SetMenuContentWidgetToFocus(SearchBox);
	
	ChildSlot
	[
		PickerComboButton.ToSharedRef()
	];
}

void SNameListPicker::UpdateOptionsSource()
{
	// Number of options with the <None> option
	const int32 NumOptions = OptionsSourceAttr.Get().Num() + 1;

	OptionsSource.Reset();
	OptionsSource.Reserve(NumOptions);

	// <None> is the first option
	OptionsSource.Add(MakeShared<FName>(NAME_None));

	for (const FName& Name : OptionsSourceAttr.Get())
	{
		OptionsSource.Add(MakeShared<FName>(Name));
	}
}

TSharedRef<ITableRow> SNameListPicker::GenerateNameItemWidget(TSharedPtr<FName> InItem, const TSharedRef<STableViewBase>& OwnerTable) const
{
	TSharedPtr<STextBlock> RowTextBlock = nullptr; 
	TSharedRef< STableRow< TSharedPtr<FName> > > TableRow =
		SNew(STableRow< TSharedPtr<FName> >, OwnerTable)
		.ShowSelection(true)
		.Content()
		[
			SAssignNew(RowTextBlock, STextBlock)
		];

	if (!InItem.IsValid())
	{
		UE_LOG_DMXEDITOR(Warning, TEXT("InItem for GenerateProtocolItemWidget was null!"));
		RowTextBlock->SetText(LOCTEXT("NullComboBoxItemLabel", "Null Error"));
		return TableRow;
	}

	if (InItem->IsEqual(NAME_None))
	{
		RowTextBlock->SetText(NoneLabel);
		return TableRow;
	}

	RowTextBlock->SetText(FText::FromName(*InItem));
	return TableRow;
}

void SNameListPicker::OnSelectionChanged(const TSharedPtr<FName> Item, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo == ESelectInfo::OnKeyPress || SelectInfo == ESelectInfo::OnNavigation)
	{
		return;
	}

	if (Item.IsValid())
	{
		EdititableTextBox->SetText(FText::FromString(Item->ToString()));
	}

	OnUserSelectedItem(Item);
}

void SNameListPicker::OnClickItem(const TSharedPtr<FName> Item)
{
	OnUserSelectedItem(Item);
}

void SNameListPicker::OnUserSelectedItem(const TSharedPtr<FName> Item)
{
	if (!Item.IsValid())
	{
		return;
	}

	if (OnValueChangedDelegate.IsBound())
	{
		OnValueChangedDelegate.Execute(*Item);
	}
	else if (!ValueAttribute.IsBound())
	{
		ValueAttribute = *Item;
	}

	if (PickerComboButton.IsValid())
	{
		PickerComboButton->SetIsOpen(false);
	}
}

TSharedPtr<FName> SNameListPicker::GetSelectedItemFromCurrentValue() const
{
	TSharedPtr<FName> InitiallySelected = nullptr;

	const bool bHasMultipleValues = HasMultipleValuesAttribute.Get();
	if (bHasMultipleValues)
	{
		return InitiallySelected;
	}

	const FName CurrentValue = ValueAttribute.Get();

	for (TSharedPtr<FName> NameItem : OptionsSource)
	{
		if (NameItem != nullptr)
		{
			if (CurrentValue.IsEqual(*NameItem))
			{
				InitiallySelected = NameItem;
				break;
			}
		}
	}

	return InitiallySelected;
}

void SNameListPicker::OnMenuOpened()
{
	if (GetSearchBoxVisibility() != EVisibility::Collapsed)
	{
		SearchBox->SetText(FText::GetEmpty());
		UpdateFilteredOptions(TEXT(""));
	}
	else
	{
		FSlateApplication::Get().SetKeyboardFocus(OptionsListView, EFocusCause::SetDirectly);
	}

	if (OptionsListView.IsValid())
	{
		const TSharedPtr<FName> SelectedName = GetSelectedItemFromCurrentValue();
		OptionsListView->SetSelection(SelectedName, ESelectInfo::OnKeyPress);
		OptionsListView->RequestScrollIntoView(SelectedName);
	}
}

EVisibility SNameListPicker::GetSearchBoxVisibility() const
{
	return OptionsSource.Num() > MaxVisibleItems ? EVisibility::Visible : EVisibility::Collapsed;
}

void SNameListPicker::OnSearchBoxTextChanged(const FText& InSearchText)
{
	UpdateFilteredOptions(InSearchText.ToString());
}

void SNameListPicker::OnSearchBoxTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter && FilteredOptions.Num() > 0)
	{
		OptionsListView->SetSelection(FilteredOptions[0], ESelectInfo::Direct);
	}
}

void SNameListPicker::UpdateFilteredOptions(const FString& Filter)
{
	// Don't bother filtering if we have nothing to filter
	if (OptionsSource.Num() == 0 || Filter.IsEmpty())
	{
		FilteredOptions = OptionsSource;
	}
	else
	{
		FilteredOptions.Reset();

		const FString FilterTrimmed = Filter.TrimStartAndEnd();
		TArray<FString> FilterTerms;
		FilterTrimmed.ParseIntoArray(FilterTerms, TEXT(" "), /*bInCullEmpty=*/true);

		for (TSharedPtr<FName>& NameOption : OptionsSource)
		{
			if (!NameOption.IsValid() || NameOption->IsEqual(NAME_None))
			{
				continue;
			}

			for (const FString& FilterTerm : FilterTerms)
			{
				if (NameOption->ToString().Contains(FilterTerm))
				{
					FilteredOptions.Add(NameOption);
					break;
				}
			}
		}

		// select the first entry that passed the filter
		if (FilteredOptions.Num() > 0)
		{
			OptionsListView->SetSelection(FilteredOptions[0], ESelectInfo::OnKeyPress);
		}
	}

	// Ask the list to update its contents on next tick
	OptionsListView->RequestListRefresh();
}

FText SNameListPicker::GetCurrentNameLabel() const
{
	const bool bHasMultipleValues = HasMultipleValuesAttribute.Get();
	if (bHasMultipleValues)
	{
		return LOCTEXT("MultipleValuesText", "<multiple values>");
	}

	const FName CurrentName = ValueAttribute.IsSet() ? ValueAttribute.Get() : NAME_None;
	if (CurrentName.IsEqual(NAME_None))
	{
		return NoneLabel;
	}

	return FText::FromName(CurrentName);
}

void SNameListPicker::OnTextCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	if (InNewText.IsEmpty())
	{
		return;
	}

	const FName Value = *InNewText.ToString().TrimStartAndEnd();
	EdititableTextBox->SetText(FText::FromName(Value));

	if (OnValueChangedDelegate.IsBound())
	{
		OnValueChangedDelegate.Execute(Value);
	}
	else if (!ValueAttribute.IsBound())
	{
		ValueAttribute = Value;
	}
}

#undef LOCTEXT_NAMESPACE
