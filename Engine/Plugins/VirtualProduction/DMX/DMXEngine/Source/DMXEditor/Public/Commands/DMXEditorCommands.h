// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

class DMXEDITOR_API FDMXEditorCommands
	: public TCommands<FDMXEditorCommands>
{
public:
	FDMXEditorCommands();

	//~ Begin TCommands implementation
	virtual void RegisterCommands() override;
	//~ End TCommands implementation

	// Documentation related
	TSharedPtr<FUICommandInfo> GoToDocumentation;

	// DMX Library related
	TSharedPtr<FUICommandInfo> ImportDMXLibrary;
	TSharedPtr<FUICommandInfo> ExportDMXLibrary;

	TSharedPtr<FUICommandInfo> AddNewEntityFixtureType;
	TSharedPtr<FUICommandInfo> AddNewEntityFixturePatch;
	
	TSharedPtr<FUICommandInfo> AddNewFixtureTypeMode;
	TSharedPtr<FUICommandInfo> AddNewModeFunction;

	// Level Editor Tool Bar related
	TSharedPtr<FUICommandInfo> OpenChannelsMonitor;
	TSharedPtr<FUICommandInfo> OpenActivityMonitor;
	TSharedPtr<FUICommandInfo> OpenConflictMonitor;
	TSharedPtr<FUICommandInfo> OpenPatchTool;
	TSharedPtr<FUICommandInfo> ToggleReceiveDMX;
	TSharedPtr<FUICommandInfo> ToggleSendDMX;

	// Auto assign
	TSharedPtr<FUICommandInfo> Align;
	TSharedPtr<FUICommandInfo> Stack;
	TSharedPtr<FUICommandInfo> AutoAssignSelectedUniverse;
};
