// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Analytics/DMXEditorToolAnalyticsProvider.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"
#include "Misc/NotifyHook.h"

class FDMXEditorToolbar;
class FDMXFixtureTypeSharedData;
class FDMXFixturePatchSharedData;
class SDMXEntityEditor;
class SDMXInputConsole;
class SDMXFixturePatchEditor;
class SDMXFixtureTypeEditor;
class SDMXLibraryEditorTab;
class SDockTab;
class UDMXEntity;
class UDMXLibrary;
class UFactory;


// Used to enable Entity creator code to inject a base name before the entity creation
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnGetBaseNameForNewEntity, TSubclassOf<UDMXEntity>, FString&);
// Used to enable Entity creator code to set values in a newly created entity
DECLARE_MULTICAST_DELEGATE_OneParam(FOnSetupNewEntity, UDMXEntity*);

/** 
 * *** The DMX Library Editor, displayed when opening a DMX Library Asset ***
 */
class DMXEDITOR_API FDMXEditor
	: public FWorkflowCentricApplication	// Allow Add ApplicationModes
	, public FNotifyHook
{	
public:
	FDMXEditor();

	//~ Begin IToolkit implementation
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	//~ End IToolkit Implementation

	/** Adds a new Entity to this DMX Library */
	void OnAddNewEntity(TSubclassOf<UDMXEntity> InEntityClass);

	/** Activate the editor tab suited to edit Entities of type InEntityClass */
	bool InvokeEditorTabFromEntityType(TSubclassOf<UDMXEntity> InEntityClass);

	FReply OnAddNewEntity_OnClick(TSubclassOf<UDMXEntity> InEntityClass) { OnAddNewEntity(InEntityClass); return FReply::Handled(); }

	UE_DEPRECATED(5.0, "Deprecated, this always yields true.")
	bool CanAddNewEntity(TSubclassOf<UDMXEntity> InEntityClass) const { return true; }

	bool NewEntity_IsVisibleForType(TSubclassOf<UDMXEntity> InEntityClass) const;

	/** Utility function to handle all steps required to rename a newly added Entity */
	void RenameNewlyAddedEntity(UDMXEntity* InEntity, TSubclassOf<UDMXEntity> InEntityClass);

	/** Gets the content widget for the tab that edits Entities from InEntityClass */
	TSharedPtr<SDMXEntityEditor> GetEditorWidgetForEntityType(TSubclassOf<UDMXEntity> InEntityClass) const;

	/** Switch to the correct tab to select an Entity for editing */
	void SelectEntityInItsTypeTab(UDMXEntity* InEntity, ESelectInfo::Type InSelectionType = ESelectInfo::Type::Direct);
	/** Switch to the correct tab for the first Entity's type and select the Entities */
	void SelectEntitiesInTypeTab(const TArray<UDMXEntity*>& InEntities, ESelectInfo::Type InSelectionType = ESelectInfo::Type::Direct);
	/** Get the selected entities from the tab that stores the passed in type */
	TArray<UDMXEntity*> GetSelectedEntitiesFromTypeTab(TSubclassOf<UDMXEntity> InEntityClass) const;

	//~ Getters for the various DMX widgets
	TSharedRef<SDMXLibraryEditorTab> GetDMXLibraryEditorTab() const { return DMXLibraryEditorTab.ToSharedRef(); }
	TSharedRef<SDMXFixturePatchEditor> GetFixturePatchEditor() const { return FixturePatchEditor.ToSharedRef(); }
	TSharedRef<SDMXFixtureTypeEditor> GetFixtureTypeEditor() const { return FixtureTypeEditor.ToSharedRef(); }

	UE_DEPRECATED(5.0, "Deprecated since naming is now handled consistently when creating new entities via UDMXEntityFixtureType::CreateFixtureType or UDMXEntityFixturePatch::CreateFixturePatch.")
	FOnGetBaseNameForNewEntity& GetOnGetBaseNameForNewEntity() { return OnGetBaseNameForNewEntity; }
	
	UE_DEPRECATED(5.0, "Deprecated in favor of UDMXLibrary::OnEntitiesAddedOrRemoved")
	FOnSetupNewEntity& GetOnSetupNewEntity() { return OnSetupNewEntity; }

	/** Edits the specified DMX library */
	virtual void InitEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UDMXLibrary* DMXLibrary);

	/** Should be called when initializing */
	void CommonInitialization(UDMXLibrary* DMXLibrary);

	/** Get the DMX library being edited */
	UDMXLibrary* GetDMXLibrary() const;

	TSharedPtr<FDMXEditorToolbar> GetToolbarBuilder() { return Toolbar; }

	void RegisterToolbarTab(const TSharedRef<class FTabManager>& TabManager);


protected:
	/** Creates the widgets that go into the tabs (note: does not create the tabs themselves) **/
	virtual void CreateDefaultTabContents(UDMXLibrary* DMXLibrary);

	/** Create Default Commands **/
	virtual void CreateDefaultCommands();

	/** Called during initialization of the DMX editor to register commands and extenders. */
	virtual void InitalizeExtenders();

	/** Called during initialization of the DMX editor to register any application modes. */
	virtual void RegisterApplicationModes(UDMXLibrary* DMXLibrary, bool bShouldOpenInDefaultsMode, bool bNewlyCreated = false);

private:
	/** Imports the DMX Library from an MVR File */
	void ImportDMXLibrary() const;

	/** Exports the DMX Library as MVR File */
	void ExportDMXLibrary() const;

	//~ Generate Editor widgets for tabs
	TSharedRef<SDMXLibraryEditorTab> CreateDMXLibraryEditorTab();
	TSharedRef<SDMXFixtureTypeEditor> CreateFixtureTypeEditor();
	TSharedRef<SDMXFixturePatchEditor> CreateFixturePatchEditor();

	/** The toolbar builder class */
	TSharedPtr<FDMXEditorToolbar> Toolbar;

	/** The name given to all instances of this type of editor */
	static const FName ToolkitFName;

	/** UI for the "DMX Library Editor" tab */
	TSharedPtr<SDMXLibraryEditorTab> DMXLibraryEditorTab;

	/** UI for the "DMX Fixture Types" tab */
	TSharedPtr<SDMXFixtureTypeEditor> FixtureTypeEditor;

	/** UI for the "DMX Fixture Patch" tab */
	TSharedPtr<SDMXFixturePatchEditor> FixturePatchEditor;

	//~ Event dispatchers
	FOnGetBaseNameForNewEntity OnGetBaseNameForNewEntity;
	FOnSetupNewEntity OnSetupNewEntity;

public:
	/** Gets the fixture type shared data instance */
	TSharedPtr<FDMXFixtureTypeSharedData> GetFixtureTypeSharedData() const;

	/** Gets the Fixture Patch shared Data Instance */
	const TSharedPtr<FDMXFixturePatchSharedData>& GetFixturePatchSharedData() const;

private:
	/** Fixture type shared data instance */
	TSharedPtr<FDMXFixtureTypeSharedData> FixtureTypeSharedData;

	/** Shared Data for Fixture Patch editors */
	TSharedPtr<FDMXFixturePatchSharedData> FixturePatchSharedData;

	/** The analytics provider for this tool */
	UE::DMX::FDMXEditorToolAnalyticsProvider AnalyticsProvider;
};
