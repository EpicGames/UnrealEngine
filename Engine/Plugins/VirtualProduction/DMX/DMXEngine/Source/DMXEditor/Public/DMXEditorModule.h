// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeCategories.h"
#include "DMXPIEManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ISequencerModule.h"
#include "Modules/ModuleInterface.h"
#include "PropertyEditorDelegates.h"
#include "Toolkits/AssetEditorToolkit.h"

class FDMXEditor;
class FExtender;
class IAssetTools;
class IAssetTypeActions;
class UDMXLibrary;


/**
 * Implements the DMX Editor Module.
 */
class DMXEDITOR_API FDMXEditorModule
	: public IModuleInterface
	, public IHasMenuExtensibility
	, public IHasToolBarExtensibility
{
public:
	//~ Begin IModuleInterface implementation
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface implementation

	//~ Begin IHasMenuExtensibility implementation
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override { return MenuExtensibilityManager; }
	//~ End IHasMenuExtensibility implementation

	//~ Begin IHasToolBarExtensibility implementation
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override { return ToolBarExtensibilityManager; }
	//~ End IHasToolBarExtensibility implementation

	/** Get the instance of this module. */
	static FDMXEditorModule& Get();

	/**
	 * Creates an instance of a DMX Library editor object.
	 *
	 * Note: This function should not be called directly. It should be called from AssetTools handler
	 *
	 * @param	Mode					Mode that this editor should operate in
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	DMXLibrary				The DMX object to start editing
	 *
	 * @return	Interface to the new DMX editor
	 */
	TSharedRef<FDMXEditor> CreateEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UDMXLibrary* DMXLibrary );

	/** Exports a DMX Library as MVR file */
	void ExportDMXLibraryAsMVRFile(UDMXLibrary* DMXLibrary, const FString& DesiredName = TEXT("")) const;

	/** Get Level Editor Toolbar Menu extender */
	TSharedPtr<FExtender> GetLevelEditorToolbarDMXMenuExtender() const { return LevelEditorToolbarDMXMenuExtender; }

	static EAssetTypeCategories::Type GetAssetCategory() { return DMXEditorAssetCategory; }

	/** DataTable Editor app identifier string */
	static const FName DMXEditorAppIdentifier;

	/** The module DMX Editor Module name */
	static const FName ModuleName;

	/** The DMX Editor asset category */
	static EAssetTypeCategories::Type DMXEditorAssetCategory;

	//~ Gets the extensibility managers for outside entities to DMX editor's menus and toolbars
	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;

private:
	/** Before 5.4, GDTF assets could be stored containing invalid characters such as @ common to most GDTFs. This function mends all asset names.*/
	static void FixGDTFAssetNames();

	/** Binds commands for the DMX editor */
	void BindDMXEditorCommands();

	/** Creates the level editor toolbar extender */
	void ExtendLevelEditorToolbar();

	/** Creates the Extender for the Level Editor Toolbar DMX Menu */
	void CreateLevelEditorToolbarDMXMenuExtender();

	/** Generates the level editor toolbar DMX Menu */
	static TSharedRef<class SWidget> GenerateDMXLevelEditorToolbarMenu();

protected:
	/** Registers asset types categories */
	void RegisterAssetTypeCategories();

	/** Registers global class customizations */
	void RegisterClassCustomizations();

	/** Registers global property type customizations */
	void RegisterPropertyTypeCustomizations();

	/** Registers sequencer related types */
	void RegisterSequencerTypes();

	/** Registers nomad tab spawners */
	void RegisterNomadTabSpawners();

	/** Starts up the pie manager */
	void StartupPIEManager();

private:
	/** Called when the nomad tab spawner tries to spawn a channels monitor tab */
	static TSharedRef<class SDockTab> OnSpawnChannelsMonitorTab(const class FSpawnTabArgs& InSpawnTabArgs);

	/** Called when the nomad tab spawner tries to spawn an activity monitor tab */
	static TSharedRef<class SDockTab> OnSpawnActivityMonitorTab(const class FSpawnTabArgs& InSpawnTabArgs);

	/** Called when the nomad tab spawner tries to spawn an activity monitor tab */
	static TSharedRef<class SDockTab> OnSpawnConflictMonitorTab(const class FSpawnTabArgs& InSpawnTabArgs);

	/** Called when the nomad tab spawner tries to spawn a patch tool tab */
	static TSharedRef<class SDockTab> OnSpawnPatchToolTab(const class FSpawnTabArgs& InSpawnTabArgs);

	/** Opens the channel monitor */
	static void OpenChannelsMonitor();

	/** Opens the activity monitor */
	static void OpenActivityMonitor();

	/** Opens the conflict monitor*/
	static void OpenConflictMonitor();

	/** Opens the patch tool */
	static void OpenPatchTool();

	/** Called when the Toggle Receive DMX menu command is selected */
	static void OnToggleSendDMX();

	/** Returns true if send dmx is enabled */
	static bool IsSendDMXEnabled();

	/** Called when the Toggle Receive DMX menu command is selected */
	static void OnToggleReceiveDMX();

	/** Returns true if receive dmx is enabled */
	static bool IsReceiveDMXEnabled();

	/** Command list for the DMX level editor menu */
	TSharedPtr<class FUICommandList> DMXLevelEditorMenuCommands;

private:
	/**
	 * Helper to register a custom class layout, to ease unregistering with the corresponding unregister method.
	 *
	 * @param ClassName					The class name to register for details customization
	 * @param DetailLayoutDelegate		The delegate to call to get the custom detail layout instance
	 */
	void RegisterCustomClassLayout(FName ClassName, FOnGetDetailCustomizationInstance DetailLayoutDelegate);

	/** Unregisteres all registered custom class layouts */
	void UnregisterCustomClassLayouts();

	/**
	 * Helper to register a custom struct, to ease unregistering with the corresponding unregister method.
	 *
	 * @param StructName				The name of the struct to register for property customization
	 * @param StructLayoutDelegate		The delegate to call to get the custom detail layout instance
	 */
	void RegisterCustomPropertyTypeLayout(FName PropertyTypeName, FOnGetPropertyTypeCustomizationInstance PropertyTypeLayoutDelegate);

	/** Unregisteres all registered custom property type layouts */
	void UnregisterCustomPropertyTypeLayouts();

	/**
	 * Helper to register a custom sequencer track type, to ease unregistering with the corresponding unregister method.
	 * 	
	 * @param CreateTrackEditorDelegate		Delegate executed to create the custom track type
	 */
	void RegisterCustomSequencerTrackType(const FOnCreateTrackEditor& CreateTrackEditorDelegate);

	/** Unregisteres all custom sequencer track types */
	void UnregisterCustomSequencerTrackTypes();

	/**
	 * Helper to register a new normad tab spawner with the tab manager, to ease unregistering with the corresponding unregister method.
	 * 
	 * @param			TabId The TabId to register the spawner for.
	 * @param			OnSpawnTab The callback that will be used to spawn the tab.
	 * @param			CanSpawnTab The callback that will be used to ask if spawning the tab is allowed
	 * @return			The registration entry for the spawner.
	 */
	FTabSpawnerEntry& RegisterNomadTabSpawner(const FName TabId, const FOnSpawnTab& OnSpawnTab, const FCanSpawnTab& CanSpawnTab = FCanSpawnTab());

	/** Unregisteres all nomad tab spawners */
	void UnregisterNomadTabSpawners();

	/** Extender for the Level Editor Toolbar Menu */
	static TSharedPtr<FExtender> LevelEditorToolbarDMXMenuExtender;

	/** List of registered class that must be unregistered when the module shuts down */
	TSet<FName> RegisteredClassNames;

	/** List of registered property types that must be unregistered when the module shuts down */
	TSet<FName> RegisteredPropertyTypes;

	/** All custom sequencer track handles */
	TSet<FDelegateHandle> RegisteredSequencerTrackHandles;

	/** Names of all registered nomad tabs */
	TSet<FName> RegisteredNomadTabNames;

	/** Helper to track when the editor is in PIE */
	TUniquePtr<FDMXPIEManager> PIEManager;
};
