// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Engine/DeveloperSettings.h"
#include "Widgets/Views/SHeaderRow.h"

#include "DMXEditorSettings.generated.h"


/** Settings for the Fixture Patch List */
USTRUCT()
struct FDMXMVRFixtureListSettings
{
	GENERATED_BODY()

	UPROPERTY()
	float ListWidth = .5f;

	/** Width of the Fixture ID column */
	UPROPERTY()
	float FixtureIDColumnWidth = .1f;

	/** With of the Fixture Type column */
	UPROPERTY()
	float FixtureTypeColumnWidth = .2f;

	/** With of the Mode column */
	UPROPERTY()
	float ModeColumnWidth = .2f;

	/** With of the Patch column */
	UPROPERTY()
	float PatchColumnWidth = .2f;

	UPROPERTY()
	FName SortByCollumnID = NAME_None;

	UPROPERTY()
	int32 SortPriorityEnumIndex = 1;
};

UENUM()
enum class EDMXFixturePatcherNameDisplayMode : uint8
{
	FixtureIDAndFixturePatchName,
	FixtureID,
	FixturePatchName
};

/** Settings for the Fixture Patcher */
USTRUCT()
struct FDMXMVRFixturePatcherSettings
{
	GENERATED_BODY()

	UPROPERTY()
	bool bMonitorEnabled = false;

	UPROPERTY()
	EDMXFixturePatcherNameDisplayMode FixturePatchNameDisplayMode = EDMXFixturePatcherNameDisplayMode::FixturePatchName;
};


/** Settings for the Fixture Type Functions Editor */
USTRUCT()
struct FDMXFixtureTypeFunctionsEditorSettings
{
	GENERATED_BODY()

	/** Width of the function name column */
	UPROPERTY()
	float NameColumnWidth = 0.f;

	/** With of the attribute column */
	UPROPERTY()
	float AttributeColumnWidth = 0.f;
};

/** Struct to describe a single fader, so it can be stored in the config */
struct UE_DEPRECATED(5.2, "FDMXOutputConsoleFaderDescriptor is no longer supported. Please see FDMXControlConsoleEditorModule") FDMXOutputConsoleFaderDescriptor;
USTRUCT()
struct FDMXOutputConsoleFaderDescriptor
{
	GENERATED_BODY()

	FDMXOutputConsoleFaderDescriptor()
		: Value(0)
		, MaxValue(255)
		, MinValue(0)
		, UniversID(1)
		, StartingAddress(1)
		, EndingAddress(1)
	{}

	UPROPERTY()
	FString FaderName;

	UPROPERTY()
	uint8 Value;

	UPROPERTY()
	uint8 MaxValue;

	UPROPERTY()
	uint8 MinValue;
	
	UPROPERTY()
	int32 UniversID;

	UPROPERTY()
	int32 StartingAddress;

	UPROPERTY()
	int32 EndingAddress;

	UPROPERTY()
	FName ProtocolName;
};

/**
 * Struct to describe a monitor source, so it can be stored in settings 
 * Defaults to Monitor all Inputs.
 */
USTRUCT()
struct FDMXMonitorSourceDescriptor
{
	GENERATED_BODY()

	FDMXMonitorSourceDescriptor()
		: bMonitorAllPorts(true)
		, bMonitorInputPorts(true)
		, MonitoredPortGuid(FGuid())
	{}

	/** True if all ports should be monitored */
	UPROPERTY()
	bool bMonitorAllPorts;

	/** True if Input Ports should be monitored. Only relevant if bMonitorAllPorts */
	UPROPERTY()
	bool bMonitorInputPorts;

	/** The monitored Port Guid. Only relevant if !bMonitorAllPorts*/
	UPROPERTY()
	FGuid MonitoredPortGuid;
};

/** Settings for the conflict monitor */
USTRUCT()
struct FDMXConflictMonitorSettings
{
	GENERATED_BODY()

	/** When enabled, the conflict monitor pauses when a conflict occurs */
	UPROPERTY()
	bool bAutoPause = false;

	/** When enabled, the the conflict monitor prints conflicts to log */
	UPROPERTY()
	bool bPrintToLog = false;

	/** True if the conflict monitor starts when oppened */
	UPROPERTY()
	bool bRunWhenOpened = false;

	/** The displayed depth of traces */
	UPROPERTY()
	uint8 Depth = 3;
};

/** Settings that holds editor configurations. Not accessible in Project Settings. TODO: Idealy rename to UDMXEditorConfiguration */
UCLASS(Config = EditorPerProjectUserSettings)
class DMXEDITOR_API UDMXEditorSettings : public UObject
{
	GENERATED_BODY()

	// GDTF
public:
	UPROPERTY(Config)
	FString LastGDTFImportPath;

	// MVR
public:
	UPROPERTY(Config)
	FString LastMVRImportPath;

	UPROPERTY(Config)
	FString LastMVRExportPath;

	// DMX Library
public:
	/** Deprecated 5.3, moved to FDMXMVRFixturePatcherSettings */
	UPROPERTY(Config)
	bool bFixturePatcherDMXMonitorEnabled_DEPRECATED = false;

	UPROPERTY(Config)
	FDMXFixtureTypeFunctionsEditorSettings FixtureTypeFunctionsEditorSettings;

	UPROPERTY(Config)
	FDMXMVRFixtureListSettings MVRFixtureListSettings;

	UPROPERTY(Config)
	FDMXMVRFixturePatcherSettings FixturePatcherSettings;

	// Output Console (DEPRECATED 5.1)
public:
#if WITH_EDITORONLY_DATA
	/** Stores the faders specified in Output Console */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY(Config, Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated since Console is now saved in the new DMXControlConsolePreset, upgrade path is in DMXControlConsoleModule."))
	TArray<FDMXOutputConsoleFaderDescriptor> OutputConsoleFaders_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA

	// Control Console
public:
	/** Path to the last control console opened in the editor */
	UPROPERTY(Config)
	FString LastOpenedControlConsolePath;

	// Channels Monitor
public:
	/** The Universe ID to be monitored in the Channels Monitor */
	UPROPERTY(Config)
	int32 ChannelsMonitorUniverseID = 1;

	/** Source for the channels monitor */
	UPROPERTY(Config)
	FDMXMonitorSourceDescriptor ChannelsMonitorSource;

	// Activity Monitor
public:
	/** Source for the DMX Activity Monitor */
	UPROPERTY(Config)
	FDMXMonitorSourceDescriptor ActivityMonitorSource;

	/** ID of the first universe to monitor in the DMX Activity Monitor  */
	UPROPERTY(Config)
	int32 ActivityMonitorMinUniverseID = 1;

	/** ID of the last universe to monitor in the DMX Activity Monitor */
	UPROPERTY(Config)
	int32 ActivityMonitorMaxUniverseID = 100;

	// Conflict Monitor
public:
	/** Settings for the conflict monitor */
	UPROPERTY(Config)
	FDMXConflictMonitorSettings ConflictMonitorSettings;
};
