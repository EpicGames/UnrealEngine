// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateStyle.h"


/**  DMX editor style */
class DMXEDITOR_API FDMXEditorStyle
	: public FSlateStyleSet
{
public:
	/** Constructor */
	FDMXEditorStyle();

	/** Desonstructor */
	virtual ~FDMXEditorStyle();

	/**  Returns the singleton ISlateStyle instance for DMX editor style */
	static const FDMXEditorStyle& Get();

/// <summary>
/// DEPRECATED Members
/// </summary>

public:	
	/** DEPRECATED 5.0 */
	UE_DEPRECATED(5.0, "Deprecated since the public API leaves it unclear if the style is initialized or shut down. The style does not need to be Initialized explicitly anymore. Note, the StyleSetName now needs be accessed via FDMXEditorStyle::Get().GetStyleSetName().")
	static void Initialize();

	/** DEPRECATED 5.0 */
	UE_DEPRECATED(5.0, "Deprecated since the public API leaves it unclear if the style is initialized or shut down. The style does not need to be shut down explicitly anymore. Note, the StyleSetName now needs be accessed via FDMXEditorStyle::Get().GetStyleSetName().")
	static void Shutdown();

	/** DEPRECATED 5.0 */
	UE_DEPRECATED(5.0, "Deprecated since no clear use (call FSlateApplication::Get().GetRenderer()->ReloadTextureResources() directly where needed).")
	static void ReloadTextures();

private:
	/** Singleton style instance */
	static TSharedPtr<FSlateStyleSet> StyleInstance_DEPRECATED;
};
