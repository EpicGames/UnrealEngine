// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"
#include "Library/DMXEntity.h"

class UDMXLibrary;
class UDMXEntityFixtureType;
class UDMXEntityFader;
class UDMXEntityFixturePatch;

/** 
 * Generic Editor Utilities.
 * for Fixture Type, refer to DMXFixtureTypeSharedData instead.
 */
class DMXEDITOR_API FDMXEditorUtils
{
public:
	/**
	 * Validates an Entity name, also checking for uniqueness among others of the same type.
	 * @param NewEntityName		The name to validate.
	 * @param InLibrary			The DMXLibrary object to check for name uniqueness.
	 * @param InEntityClass		The type to check other Entities' names
	 * @param OutReason			If false is returned, contains a text with the reason for it.
	 * @return True if the name would be a valid one.
	 */
	static bool ValidateEntityName(const FString& NewEntityName, const UDMXLibrary* InLibrary, UClass* InEntityClass, FText& OutReason);

	/**  Renames an Entity */
	static void RenameEntity(UDMXLibrary* InLibrary, UDMXEntity* InEntity, const FString& NewName);

	/**  Checks if the Entity is being referenced by other objects. */
	static bool IsEntityUsed(const UDMXLibrary* InLibrary, const UDMXEntity* InEntity);

	/**  Copies Entities to the operating system's clipboard. */
	static void CopyEntities(const TArray<UDMXEntity*>&& EntitiesToCopy);

	/**  Determines whether the current contents of the clipboard contain paste-able DMX Entity information */
	static bool CanPasteEntities(UDMXLibrary* ParentLibrary);

	/**
	 * Creates the copied DMX Entities from the clipboard without attempting to paste/apply them in any way
	 * 
	 * @param ParentLibrary			The library in which the entities are created.
	 * @return						The array of newly created enities.
	 */
	static TArray<UDMXEntity*> CreateEntitiesFromClipboard(UDMXLibrary* ParentLibrary);

	/**
	 * Compares the property values of two Fixture Types, including properties in arrays,
	 * and returns true if they are almost all the same.
	 * Name, ID and Parent Library are ignored.
	 */
	static bool AreFixtureTypesIdentical(const UDMXEntityFixtureType* A, const UDMXEntityFixtureType* B);

	/**  Returns the Entity class type name (e.g: Fixture Type for UDMXEntityFixtureType) in singular or plural */
	static FText GetEntityTypeNameText(TSubclassOf<UDMXEntity> EntityClass, bool bPlural = false);

	/**
	 * Creates a unique color for all patches that use the default color FLinearColor(1.0f, 0.0f, 1.0f)
	 *
	 * @param Library				The library the patches resides in.
	 */
	static void UpdatePatchColors(UDMXLibrary* Library);

	/**
	 * Retrieve all assets for a given class via the asset registry. Will load into memory if needed.
	 *
	 * @param Class					The class to lookup.
	 * @param OutObjects			All found objects.
	 * 
	 */
	static void GetAllAssetsOfClass(UClass* Class, TArray<UObject*>& OutObjects);

	/**
	 * Locate universe conflicts between libraries
	 *
	 * @param Library					The library to be tested.
	 * @param OutConflictMessage		Message containing found conflicts
	 * @return							True if there is at least one conflict found.
	 */
	static bool DoesLibraryHaveUniverseConflicts(UDMXLibrary* Library, FText& OutInputPortConflictMessage, FText& OutOutputPortConflictMessage);

	/** Zeros memory in all active DMX buffers of all protocols */
	UE_DEPRECATED(5.5, "Instead use FDMXPortManager::ClearPortBuffers")
	static void ClearAllDMXPortBuffers();

	/** Clears cached data fixture patches received */
	UE_DEPRECATED(5.5, "Instead use UDMXSubsystem::ClearDMXBuffers.")
	static void ClearFixturePatchCachedData();

	/** Gets the package or creates a new one if it doesn't exist */
	static UPackage* GetOrCreatePackage(TWeakObjectPtr<UObject> Parent, const FString& DesiredName);
	
	/** Parses Attribute Names from a String, format can be 'Dimmer', 'Dimmer, Pan, Tilt''. If no valid Attribute Names  can be parsed, returned Array is empty. */
	[[nodiscard]] static TArray<FString> ParseAttributeNames(const FString& InputString);

	/** Parses Universes from a String, format can be '1.', 'Uni 1', 'Universe 1', 'Uni 1, 3-4'. If no valid Universe can be parsed, returned Array is empty. */
	[[nodiscard]] static TArray<int32> ParseUniverses(const FString& InputString);

	/** Parses an Address from a String, exected format is in the form of 'universe.address', e.g. '1.1' */
	[[nodiscard]] static  bool ParseAddress(const FString& InputString, int32& OutAddress);

	/** Parses Fixture IDs from a String. Format can be '1', '1, 3-4'. If no valid Fixture ID can be parsed, returned Array is empty */
	[[nodiscard]] static TArray<int32> ParseFixtureIDs(const FString& InputString);

	/** Parses a Fixture ID from a String, exected format is an integral value, e.g. '1' */
	[[nodiscard]] static bool ParseFixtureID(const FString& InputString, int32& OutFixtureID);

	// can't instantiate this class
	FDMXEditorUtils() = delete;
};
