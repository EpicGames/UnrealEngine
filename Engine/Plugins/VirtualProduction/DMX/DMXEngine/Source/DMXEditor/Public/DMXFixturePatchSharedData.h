// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "Templates/SharedPointer.h"
#include "UObject/GCObject.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FDMXEditor;
class FDMXFixturePatchNode;
class SDMXEntityList;
class UDMXEntityFixturePatch;
class UDMXFixturePatchSharedDataSelection;
class UDMXLibrary;


/** Shared data for fixture patch editors */
class FDMXFixturePatchSharedData
	: public FGCObject
	, public FSelfRegisteringEditorUndoClient
	, public TSharedFromThis<FDMXFixturePatchSharedData>
{
public:
	/** Constructor */
	FDMXFixturePatchSharedData(TWeakPtr<FDMXEditor> InDMXEditorPtr);

	//~ Begin FGCObject
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("DMXEditor::DMXFixturePatchSharedData");
	}
	//~End FGCObject

	//~Begin EditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	//~End EditorUndoClient interface

	/** Broadcast when a universe is selected by an editor */
	FSimpleMulticastDelegate OnUniverseSelectionChanged;

	/** Broadcast when a patch node is selected by an editor */
	FSimpleMulticastDelegate OnFixturePatchSelectionChanged;

	/** Selects the universe */
	void SelectUniverse(int32 UniverseID);

	/** Returns the selected universe */
	int32 GetSelectedUniverse();

	/** Selects the patch node */
	void SelectFixturePatch(TWeakObjectPtr<UDMXEntityFixturePatch> Patch);

	/** Selects the patch node, does not clear selection */
	void AddFixturePatchToSelection(TWeakObjectPtr<UDMXEntityFixturePatch> Patch);

	/** Selects the patch nodes */
	void SelectFixturePatches(const TArray<TWeakObjectPtr<UDMXEntityFixturePatch>>& Patches);

	/** Returns the selected patch node or nullptr if nothing is selected */
	const TArray<TWeakObjectPtr<UDMXEntityFixturePatch>>& GetSelectedFixturePatches() const;

private:
	/** The current selection */
	TObjectPtr<UDMXFixturePatchSharedDataSelection> Selection;

	/** Weak reference to the DMX editor */
	TWeakPtr<FDMXEditor> DMXEditorPtr;
};
