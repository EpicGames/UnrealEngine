// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DMXFixturePatchSharedDataSelection.generated.h"

class UDMXEntityFixturePatch;


/** An Array of Fixture Patches that supports a transaction history resp. undo / redo */
UCLASS()
class UDMXFixturePatchSharedDataSelection
	: public UObject
{
	GENERATED_BODY()

public:
	/** Selected Fixture Patches */
	UPROPERTY()
	TArray<TWeakObjectPtr<UDMXEntityFixturePatch>> SelectedFixturePatches;

	/** Selected Universe */
	UPROPERTY()
	int32 SelectedUniverse = 1;
};
