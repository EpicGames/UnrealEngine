// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDMXNamedType.h"

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "Templates/SharedPointer.h"
#include "UObject/GCObject.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "JsonObjectConverter.h"

enum class EDMXFixtureSignalFormat : uint8;
class FDMXEditor;
class FDMXFixtureTypeSharedData;
class UDMXEntityFixtureType;
class UDMXFixtureTypeSharedDataSelection;

class FScopedTransaction;


/** Shared data for Fixture Types in a DMX Editor */
class FDMXFixtureTypeSharedData
	: public FGCObject
	, public FSelfRegisteringEditorUndoClient
	, public TSharedFromThis<FDMXFixtureTypeSharedData>
{
public:
	/** Constructor */
	FDMXFixtureTypeSharedData(TWeakPtr<FDMXEditor> InDMXEditorPtr);

	//~ Begin FGCObject
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("DMXEditor::DMXFixtureTypeSharedData");
	}
	//~End FGCObject

	//~Begin EditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	//~End EditorUndoClient interface

	/** Selects specified Fixture Types */
	void SelectFixtureTypes(const TArray<TWeakObjectPtr<UDMXEntityFixtureType>>& InFixtureTypes);

	/** Selects specified Modes */
	void SelectModes(const TArray<int32>& InModeIndices);

	/** Selects specified Functions */
	void SetFunctionAndMatrixSelection(const TArray<int32>& InFunctionIndices, bool bMatrixSelected);

	/** Selects specified Functions */
	UE_DEPRECATED(5.0, "Deprecated in favor of SetFunctionAndMatrixSelection to avoid the unclear state where both need change, but one contains the old state while the other changed.")
	void SelectFunctions(const TArray<int32>& InFunctionIndices);

	const TArray<TWeakObjectPtr<UDMXEntityFixtureType>>& GetSelectedFixtureTypes() const;
	const TArray<int32>& GetSelectedModeIndices() const;
	const TArray<int32>& GetSelectedFunctionIndices() const;
	bool IsFixtureMatrixSelected() const;

	/** Returns true if selected Modes can be copied */
	bool CanCopyModesToClipboard() const;

	/** Copies selected Modes. Only should be called when CanCopyModesToClipboard */
	void CopyModesToClipboard();

	/** Returns true if Modes can be pasted */
	bool CanPasteModesFromClipboard() const;

	/** Pastes the Modes clipboard to selected Modes, returns the newly added Mode indices. Only should be called when CanPasteModesFromClipboard */
	void PasteModesFromClipboard(TArray<int32>& OutNewlyAddedModeIndices);

	/** Returns true if selected Functions can be copied */
	bool CanCopyFunctionsToClipboard() const;

	/** Copies selected Functions. Only should be called when CanCopyFunctionsToClipboard */
	void CopyFunctionsToClipboard();

	/** Returns true if Functions can be pasted */
	bool CanPasteFunctionsFromClipboard() const;

	/** Pastes the Modes clipboard to selected Functions, returns the newly added Function indices. Only should be called when CanPasteFunctionsFromClipboard */
	void PasteFunctionsFromClipboard(TArray<int32>& OutNewlyAddedFunctionIndices);

	UE_DEPRECATED(5.0, "Removed for a cleaner API. Please make up logic in place where this was used (e.g. instead use GetSelectedModes().Num() == 1 from this class).")
	bool CanAddMode() const;

	UE_DEPRECATED(5.0, "Replaced with UDMXEntityFixtureType::AddMode.")
	void AddMode();

	UE_DEPRECATED(5.0, "Replaced with UDMXEntityFixtureType::DuplicateModes.")
	void DuplicateModes(const TArray<int32>& ModeIndicesToDuplicate);

	UE_DEPRECATED(5.0, "Replaced with UDMXEntityFixtureType::DeleteModes.")
	void DeleteModes(const TArray<int32>& ModeIndicesToDelete);

	UE_DEPRECATED(5.0, "Replaced with FDMXFixtureTypeSharedData::PasteModesFromClipboard.")
	void PasteClipboardToModes(const TArray<int32>& ModeIndices);

	UE_DEPRECATED(5.0, "Removed for a cleaner API. Please make up logic in place where this was used (e.g. instead use GetSelectedFunctions().Num() == 1 from this class).")
	bool CanAddFunction() const;

	UE_DEPRECATED(5.0, "Replaced with UDMXEntityFixtureType::AddFunction.")
	void AddFunctionToSelectedMode();

	UE_DEPRECATED(5.0, "Replaced with UDMXEntityFixtureType::DuplicateFunctions.")
	void DuplicateFunctions(const TArray<int32>& FunctionIndicesToDuplicate);

	UE_DEPRECATED(5.0, "Replaced with UDMXEntityFixtureType::DeleteFunctions.")
	void DeleteFunctions(const TArray<int32>& FunctionIndicesToDelete);

	UE_DEPRECATED(5.0, "Replaced with FDMXFixtureTypeSharedData::PasteFunctionsFromClipboard.")
	void PasteClipboardToFunctions(const TArray<int32>& FunctionIndices);

	UE_DEPRECATED(5.0, "Removed for a cleaner API. Please make up logic in place where this was used (e.g. instead use IsMatrixSelected() from this class).")
	bool CanAddCellAttribute() const;

	UE_DEPRECATED(5.0, "Replaced with UDMXEntityFixtureType::AddCellAttribute.")
	void AddCellAttributeToSelectedMode();

	/** Broadcasts when selected Fixture Types changed */
	FSimpleMulticastDelegate OnFixtureTypesSelected;

	/** Broadcasts when selected Modes changed */
	FSimpleMulticastDelegate OnModesSelected;

	/** Broadcasts when selected Functions changed */
	FSimpleMulticastDelegate OnFunctionsSelected;

	/** Broadcasts when the Matrix was selected or unselected */
	FSimpleMulticastDelegate OnMatrixSelectionChanged;

private:
	/** The Fixture types being edited */
	TObjectPtr<UDMXFixtureTypeSharedDataSelection> Selection;

	/** Cache for multi mode copy/paste*/
	TArray<FString> ModesClipboard;

	/** Cache for multi function copy/paste*/
	TArray<FString> FunctionsClipboard;

	/** Weak reference to the DMX editor */
	TWeakPtr<FDMXEditor> DMXEditorPtr;	
};
