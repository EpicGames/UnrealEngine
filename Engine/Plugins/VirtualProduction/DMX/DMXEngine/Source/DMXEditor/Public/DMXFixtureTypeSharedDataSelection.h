// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DMXFixtureTypeSharedDataSelection.generated.h"

class UDMXEntityFixtureType;


/** An Array of Fixture Types that supports a transaction history resp. undo / redo */
UCLASS()
class UDMXFixtureTypeSharedDataSelection
	: public UObject
{
	GENERATED_BODY()

public:
	/** The Fixture types being edited */
	UPROPERTY()
	TArray<TWeakObjectPtr<UDMXEntityFixtureType>> SelectedFixtureTypes;

	/** The Mode indices in the Selected Fixture Types currently being selected */
	UPROPERTY()
	TArray<int32> SelectedModeIndices;

	/** The Function indices in the Selected Fixture Types currently being selected  */
	UPROPERTY()
	TArray<int32> SelectedFunctionIndices;

	/** If true the Fixture Matrices in the currently seleccted Modes are selected */
	UPROPERTY()
	bool bFixtureMatrixSelected = false;
};
