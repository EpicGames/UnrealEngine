// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


class FDMXPIEManager
{
public:
	/** Constructor */
	FDMXPIEManager();

	/** Destructor */
	~FDMXPIEManager();

private:
	/** Called when play in editor starts */
	void OnBeginPIE(const bool bIsSimulating);

	/** Called when play in editor ends */
	void OnEndPIE(const bool bIsSimulating);
};
