// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "DMXEditorFactoryNew.generated.h"

class UDMXLibrary;

/** DEPRECATED 5.1 */
class UE_DEPRECATED(5.1, "Deprecated class. Please use DMXLibraryFactory instead.") UDMXEditorFactoryNew;
UCLASS(hidecategories = Object)
class DMXEDITOR_API UDMXEditorFactoryNew : public UFactory
{
	GENERATED_BODY()
public:
	UDMXEditorFactoryNew();

	//~ Begin UFactory Interface
	virtual FText GetDisplayName() const override { return FText::FromString(FString("DEPRECATED_DMXEditorFactoryNew")); }
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	

private:
	UDMXLibrary* MakeNewEditor(UObject* InParent, FName Name, EObjectFlags Flags);
};
