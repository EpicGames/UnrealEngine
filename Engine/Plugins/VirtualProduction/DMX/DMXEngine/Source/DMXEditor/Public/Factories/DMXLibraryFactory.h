// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "DMXLibraryFactory.generated.h"

class UDMXLibrary;


UCLASS()
class DMXEDITOR_API UDMXLibraryFactory 
	: public UFactory
{
	GENERATED_BODY()
public:
	UDMXLibraryFactory();

	//~ Begin UFactory Interface
	virtual FText GetDisplayName() const override { return FText::FromString(TEXT("DMX Library")); }
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	uint32 GetMenuCategories() const override;
	//~ Begin UFactory Interface	
};
