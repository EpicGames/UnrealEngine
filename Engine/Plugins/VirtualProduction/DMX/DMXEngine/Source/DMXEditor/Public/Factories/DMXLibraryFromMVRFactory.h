// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "EditorReimportHandler.h"

#include "DMXLibraryFromMVRFactory.generated.h"

class UDMXMVRAssetImportData;


UCLASS()
class DMXEDITOR_API UDMXLibraryFromMVRFactory 
	: public UFactory
	, public FReimportHandler
{
	GENERATED_BODY()

public:
	/** Constructor */
	UDMXLibraryFromMVRFactory();

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateFile(UClass* InClass, UObject* Parent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCancelled);
	virtual bool FactoryCanImport(const FString& Filename) override;
	//~ End UFactory Interface	

	/** File extention for MVR files */
	static const FString MVRFileExtension;

protected:
	//~ Begin FReimportHandler Interface
	virtual bool CanReimport(UObject* Obj, TArray<FString>& OutFilenames) override;
	virtual void SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths) override;
	virtual EReimportResult::Type Reimport(UObject* Obj) override;
	virtual int32 GetPriority() const override;
	//~ End FReimportHandler Interface

private:
	/** Returns the MVR Asset Import Data for the Object, or nullptr if it cannot be obtained */
	UDMXMVRAssetImportData* GetMVRAssetImportData(UObject* DMXLibraryObject) const;
};
