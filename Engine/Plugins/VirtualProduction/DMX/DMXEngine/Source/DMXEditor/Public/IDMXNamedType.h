// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"


/** Interface for objects that want to define a unique name across a set of objects */
class IDMXNamedType
	: public TSharedFromThis<IDMXNamedType>
{

public:
	virtual ~IDMXNamedType() {}

	/** Returns the name */
	virtual void GetName(FString& OutUniqueName) const = 0;

	/** Returns if the name is unique */
	virtual bool IsNameUnique(const FString& TestedName) const = 0;

	/** Sets the name, matching desired name as closely as possible */
	virtual void SetName(const FString& InDesiredName, FString& OutUniqueName) = 0;
};
