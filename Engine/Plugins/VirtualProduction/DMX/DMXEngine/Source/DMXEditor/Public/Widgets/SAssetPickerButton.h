// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "AssetRegistry/AssetData.h"

class SButton;

DECLARE_DELEGATE_OneParam(FOnAssetSelected, const FAssetData&);
DECLARE_DELEGATE_RetVal(bool, FOnParentIsHovered);

/**  A widget which allows the user to pick a name of a specified list of names. */
class DMXEDITOR_API SAssetPickerButton
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAssetPickerButton)
		: _CurrentAssetValue(nullptr)
		, _AssetClass(UObject::StaticClass())
		, _AllowedClasses()
		, _DisallowedClasses()
		, _OnParentIsHovered()
		, _OnAssetSelected()
	{}
		/** The current asset value for the button label */
		SLATE_ATTRIBUTE(TWeakObjectPtr<UObject>, CurrentAssetValue)

		/**
		 * The class to filter the list of assets in the picker menu.
		 * Required for at least the Use Selected on Content Browser button.
		 */
		SLATE_ARGUMENT(TWeakObjectPtr<UClass>, AssetClass)
		/** Allows more than a single class on the picker list.
		 * Overrides AssetClass (doesn't affect the Use Selected on Content Browser button)
		 */
		SLATE_ARGUMENT(TArray<FTopLevelAssetPath>, AllowedClasses)
		/** Disallows subclasses from the allowed asset class(es).
		 * Doesn't affect the Use Selected on Content Browser button.
		 */
		SLATE_ARGUMENT(TArray<FTopLevelAssetPath>, DisallowedClasses)

		/** Called to decide the hovered state using the parent's interact able area */
		SLATE_EVENT(FOnParentIsHovered, OnParentIsHovered)
		/** Called when a new asset is selected */
		SLATE_EVENT(FOnAssetSelected, OnAssetSelected)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

public:
	/** The class to filter the list of assets in the picker menu */
	TWeakObjectPtr<UClass> AssetClass;
	/** Allows more than a single class. Overrides AssetClass */
	TArray<FTopLevelAssetPath> AllowedClasses;
	/** Disallows subclasses from the allowed asset class(es) */
	TArray<FTopLevelAssetPath> DisallowedClasses;

protected:
	/** Clicked Use button */
	virtual FReply OnClickUse();
	/** Clicked Browse button */
	virtual FReply OnClickBrowse();
	/** Get text tooltip for object */
	FText GetObjectToolTip() const;
	/** Get default text for the picker combo */
	virtual FText GetDefaultComboText() const;
	/** Allow self pin widget */
	virtual bool AllowSelfPinWidget() const { return true; }
	/** Generate asset picker window */
	virtual TSharedRef<SWidget> GenerateAssetPicker();
	/** Called to validate selection from picker window */
	virtual void OnAssetSelectedFromPicker(const struct FAssetData& AssetData);
	/** Called when enter is pressed when items are selected in the picker window */
	void OnAssetEnterPressedInPicker(const TArray<FAssetData>& InSelectedAssets);

	/** Used to update the combo button text */
	FText OnGetComboTextValue() const;
	/** Returns whether the parent widget is hovered to use it's area for hovered state */
	bool GetIsParentHovered() const;
	/** Combo Button Color and Opacity delegate */
	FSlateColor OnGetComboForeground() const;
	/** Button Color and Opacity delegate */
	FSlateColor OnGetWidgetForeground() const;
	/** Button Color and Opacity delegate */
	FSlateColor OnGetWidgetBackground() const;

	/** Returns asset data of currently selected object, if bRuntimePath is true this will include _C for blueprint classes, for false it will point to UBlueprint instead */
	virtual const FAssetData& GetAssetData() const;

protected:
	/** Object manipulator buttons. */
	TSharedPtr<SButton> UseButton;
	TSharedPtr<SButton> BrowseButton;

	/** Menu anchor for opening and closing the asset picker */
	TSharedPtr<class SMenuAnchor> AssetPickerAnchor;

	/** Cached AssetData of object selected */
	mutable FAssetData CachedAssetData;

	/** The current asset selected, to display its name */
	TAttribute<TWeakObjectPtr<UObject>> CurrentAssetValue;

	/** Broadcast when defining hovered state, to use the parent interact able area */
	FOnParentIsHovered OnParentIsHovered;
	/** Broadcast when a new asset is selected */
	FOnAssetSelected OnAssetSelected;
};
