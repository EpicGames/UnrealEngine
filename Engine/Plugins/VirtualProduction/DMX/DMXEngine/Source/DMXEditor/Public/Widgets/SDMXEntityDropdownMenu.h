// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commands/DMXEditorCommands.h"
#include "DMXEditor.h"
#include "DMXEditorUtils.h"
#include "Library/DMXEntity.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/DMXLibrary.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "SListViewSelectorDropdownMenu.h"
#include "Styling/AppStyle.h"
#include "Templates/SubclassOf.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Views/SListView.h"

class SSearchBox;
class SToolTip;
class UDMXLibrary;

#define LOCTEXT_NAMESPACE "SDMXEntitySelectorDropdownMenu"

enum class EEntityEntryType : uint8
{
	/** Create a new Entity in the DMX library */
	NewEntity,
	/** Select existing Entity from the DMX library */
	ExistingEntity,
	/** Heading entry */
	Heading,
	/** Separator bar entry */
	Separator
};

class FDMXEntityEntry
	: public TSharedFromThis<FDMXEntityEntry>
{
public:
	/** Constructor for the entry that creates a new Entity or a heading */
	FDMXEntityEntry(const FText& InLabelText, EEntityEntryType InEntryType = EEntityEntryType::Heading)
		: Entity(nullptr)
		, LabelText(InLabelText)
		, EntryType(InEntryType)
	{}

	/** Constructor for existing Entity entry */
	FDMXEntityEntry(UDMXEntity* InEntity)
		: Entity(InEntity)
		, LabelText(FText::GetEmpty())
		, EntryType(EEntityEntryType::ExistingEntity)
	{}

	/** Constructor for separator entry */
	FDMXEntityEntry()
		: Entity(nullptr)
		, LabelText(FText::GetEmpty())
		, EntryType(EEntityEntryType::Separator)
	{}


	UDMXEntity* GetEntity() const { return Entity; }

	FText GetLabelText() const
	{
		if (IsEntity())
		{
			return FText::FromString(Entity->GetDisplayName());
		}
		return LabelText;
	}

	bool IsHeading() const { return EntryType == EEntityEntryType::Heading; }

	bool IsSeparator() const { return EntryType == EEntityEntryType::Separator; }

	bool IsEntity() const { return EntryType == EEntityEntryType::ExistingEntity; }

	bool IsCreateNew() const { return EntryType == EEntityEntryType::NewEntity; }

	EEntityEntryType GetEntryType() const { return EntryType; }

private:
	UDMXEntity* Entity;
	FText LabelText;
	EEntityEntryType EntryType;
};

//////////////////////////////////////////////////////////////////////////

DECLARE_DELEGATE_OneParam(FOnEntitySelected, UDMXEntity*);
DECLARE_DELEGATE(FOnCreateNewEntitySelected);

/**
 * Generates a dropdown menu list of Entities from a specific type to be used as
 * a dropdown for a SComboButton's MenuContent.
 */
template<typename TEntityType>
class SDMXEntityDropdownMenu
	: public SListViewSelectorDropdownMenu< TSharedPtr< FDMXEntityEntry > >
{
	using SDMXDropdownMenuType = SDMXEntityDropdownMenu<TEntityType>;

public:
	SLATE_BEGIN_ARGS(SDMXEntityDropdownMenu)
		: _DMXEditor(nullptr)
		, _DMXLibrary(nullptr)
		, _EntityTypeFilter(nullptr)
		, _OnCreateNewEntitySelected()
		, _OnEntitySelected()
		{}

		/** If valid, enables the option to create a new entity. */
		SLATE_ATTRIBUTE(TWeakPtr<FDMXEditor>, DMXEditor)

		/** Needed to find existing Entities. Can be inferred from DMXEditor, if that's valid. */
		SLATE_ATTRIBUTE(TWeakObjectPtr<UDMXLibrary>, DMXLibrary)

		/** If set, overrides the template type. Use UDMXEntity as template in this case */
		SLATE_ATTRIBUTE(TSubclassOf<UDMXEntity>, EntityTypeFilter)
		
		/** Called when the user selects the option to create a new Entity */
		SLATE_EVENT(FOnCreateNewEntitySelected, OnCreateNewEntitySelected)
		/** Called when the user selects an existing Entity from the list */
		SLATE_EVENT(FOnEntitySelected, OnEntitySelected)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		OnCreateNewEntityDelegate = InArgs._OnCreateNewEntitySelected;
		OnEntitySelectedDelegate = InArgs._OnEntitySelected;
		CurrentExistingEntitiesNum = 0;
		PrevSelectedIndex = INDEX_NONE;

		EntityTypeFilter = InArgs._EntityTypeFilter;

		EntityTypeName = FDMXEditorUtils::GetEntityTypeNameText(GetEntityFilterClass());

		DMXEditor = InArgs._DMXEditor;
		DMXLibrary = InArgs._DMXLibrary;

		SAssignNew(EntitiesListView, SListView< TSharedPtr<FDMXEntityEntry> >)
			.ListItemsSource(&FilteredEntityEntries)
			.OnSelectionChanged(this, &SDMXDropdownMenuType::OnEntrySelectionChanged)
			.OnGenerateRow(this, &SDMXDropdownMenuType::GenerateEntryRow)
			.SelectionMode(ESelectionMode::Single);

		SAssignNew(SearchBox, SSearchBox)
			.HintText(FText::Format(LOCTEXT("EntitiesComboSearchBoxHint", "Search {0}"), EntityTypeName))
			.OnTextChanged(this, &SDMXDropdownMenuType::OnSearchBoxTextChanged)
			.OnTextCommitted(this, &SDMXDropdownMenuType::OnSearchBoxTextCommitted);

		// Construct the parent class
		SListViewSelectorDropdownMenu<TSharedPtr<FDMXEntityEntry>>::Construct
		(
			// We need to use typename in order to compile on Linux
			typename SListViewSelectorDropdownMenu<TSharedPtr<FDMXEntityEntry> >::FArguments()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				.Padding(2)
				[
					SNew(SBox)
					.WidthOverride(250)
					[				
						SNew(SVerticalBox)
						+SVerticalBox::Slot()
						.Padding(1.f)
						.AutoHeight()
						[
							SearchBox.ToSharedRef()
						]
						+SVerticalBox::Slot()
						.MaxHeight(400)
						[
							EntitiesListView.ToSharedRef()
						]
					]
				]
			],
			SearchBox,
			EntitiesListView
		);

		EntitiesListView->EnableToolTipForceField(true);

		RefreshEntitiesList();
	}

	/** Updates the entities list */
	void RefreshEntitiesList()
	{
		EntityEntries.Empty();

		const UDMXLibrary* Library = DMXLibrary.Get().Get();

		// Two chances to get a valid DMX Library
		if (Library == nullptr)
		{
			if (TSharedPtr<FDMXEditor> Editor = DMXEditor.Get().Pin())
			{
				Library = Editor->GetDMXLibrary();
			}
		}

		if (Library == nullptr)
		{
			// Missing DMX Library message
			static const FText NoLibraryLabel(LOCTEXT("NoLibraryHeading", "No DMX Library Selected"));
			TSharedPtr<FDMXEntityEntry> NoLibraryHeadingEntry = MakeShared<FDMXEntityEntry>(NoLibraryLabel);
			EntityEntries.Add(NoLibraryHeadingEntry);

			UpdateFilteredEntitiesList(SearchBox.IsValid() ? SearchBox->GetSearchText().ToString() : TEXT(""));

			return;
		}

		if (DMXEditor.Get().IsValid())
		{
			// First heading entry, for Entity creation category
			static const FText CreationHeadingLabel(LOCTEXT("CreationHeadingLabel", "Create"));
			TSharedPtr<FDMXEntityEntry> CreationHeadingEntry = MakeShared<FDMXEntityEntry>(CreationHeadingLabel);
			EntityEntries.Add(CreationHeadingEntry);

			// Entry to create a new Entity
			static const FText CreationEntryLabel(FText::Format(LOCTEXT("CreationEntryLabel", "New {0}..."), EntityTypeName));
			TSharedPtr<FDMXEntityEntry> CreationButtonEntry = MakeShared<FDMXEntityEntry>(CreationEntryLabel, EEntityEntryType::NewEntity);
			EntityEntries.Add(CreationButtonEntry);

			// Separator between creation and existing types sections
			EntityEntries.Add(MakeShared<FDMXEntityEntry>());

			// Second heading entry, for existing Entity selection category
			static const FText ExistingTypesLabel(LOCTEXT("ExistingTypesLabel", "Select Existing"));
			TSharedPtr<FDMXEntityEntry> ExistingHeadingEntry = MakeShared<FDMXEntityEntry>(ExistingTypesLabel);
			EntityEntries.Add(ExistingHeadingEntry);
		}

		// Add an entry for each Entity in the DMX library asset
		CurrentExistingEntitiesNum = 0;
		Library->ForEachEntityOfType(GetEntityFilterClass(), [this](UDMXEntity* Entity)
			{
				++CurrentExistingEntitiesNum;
				TSharedPtr<FDMXEntityEntry> ExistingEntityEntry = MakeShared<FDMXEntityEntry>(Entity);
				EntityEntries.Add(ExistingEntityEntry);
			});

		UpdateFilteredEntitiesList(SearchBox.IsValid() ? SearchBox->GetSearchText().ToString() : TEXT(""));
	}

	/**
	 * Sets the button that opens this dropdown menu.
	 * This automates closing the menu when an entry is selected
	 * and focusing on the Search Box.
	 */
	void SetComboButton(TSharedPtr<SComboButton> InComboButton)
	{
		if (InComboButton.IsValid())
		{
			ComboButtonPtr = InComboButton;
			// When the menu is opened, the search box will get focused
			InComboButton->SetMenuContentWidgetToFocus(SearchBox);
		}
	}

	void ClearSelection()
	{
		SearchBox->SetText(FText::GetEmpty());

		// Clear the selection and removes keyboard focus
		EntitiesListView->SetSelection(nullptr, ESelectInfo::OnNavigation);

		// Make sure we scroll to the top
		if (CurrentExistingEntitiesNum > 0)
		{
			EntitiesListView->RequestScrollIntoView(FilteredEntityEntries[0]);
		}
	}

private:
	TSubclassOf<UDMXEntity> GetEntityFilterClass() const
	{
		TSubclassOf<UDMXEntity> FromFilter = EntityTypeFilter.Get();
		if (FromFilter != nullptr)
		{
			return FromFilter; 
		}

		return TEntityType::StaticClass();
	}

	void UpdateFilteredEntitiesList(const FString& InSearchText /*= TEXT("") */)
	{
		// Don't bother filtering if we have nothing to filter
		if (CurrentExistingEntitiesNum == 0 || InSearchText.IsEmpty())
		{
			FilteredEntityEntries = EntityEntries;
		}
		else
		{
			FilteredEntityEntries.Empty();

			for (TSharedPtr<FDMXEntityEntry>& MenuEntry : EntityEntries)
			{
				// skip separators and headers when filtering
				if (MenuEntry->IsHeading() || MenuEntry->IsSeparator())
				{
					continue;
				}

				if (MenuEntry->GetLabelText().ToString().Contains(InSearchText))
				{
					FilteredEntityEntries.Add(MenuEntry);
				}
			}

			// select the first entry that passed the filter
			if (FilteredEntityEntries.Num() > 0)
			{
				EntitiesListView->SetSelection(FilteredEntityEntries[0], ESelectInfo::OnNavigation);
			}
		}

		// Ask the list to update its contents on next tick
		EntitiesListView->RequestListRefresh();
	}

	void OnEntrySelectionChanged(TSharedPtr<FDMXEntityEntry> InItem, ESelectInfo::Type SelectInfo)
	{
		if (InItem.IsValid() && (InItem->IsEntity() || InItem->IsCreateNew()) && SelectInfo != ESelectInfo::OnNavigation)
		{
			// We don't want the item to remain selected
			ClearSelection();
			// Close the menu
			if (TSharedPtr<SComboButton> PinnedComboBtn = ComboButtonPtr.Pin())
			{
				PinnedComboBtn->SetIsOpen(false);
			}

			if (InItem->IsEntity())
			{
				if (OnEntitySelectedDelegate.IsBound())
				{
					UDMXEntity* Entity = InItem->GetEntity();
					check(Entity != nullptr);
				
					OnEntitySelectedDelegate.ExecuteIfBound(Entity);
				}
			}
			else if (InItem->IsCreateNew())
			{
				if (TSharedPtr<FDMXEditor> PinnedEditor = DMXEditor.Get().Pin())
				{
					const TSubclassOf<UDMXEntity>&& FilterClass = GetEntityFilterClass();

					const FText TransactionText = FilterClass->IsChildOf(UDMXEntityFixtureType::StaticClass()) ?
						LOCTEXT("AddFixtureTypeTransaction", "Add Fixture Type") :
						LOCTEXT("AddFixturePatchTransaction", "Add Fixture Patch");

					const FScopedTransaction AddEntityTransaction(TransactionText);

					if (FilterClass->IsChildOf(UDMXEntityFixtureType::StaticClass()))
					{
						PinnedEditor->GetToolkitCommands()->ExecuteAction(FDMXEditorCommands::Get().AddNewEntityFixtureType.ToSharedRef());

						PinnedEditor->InvokeEditorTabFromEntityType(UDMXEntityFixtureType::StaticClass());
					}
					else if (FilterClass->IsChildOf(UDMXEntityFixturePatch::StaticClass()))
					{
						PinnedEditor->GetToolkitCommands()->ExecuteAction(FDMXEditorCommands::Get().AddNewEntityFixturePatch.ToSharedRef());

						PinnedEditor->InvokeEditorTabFromEntityType(UDMXEntityFixturePatch::StaticClass());
					}
				}

				OnCreateNewEntityDelegate.ExecuteIfBound();
			}
		}
		else if (InItem.IsValid() && SelectInfo != ESelectInfo::OnMouseClick)
		{
			int32 SelectedIdx = INDEX_NONE;
			if (FilteredEntityEntries.Find(InItem, /*out*/ SelectedIdx))
			{
				if (InItem->IsSeparator() || InItem->IsHeading())
				{
					int32 SelectionDirection = SelectedIdx - PrevSelectedIndex;

					// Update the previous selected index
					PrevSelectedIndex = SelectedIdx;

					// Make sure we select past the category heading if we started filtering with it selected somehow (avoiding the infinite loop selecting the same item forever)
					if (SelectionDirection == 0)
					{
						SelectionDirection = 1;
					}

					if (SelectedIdx + SelectionDirection >= 0 && SelectedIdx + SelectionDirection < FilteredEntityEntries.Num())
					{
						EntitiesListView->SetSelection(FilteredEntityEntries[SelectedIdx + SelectionDirection], ESelectInfo::OnNavigation);
					}
				}
				else
				{
					// Update the previous selected index
					PrevSelectedIndex = SelectedIdx;
				}
			}
		}
	}

	TSharedRef<ITableRow> GenerateEntryRow(TSharedPtr<FDMXEntityEntry> Entry, const TSharedRef<STableViewBase>& OwnerTable) const
	{
		if (Entry->IsHeading())
		{
			return 
				SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
				.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.NoHoverTableRow"))
				.ShowSelection(false)
				[
					SNew(SBox)
					.Padding(1.f)
					[
						SNew(STextBlock)
						.Text(Entry->GetLabelText())
						.TextStyle(FAppStyle::Get(), TEXT("Menu.Heading"))
					]
				];
		}
		else if (Entry->IsSeparator())
		{
			return 
				SNew(STableRow<TSharedPtr<FString>>, OwnerTable )
				.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.NoHoverTableRow"))
				.ShowSelection(false)
				[
					SNew(SBox)
					.Padding(1.f)
					[
						SNew(SBorder)
						.Padding(FAppStyle::GetMargin(TEXT("Menu.Separator.Padding")))
						.BorderImage(FAppStyle::GetBrush(TEXT("Menu.Separator")))
					]
				];
		}
		else if (Entry->IsEntity() || Entry->IsCreateNew())
		{
			static const FVector2D EntriesMargin(
				DMXEditor.Get().IsValid() ? 8.0f : 0.0f,
				1.0f);

			return
				SNew(SComboRow< TSharedPtr<FString> >, OwnerTable)
				.ToolTip(GetToolTipForEntry(Entry))
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SSpacer)
						.Size(EntriesMargin)
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.HighlightText(this, &SDMXDropdownMenuType::GetCurrentSearchString)
						.Text(Entry->GetLabelText())
					]
				];
		}

		return
			SNew(STableRow<TSharedPtr<FString>>, OwnerTable);
	}

	void OnSearchBoxTextChanged(const FText& InSearchText)
	{
		// filter the Entities list accordingly
		UpdateFilteredEntitiesList(InSearchText.ToString());
	}

	void OnSearchBoxTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
	{
		if (CommitInfo == ETextCommit::OnEnter)
		{
			TArray<TSharedPtr<FDMXEntityEntry>> SelectedItems = EntitiesListView->GetSelectedItems();
			if (SelectedItems.Num() > 0)
			{
				EntitiesListView->SetSelection(SelectedItems[0]);
			}
		}
	}

	FText GetCurrentSearchString() const
	{
		return SearchBox->GetText();
	}

	TSharedRef<SToolTip> GetToolTipForEntry(TSharedPtr<FDMXEntityEntry> Entry) const
	{
		if (Entry->IsCreateNew())
		{
			return SNew(SToolTip)
				.Text(FText::Format(LOCTEXT("AddEntityToolTip", "Create a new {0}"), EntityTypeName));
		}
		else if (Entry->IsEntity())
		{
			return SNew(SToolTip)
				.Text(FText::Format(
					LOCTEXT("UseEntityToolTip", "Existing {0}: {1}"),
					EntityTypeName,
					FText::FromString(Entry->GetEntity()->GetDisplayName())
				));
		}

		// Fallback for other entry types. Shouldn't get here
		return SNew(SToolTip)
			.Text(FText::GetEmpty());
	}

private:
	/** All entries that should be displayed in the menu, including headers and separators */
	TArray<TSharedPtr<FDMXEntityEntry>> EntityEntries;
	TArray<TSharedPtr<FDMXEntityEntry>> FilteredEntityEntries;

	/** The list widget, with the entries the user will see in the UI */
	TSharedPtr<SListView<TSharedPtr<FDMXEntityEntry>>> EntitiesListView;
	int32 PrevSelectedIndex;
	/** The search box to filter the available Entities */
	TSharedPtr<SSearchBox> SearchBox;

	FOnCreateNewEntitySelected OnCreateNewEntityDelegate;
	FOnEntitySelected OnEntitySelectedDelegate;

	TAttribute<TWeakObjectPtr<UDMXLibrary>> DMXLibrary;
	TAttribute<TSubclassOf<UDMXEntity>> EntityTypeFilter;
	TAttribute<TWeakPtr<FDMXEditor>> DMXEditor;
	uint32 CurrentExistingEntitiesNum;

	TWeakPtr<SComboButton> ComboButtonPtr;
	FText EntityTypeName;
};

//////////////////////////////////////////////////////////////////////////

/**
 * Generates a dropdown menu list of Entities from a specific type to be used as
 * a dropdown for a SComboButton's MenuContent.
 */
template<typename TEntityType>
class SDMXEntityPickerButton
	: public SCompoundWidget
{
	using SDMXEntityPickerType = SDMXEntityPickerButton<TEntityType>;

public:
	SLATE_BEGIN_ARGS(SDMXEntityPickerButton)
		: _CurrentEntity(nullptr)
		, _HasMultipleValues(false)
		, _DMXEditor(nullptr)
		, _DMXLibrary(nullptr)
		, _EntityTypeFilter(nullptr)
		, _ForegroundColor(FLinearColor::White)
		, _OnCreateNewEntitySelected()
		, _OnEntitySelected()
		{}

		/** The current selected Entity. Only used if EntityPropertyHandle is null */
		SLATE_ATTRIBUTE(TWeakObjectPtr<TEntityType>, CurrentEntity)

		/** Pass in true if the represented property has multiple Entity values */
		SLATE_ATTRIBUTE(bool, HasMultipleValues)

		/** Enables the option to create a new entity and contains the DMX Library reference */
		SLATE_ATTRIBUTE(TWeakPtr<FDMXEditor>, DMXEditor)
		/** Needed to find existing Entities. Can be inferred from DMXEditor, if that's valid. */
		SLATE_ATTRIBUTE(TWeakObjectPtr<UDMXLibrary>, DMXLibrary)

		/** If set, overrides the template type. Use UDMXEntity as template in this case. */
		SLATE_ATTRIBUTE(TSubclassOf<UDMXEntity>, EntityTypeFilter)

		/** Color/opacity multiplier for the combo button */
		SLATE_ATTRIBUTE(FLinearColor, ForegroundColor)
		
		/** Called when the user selects the option to create a new Entity */
		SLATE_EVENT(FOnCreateNewEntitySelected, OnCreateNewEntitySelected)
		/** Called when the user selects an existing Entity from the list or clicks on the Use Selected button */
		SLATE_EVENT(FOnEntitySelected, OnEntitySelected)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		OnCreateNewEntityDelegate = InArgs._OnCreateNewEntitySelected;
		OnEntitySelectedDelegate = InArgs._OnEntitySelected;

		CurrentEntity = InArgs._CurrentEntity;
		HasMultipleValues = InArgs._HasMultipleValues;
		EntityTypeFilter = InArgs._EntityTypeFilter;

		DMXEditor = InArgs._DMXEditor;
		DMXLibrary = InArgs._DMXLibrary;


		TSharedPtr<SComboButton> ComboButton;
		TSharedPtr<SDMXEntityDropdownMenu<TEntityType>> EntitiesDropdownList;
		TSharedPtr<SHorizontalBox> ButtonsContainer;

		ChildSlot
		[
			// The border allow us to have transparency on all of its widgets at once
			SNew(SBorder)
			.BorderBackgroundColor(FLinearColor(0, 0, 0, 0))
			.ColorAndOpacity(InArgs._ForegroundColor)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SAssignNew(ComboButton, SComboButton)
					.ButtonContent()
					[
						// Show the name of the asset or actor
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "PropertyEditor.AssetClass")
						.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
						.Text(this, &SDMXEntityPickerType::OnGetSelectedEntityName)
					]
					.MenuContent()
					[
						SAssignNew(EntitiesDropdownList, SDMXEntityDropdownMenu<TEntityType>)
						.DMXEditor(DMXEditor)
						.EntityTypeFilter(this, &SDMXEntityPickerType::GetEntityFilterClass)
						.DMXLibrary(InArgs._DMXLibrary)
						.OnEntitySelected(this, &SDMXEntityPickerType::OnEntitySelected)
						.OnCreateNewEntitySelected(InArgs._OnCreateNewEntitySelected)
					]
					.IsFocusable(true)
					.ToolTipText(this, &SDMXEntityPickerType::OnGetSelectedEntityName)
					.ContentPadding(2.0f)
					.ButtonStyle(FAppStyle::Get(), "PropertyEditor.AssetComboStyle")
					.ForegroundColor(FAppStyle::GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
					.OnComboBoxOpened(FOnComboBoxOpened::CreateLambda([EntitiesDropdownList]()
						{
							EntitiesDropdownList->ClearSelection();
							EntitiesDropdownList->RefreshEntitiesList();
						}))
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(5.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					// Wrapper to optionally hide the buttons when the editor is not available
					SNew(SBox)
					.Visibility(this, &SDMXEntityPickerType::GetEditorButtonsVisibility)
					[
						PropertyCustomizationHelpers::MakeUseSelectedButton(
							FSimpleDelegate::CreateSP(this, &SDMXEntityPickerType::OnUseSelectedEntityClicked),
							FText::Format(
								LOCTEXT("UseSelectedButtonLabel", "Use selected from {0} tab"),
								FDMXEditorUtils::GetEntityTypeNameText(GetEntityFilterClass(), true)
							))
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(5.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					// Wrapper to optionally hide the buttons when the editor is not available
					SNew(SBox)
					.Visibility(this, &SDMXEntityPickerType::GetEditorButtonsVisibility)
					[
						PropertyCustomizationHelpers::MakeBrowseButton(
							FSimpleDelegate::CreateSP(this, &SDMXEntityPickerType::OnBrowseEntityClicked),
							FText::Format(
								LOCTEXT("BrowseButtonLabel", "Browse to {0} in its tab"),
								FDMXEditorUtils::GetEntityTypeNameText(GetEntityFilterClass(), false)
							),
							TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &SDMXEntityPickerType::GetCanBrowseEntity))
						)
					]
				]
			]
		];

		EntitiesDropdownList->SetComboButton(ComboButton);
	}

protected:
	TSubclassOf<UDMXEntity> GetEntityFilterClass() const
	{
		TSubclassOf<UDMXEntity> FromFilter = EntityTypeFilter.Get();
		if (FromFilter != nullptr)
		{
			return FromFilter; 
		}

		return TEntityType::StaticClass();
	}

	EVisibility GetEditorButtonsVisibility() const
	{
		return DMXEditor.Get().IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	void OnBrowseEntityClicked() const
	{
		UDMXEntity* Entity = CurrentEntity.Get().Get();

		TSharedPtr<FDMXEditor> PinnedEditor = DMXEditor.Get().Pin();
		if (PinnedEditor.IsValid() && Entity != nullptr)
		{
			PinnedEditor->SelectEntityInItsTypeTab(Entity, ESelectInfo::OnMouseClick);
		}
	}

	void OnUseSelectedEntityClicked()
	{
		if (TSharedPtr<FDMXEditor> PinnedEditor = DMXEditor.Get().Pin())
		{
			const TArray<UDMXEntity*>&& SelectedEntities = PinnedEditor->GetSelectedEntitiesFromTypeTab(GetEntityFilterClass());

			// Only set property if there's just a single Entity selected
			if (SelectedEntities.Num() == 1)
			{
				OnEntitySelected(SelectedEntities[0]);
			}
		}
	}

	void OnEntitySelected(UDMXEntity* InEntity)
	{
		check(InEntity != nullptr);
		if (OnEntitySelectedDelegate.IsBound())
		{
			OnEntitySelectedDelegate.Execute(InEntity); 
		}
	}

	FText OnGetSelectedEntityName() const
	{
		if (HasMultipleValues.Get())
		{
			return LOCTEXT("MultipleValues_Label", "Multiple Values");
		}
		else if (const UDMXEntity* Entity = CurrentEntity.Get().Get())
		{
			return FText::FromString(Entity->GetDisplayName());
		}

		return FText::Format(
			LOCTEXT("NullReference_Label", "Select {0}"),
			FDMXEditorUtils::GetEntityTypeNameText(GetEntityFilterClass(), false)
		);
	}
	
	bool GetCanBrowseEntity() const
	{
		return !HasMultipleValues.Get() && CurrentEntity.Get().Get() != nullptr;
	}

private:
	FOnCreateNewEntitySelected OnCreateNewEntityDelegate;
	FOnEntitySelected OnEntitySelectedDelegate;

	TAttribute<TWeakPtr<FDMXEditor>> DMXEditor;
	TAttribute<TWeakObjectPtr<UDMXLibrary>> DMXLibrary;
	TAttribute<TSubclassOf<UDMXEntity>> EntityTypeFilter;
	TAttribute<TWeakObjectPtr<TEntityType>> CurrentEntity;
	TAttribute<bool> HasMultipleValues;
};

#undef LOCTEXT_NAMESPACE
