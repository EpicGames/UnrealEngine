// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "Types/SlateEnums.h"
#include "UObject/GCObject.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SHeaderRow.h"

#include "SDMXReadOnlyFixturePatchList.generated.h"

class FDMXReadOnlyFixturePatchListItem;
class ITableRow;
class SDMXReadOnlyFixturePatchListRow;
class SHeaderRow;
template <typename ItemType> class SListView;
class SSearchBox;
class STableViewBase;
class UDMXEntity;
class UDMXEntityFixturePatch;
class UDMXEntityFixtureType;
class UDMXLibrary;


enum class EDMXReadOnlyFixturePatchListShowMode : uint8
{
	/** Show All Fixture Patches in the list */
	All,

	/** Show Only Active Fixture Patches in the list */
	Active,

	/** Show Only Inactive Fixture Patches in the list */
	Inactive
};

/** Collumn IDs in the Fixture Patch List */
struct DMXEDITOR_API FDMXReadOnlyFixturePatchListCollumnIDs
{
	static const FName EditorColor;
	static const FName FixturePatchName;
	static const FName FixtureID;
	static const FName FixtureType;
	static const FName Mode;
	static const FName Patch;
};

/** Struct to describe a fixture patch list, so it can be stored in settings */
USTRUCT()
struct DMXEDITOR_API FDMXReadOnlyFixturePatchListDescriptor
{
	GENERATED_BODY()

	FDMXReadOnlyFixturePatchListDescriptor()
		: SortedByColumnID(FDMXReadOnlyFixturePatchListCollumnIDs::Patch)
		, ColumnIDToShowStateMap(TMap<FName, bool>())
	{
		ColumnIDToShowStateMap.Add(FDMXReadOnlyFixturePatchListCollumnIDs::EditorColor) = true;
		ColumnIDToShowStateMap.Add(FDMXReadOnlyFixturePatchListCollumnIDs::FixturePatchName) = true;
		ColumnIDToShowStateMap.Add(FDMXReadOnlyFixturePatchListCollumnIDs::FixtureID) = true;
		ColumnIDToShowStateMap.Add(FDMXReadOnlyFixturePatchListCollumnIDs::FixtureType) = false;
		ColumnIDToShowStateMap.Add(FDMXReadOnlyFixturePatchListCollumnIDs::Mode) = false;
		ColumnIDToShowStateMap.Add(FDMXReadOnlyFixturePatchListCollumnIDs::Patch) = true;
	}

	/** By which column ID the List is sorted */
	UPROPERTY()
	FName SortedByColumnID;

	/** Map for storing the show state of each column by name ID */
	UPROPERTY()
	TMap<FName, bool> ColumnIDToShowStateMap;
};

/** Generic list of Fixture Patches in a DMX library for read only purposes */
class DMXEDITOR_API SDMXReadOnlyFixturePatchList
	: public SCompoundWidget
	, public FGCObject
{
public:
	DECLARE_DELEGATE_OneParam(FDMXFixturePatchListRowDelegate, TSharedPtr<FDMXReadOnlyFixturePatchListItem>)
	DECLARE_DELEGATE_RetVal_OneParam(bool, FDMXFixturePatchListRowRetValDelegate, TSharedPtr<FDMXReadOnlyFixturePatchListItem>)
	DECLARE_DELEGATE_TwoParams(FDMXFixturePatchListRowSelectionDelegate, TSharedPtr<FDMXReadOnlyFixturePatchListItem>, ESelectInfo::Type)

	SLATE_BEGIN_ARGS(SDMXReadOnlyFixturePatchList)
		: _ListDescriptor(FDMXReadOnlyFixturePatchListDescriptor())
		, _DMXLibrary(nullptr)
	{}
		/** Descriptor of the list, useful for loading and storing the state of the list to disk */
		SLATE_ARGUMENT(FDMXReadOnlyFixturePatchListDescriptor, ListDescriptor)

		/** The displayed DMX Library */
		SLATE_ARGUMENT(UDMXLibrary*, DMXLibrary)

		/** Fixture patches not displayed in the list */
		SLATE_ARGUMENT(TArray<UDMXEntityFixturePatch*>, ExcludedFixturePatches)

		/** Called when a row of the list is right clicked */
		SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)
		
		/** Called when row selection has changed */
		SLATE_EVENT(FDMXFixturePatchListRowSelectionDelegate, OnRowSelectionChanged)

		/** Called when a row is clicked */
		SLATE_EVENT(FDMXFixturePatchListRowDelegate, OnRowClicked)

		/** Called when a row is double clicked */
		SLATE_EVENT(FDMXFixturePatchListRowDelegate, OnRowDoubleClicked)

		/** Called to get the enable state of each row of the list */
		SLATE_EVENT(FDMXFixturePatchListRowRetValDelegate, IsRowEnabled)

		/** Called to get the visibility state of each row of the list */
		SLATE_EVENT(FDMXFixturePatchListRowRetValDelegate, IsRowVisibile)

		/** Delegate executed when a row was dragged */
		SLATE_EVENT(FOnDragDetected, OnRowDragDetected)

	SLATE_END_ARGS()

	/** Constructs this widget */
	void Construct(const FArguments& InArgs);

	/** Updates the List on the next tick */
	void RequestRefresh();

	/** Gets all fixture patches in the DMX Library */
	TArray<UDMXEntityFixturePatch*> GetFixturePatchesInDMXLibrary() const;

	/** Gets all fixture patches displayed in the list. Note, the result does not contain excluded fixture patches. */
	TArray<UDMXEntityFixturePatch*> GetFixturePatchesInList() const;

	/** Gets current selected fixture patches in the list */
	TArray<UDMXEntityFixturePatch*> GetSelectedFixturePatches() const;

	/** Sets the displayed DMX library. Note, this does not refresh the list. Call RequestRefresh() to update the list. */
	void SetDMXLibrary(UDMXLibrary* InDMXLibrary);

	/** Sets the excluded fixture patches. Note, this does not refresh the list. Call RequestRefresh() to update the list. */
	void SetExcludedFixturePatches(const TArray<UDMXEntityFixturePatch*>& NewExcludedFixturePatches);

	/** Selects specified items in the list */
	void SelectItems(const TArray<TSharedPtr<FDMXReadOnlyFixturePatchListItem>>& ItemsToSelect, ESelectInfo::Type SelectInfo = ESelectInfo::Direct);

	/** Sets the selection state of the given item, if valid */
	void SetItemSelection(TSharedPtr<FDMXReadOnlyFixturePatchListItem> SelectedItem, bool bSelected, ESelectInfo::Type SelectInfo = ESelectInfo::Direct);

	/** Returns the selected items */
	TArray<TSharedPtr<FDMXReadOnlyFixturePatchListItem>> GetSelectedItems() const;

	/** Gets the a descriptor for the current parameters for this list */
	FDMXReadOnlyFixturePatchListDescriptor MakeListDescriptor() const;

	/** Gets the current displayed DMX library */
	UDMXLibrary* GetDMXLibrary() const { return WeakDMXLibrary.Get(); }

	/** Returns the items displayed int his list */
	TArray<TSharedPtr<FDMXReadOnlyFixturePatchListItem>> GetListItems() const { return ListItems; }

protected:
	//~ Begin FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;
	//~ End FGCObject interface

	/** Returns the menu name of the header row filter menu. Useful to override the default menu and extend it only for a specfic inherited class */
	virtual FName GetHeaderRowFilterMenuName() const; 

	/** Refreshes the Fixture Patch List */
	virtual void ForceRefresh();

	/** Called to generate the header row of the list */
	virtual TSharedRef<SHeaderRow> GenerateHeaderRow();

	/** Called to generate a row in the list */
	virtual TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FDMXReadOnlyFixturePatchListItem> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Toggles show state of the column with the given ID */
	virtual void ToggleColumnShowState(const FName ColumnID);

private:
	/** Generates a filter menu fot the Header Row of the List */
	TSharedRef<SWidget> GenerateHeaderRowFilterMenu();

	/** Applies settings of the list descriptor */
	void ApplyListDescriptor(const FDMXReadOnlyFixturePatchListDescriptor& InListDescriptor);

	/** Filters ListItems array by the given search text */
	TArray<TSharedPtr<FDMXReadOnlyFixturePatchListItem>> FilterListItems(const FText& SearchText);

	/** Sort the list by ColumnId */
	void SortByColumnID(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);

	/** Called when text from searchbox is changed */
	void OnSearchTextChanged(const FText& SearchText);

	/** Called when entities were added or removed from the DMX Library */
	void OnEntityAddedOrRemoved(UDMXLibrary* InDMXLibrary, TArray<UDMXEntity*> Entities);

	/** Called when a Fixture Patch changed */
	void OnFixturePatchChanged(const UDMXEntityFixturePatch* FixturePatch);

	/** Called when a Fixture Type changed */
	void OnFixtureTypeChanged(const UDMXEntityFixtureType* FixtureType);

	/** Gets show state of the column with the given ID */
	bool IsColumnShown(const FName ColumnID) const;

	/** Returns the column sort mode for the list*/
	EColumnSortMode::Type GetColumnSortMode(const FName ColumnID) const;

	/** The current Sort Mode */
	EColumnSortMode::Type SortMode = EColumnSortMode::Ascending;

	/** By which column ID the List is sorted */
	FName SortedByColumnID;

	/** Map for storing the show state of each column by name ID */
	TMap<FName, bool> ColumnIDToShowStateMap;

	/** Timer handle for the Request List Refresh method */
	FTimerHandle ListRefreshTimerHandle;

	/** The Header Row of the List */
	TSharedPtr<SSearchBox> SearchBox;

	/** The actual fixture patch list */
	TSharedPtr<SListView<TSharedPtr<FDMXReadOnlyFixturePatchListItem>>> ListView;

	/** Rows of Mode widgets in the List */
	TArray<TSharedRef<SDMXReadOnlyFixturePatchListRow>> ListRows;

	/** The list source, an array of read only fixture patch list items */
	TArray<TSharedPtr<FDMXReadOnlyFixturePatchListItem>> ListItems;

	/** Current displayed DMX Library */
	TWeakObjectPtr<UDMXLibrary> WeakDMXLibrary;

	// Slate Arguments
	TArray<TObjectPtr<UDMXEntityFixturePatch>> ExcludedFixturePatches;
	FDMXFixturePatchListRowRetValDelegate IsRowEnabledDelegate;
	FDMXFixturePatchListRowRetValDelegate IsRowVisibleDelegate;
	FOnDragDetected OnRowDragDetectedDelegate;
};
