// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SDMXReadOnlyFixturePatchList.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class FDMXReadOnlyFixturePatchListItem;


/** Entity Fixture Patch as a row in a list */
class DMXEDITOR_API SDMXReadOnlyFixturePatchListRow
	: public SMultiColumnTableRow<TSharedPtr<FDMXReadOnlyFixturePatchListItem>>
{
public:
	SLATE_BEGIN_ARGS(SDMXReadOnlyFixturePatchListRow)
	{}

		SLATE_EVENT(FOnDragDetected, OnRowDragDetected)

	SLATE_END_ARGS()
			
	/** Constructs this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, const TSharedPtr<FDMXReadOnlyFixturePatchListItem>& InItem);

protected:
	//~ Begin SMultiColumnTableRow interface
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
	//~ End SMultiColumnTableRow interface

private:
	/** Generates the row that displays the Fixture Patch Editor Color */
	TSharedRef<SWidget> GenerateEditorColorRow();

	/** Generates the row that displays the Fixture Patch Name */
	TSharedRef<SWidget> GenerateFixturePatchNameRow();

	/** Generates the row that displays the Fixture ID */
	TSharedRef<SWidget> GenerateFixtureIDRow();

	/** Generates the row that displays the Fixture Type */
	TSharedRef<SWidget> GenerateFixtureTypeRow();

	/** Generates the row that displays the Mode */
	TSharedRef<SWidget> GenerateModeRow();

	/** Generates the row that displays the Patch */
	TSharedRef<SWidget> GeneratePatchRow();

	/** The item this widget draws */
	TSharedPtr<FDMXReadOnlyFixturePatchListItem> Item;
};
