// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateWidgetStyleAsset.h"

class ITableRow;
class SComboButton;
class SEditableTextBox;
template <typename OptionType> class SListView;
template <typename OptionType> class SListViewSelectorDropdownMenu;
class SSearchBox;
class STableViewBase;


/**  A widget which allows the user to pick a name of a specified list of names. */
class DMXEDITOR_API SNameListPicker
	: public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnValueChanged, FName);

	static const FText NoneLabel;

	SLATE_BEGIN_ARGS(SNameListPicker)
		: _ComboButtonStyle(&FCoreStyle::Get().GetWidgetStyle< FComboButtonStyle >("ComboButton"))
		, _ButtonStyle(nullptr)
		, _bDisplayWarningIcon(false)
		, _bCanBeNone(false)
		, _ForegroundColor(FCoreStyle::Get().GetSlateColor("InvertedForeground"))
		, _ContentPadding(FMargin(2.f, 0.f))
		, _HasMultipleValues(false)
		, _Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
		, _MaxVisibleItems(15)
	{}

		/** The visual style of the combo button */
		SLATE_STYLE_ARGUMENT(FComboButtonStyle, ComboButtonStyle)

		/** The visual style of the button (overrides ComboButtonStyle) */
		SLATE_STYLE_ARGUMENT(FButtonStyle, ButtonStyle)

		/** List of possible names */
		SLATE_ATTRIBUTE(TArray<FName>, OptionsSource)

		/** Display warning icon for a selected invalid value? */
		SLATE_ARGUMENT(bool, bDisplayWarningIcon)

		/** Whether a <None> option can be displayed in the dropdown */
		SLATE_ARGUMENT(bool, bCanBeNone)
	
		/** Foreground color for the picker */
		SLATE_ATTRIBUTE(FSlateColor, ForegroundColor)
	
		/** Content padding for the picker */
		SLATE_ATTRIBUTE(FMargin, ContentPadding)
	
		/** Attribute used to retrieve the current value. */
		SLATE_ATTRIBUTE(FName, Value)
	
		/** Delegate for handling when for when the current value changes. */
		SLATE_EVENT(FOnValueChanged, OnValueChanged)
	
		/** Attribute used to retrieve whether the picker is representing multiple different values. */
		SLATE_ATTRIBUTE(bool, HasMultipleValues)

		/** Sets the font used to draw the text on the button */
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)

		/** If there're more than this, the list will be scrollable and a search box will be displayed */
		SLATE_ARGUMENT(int32, MaxVisibleItems)

	SLATE_END_ARGS()

	/**  Slate widget construction method */
	void Construct(const FArguments& InArgs);

private:
	EVisibility GetSearchBoxVisibility() const;
	void OnSearchBoxTextChanged(const FText& InSearchText);
	void OnSearchBoxTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo);
	void UpdateFilteredOptions(const FString& Filter);

	FText GetCurrentNameLabel() const;

	void UpdateOptionsSource();

	/** Create an option widget for the combo box */
	TSharedRef<ITableRow> GenerateNameItemWidget(TSharedPtr<FName> InItem, const TSharedRef<STableViewBase>& OwnerTable) const;
	/** Handles a selection change from the combo box */
	void OnSelectionChanged(const TSharedPtr<FName> Item, ESelectInfo::Type SelectInfo);
	void OnClickItem(const TSharedPtr<FName> Item);
	void OnUserSelectedItem(const TSharedPtr<FName> Item);

	TSharedPtr<FName> GetSelectedItemFromCurrentValue() const;

	/**
	 * Workaround to keep the correct option highlighted in the dropdown menu.
	 * When code changes the current value of the property this button represents, it's possible
	 * that the button will keep the previous value highlighted. So we set the currently highlighted
	 * option every time the menu is opened.
	 */
	void OnMenuOpened();

	/** Called when text was commited */
	void OnTextCommitted(const FText& InNewText, ETextCommit::Type InTextCommit);

private:
	TSharedPtr< SComboButton > PickerComboButton;
	TSharedPtr< SListView< TSharedPtr<FName> > > OptionsListView;
	TSharedPtr<SSearchBox> SearchBox;
	int32 MaxVisibleItems;
	TSharedPtr< SListViewSelectorDropdownMenu< TSharedPtr<FName> > > NamesListDropdown;
	TSharedPtr<SEditableTextBox> EdititableTextBox;

	TAttribute<TArray<FName>> OptionsSourceAttr;
	TArray<TSharedPtr<FName>> OptionsSource;
	TArray<TSharedPtr<FName>> FilteredOptions;

	TAttribute<FName> ValueAttribute;
	FOnValueChanged OnValueChangedDelegate;
	TAttribute<bool> HasMultipleValuesAttribute;
	bool bCanBeNone;
	bool bDisplayWarningIcon;
};
