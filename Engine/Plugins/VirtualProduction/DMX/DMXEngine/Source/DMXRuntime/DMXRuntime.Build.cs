// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DMXRuntime : ModuleRules
{
	public DMXRuntime(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
                "CoreUObject",
				"DatasmithContent",
				"DMXGDTF",
				"DMXProtocol",
				"Engine",
				"JsonUtilities",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetRegistry",
				"DMXFixtureActorInterface",
				"Json",
				"MovieScene",
				"Projects",
				"XmlParser",
			}
		);

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"UnrealEd",
				}
			);
		}
	}
}
