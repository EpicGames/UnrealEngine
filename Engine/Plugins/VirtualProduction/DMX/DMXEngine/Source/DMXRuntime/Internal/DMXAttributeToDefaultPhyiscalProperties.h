// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Editor only
#if WITH_EDITOR 

#include "GDTF/AttributeDefinitions/DMXGDTFPhysicalUnit.h"

struct FDMXFixtureFunction;

namespace UE::DMX
{
	struct DMXRUNTIME_API FDMXAttributeToDefaultPhyiscalProperties
	{
		/** Returns the value range for specified physical unit */
		static void ResetToDefaultPhysicalProperties(FDMXFixtureFunction& InOutFunction);

	private:
		struct FDefaultPhyicalProperties
		{
			FDefaultPhyicalProperties(EDMXGDTFPhysicalUnit InPhysicalUnit, double InPhysicalFrom, double InPhysicalTo)
				: PhysicalUnit(InPhysicalUnit)
				, PhysicalFrom(InPhysicalFrom)
				, PhysicalTo(InPhysicalTo)
			{}
				
			const EDMXGDTFPhysicalUnit PhysicalUnit = EDMXGDTFPhysicalUnit::None;
			const double PhysicalFrom = 0.0;
			const double PhysicalTo = 1.0;
		};

		/** A map to list all GDTF compliant Physical Units and their matching Physical Value default ranges. */
		static const TMap<FName, FDefaultPhyicalProperties> AttributeNameToDefaultPhysicalProperitesMap;
	};
}

#endif // WITH_EDITOR
