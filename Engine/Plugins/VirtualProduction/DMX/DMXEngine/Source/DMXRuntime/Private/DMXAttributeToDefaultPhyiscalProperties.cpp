// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR 

#include "DMXAttributeToDefaultPhyiscalProperties.h"

#include "Library/DMXEntityFixtureType.h"

namespace UE::DMX
{
	void FDMXAttributeToDefaultPhyiscalProperties::ResetToDefaultPhysicalProperties(FDMXFixtureFunction& InOutFunction)
	{
		const FDefaultPhyicalProperties* AttributeNameToDefaultPhysicalProperitesPtr = AttributeNameToDefaultPhysicalProperitesMap.Find(InOutFunction.Attribute.Name);
		if (AttributeNameToDefaultPhysicalProperitesPtr)
		{
			if (InOutFunction.GetPhysicalUnit() == AttributeNameToDefaultPhysicalProperitesPtr->PhysicalUnit)
			{
				return;
			}

			InOutFunction.SetPhysicalUnit(AttributeNameToDefaultPhysicalProperitesPtr->PhysicalUnit);
			InOutFunction.SetPhysicalValueRange(AttributeNameToDefaultPhysicalProperitesPtr->PhysicalFrom, AttributeNameToDefaultPhysicalProperitesPtr->PhysicalTo);
		}
	}

	const TMap<FName, FDMXAttributeToDefaultPhyiscalProperties::FDefaultPhyicalProperties> FDMXAttributeToDefaultPhyiscalProperties::AttributeNameToDefaultPhysicalProperitesMap
	{
		{ "Zoom", FDefaultPhyicalProperties(EDMXGDTFPhysicalUnit::Angle, 1.0, 120.0) },

		{ "Pan", FDefaultPhyicalProperties(EDMXGDTFPhysicalUnit::Angle, -120.0, 120.0) },
		{ "Tilt", FDefaultPhyicalProperties(EDMXGDTFPhysicalUnit::Angle, -120.0, 120.0) },

		{ "Angle", FDefaultPhyicalProperties(EDMXGDTFPhysicalUnit::Angle, 0.0, 120.0) },
	};
}

#endif // WITH_EDITOR
