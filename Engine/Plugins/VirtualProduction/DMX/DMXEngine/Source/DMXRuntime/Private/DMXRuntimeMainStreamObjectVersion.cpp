// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXRuntimeMainStreamObjectVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid FDMXRuntimeMainStreamObjectVersion::GUID(0x75A7B683, 0x4F4C7E28, 0xDDE54885, 0x85323939);

// Register the custom version with core
FCustomVersionRegistration GRegisterDMXRuntimeMainStreamCustomVersion(FDMXRuntimeMainStreamObjectVersion::GUID, FDMXRuntimeMainStreamObjectVersion::LatestVersion, TEXT("DMXRuntimeMainStreamObjectVersion"));
