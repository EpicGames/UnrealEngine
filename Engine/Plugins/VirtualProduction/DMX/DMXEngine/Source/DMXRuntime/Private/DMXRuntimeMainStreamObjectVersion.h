// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"

// Custom serialization version for changes to DMX Runtime Objects in the //Main Stream
struct FDMXRuntimeMainStreamObjectVersion
{
	enum Type
	{
		// Roughly corresponds to 4.26
		BeforeCustomVersionWasAdded = 0,

		// Update to DMX Library Section using normalized values by default
		DefaultToNormalizedValuesInDMXLibrarySection,

		// Update to DMX Library Section using normalized values by default
		ReplaceWeakWithStrongFixturePatchReferncesInLibrarySection,

		// Update DMX Entity Fixture Type to allow the Matrix to be enabled/disabled per Mode
		DMXFixtureTypeAllowMatrixInEachFixtureMode,

		// DMX Controllers are removed from the pugin
		DMXControllersAreRemoved,

		// Update Fixture Patch to use an MVR UUID
		DMXFixturePatchHasMVRUUID,

		// Remove auto assign features from fixture patch (instead it's auto assigned via FDMXEditorUtils where required)
		DMXFixturePatchNoLongerImplementsAutoAssign,

		// Change the DMXImportGDTF asset reference in Fixture Type to a soft object ptr, to avoid loading the data (since UE 5.5)
		DMXImportGDTFIsASoftObjectPtr,

		// Update Fixture Patch to hold the MVR Fixture ID (since UE 5.5)
		DMXFixturePatchHasFixtureID,

		// Rename GDTF import assets with invalid names
		DMXFixGDTFImportAssetsWithInvalidNames,

		// Upgrade Fixture Functions to make use of pysical properties where required (e.g. Pan, Tilt, Zoom)
		DMXUpgradeFixtureFunctionsToUsePhysicalProperties,

		// Upgrade Fixture Patches to have a Default Transform
		DMXFixturePatchesHaveDefaultTransform,

		// 5.5 Upgrade MVR Scene Actor to spawn Actors per Fixture Type of per GDTF
		DMXMVRSceneActorSpawnsActorsPerFixtureType,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	const static FGuid GUID;

private:
	FDMXRuntimeMainStreamObjectVersion() {}
};
