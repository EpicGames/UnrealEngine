// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXRuntimeUtils.h"

#include "Library/DMXLibrary.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Algo/Find.h"
#include "Algo/MaxElement.h"
#include "Internationalization/Regex.h"


#define LOCTEXT_NAMESPACE "FDMXRuntimeUtils"

FDMXOptionalTransform FDMXRuntimeUtils::ParseGDTFMatrix(const FString& String)
{
	auto GetVectorStringsFromMatrixStringLambda([](const FString& MatrixString) -> TArray<FString>
		{
			TArray<FString> Result;
			int32 SubstringIndex = 0;
			for (int32 SubStringNumber = 0; SubStringNumber < 4; SubStringNumber++)
			{
				SubstringIndex = MatrixString.Find(TEXT("{"), ESearchCase::IgnoreCase, ESearchDir::FromStart, SubstringIndex) + 1;
				const int32 EndIndex = MatrixString.Find(TEXT("}"), ESearchCase::IgnoreCase, ESearchDir::FromStart, SubstringIndex);

				const FString SubString = MatrixString.Mid(SubstringIndex, EndIndex - SubstringIndex);

				if (!SubString.IsEmpty())
				{
					Result.Add(SubString);
				}
			}

			return Result;
		});

	auto GetVectorFromVectorStringLambda([](const FString& VectorString) -> FVector
		{
			const int32 XIndex = 0;
			const int32 YIndex = VectorString.Find(TEXT(","), ESearchCase::IgnoreCase, ESearchDir::FromStart, 0) + 1;
			const int32 ZIndex = VectorString.Find(TEXT(","), ESearchCase::IgnoreCase, ESearchDir::FromStart, YIndex) + 1;

			const FString XString = "X=" + VectorString.Mid(XIndex, YIndex - XIndex);
			const FString YString = "Y=" + VectorString.Mid(YIndex, ZIndex - YIndex);
			const FString ZString = "Z=" + VectorString.Mid(ZIndex, VectorString.Len() - ZIndex);

			FVector Vector;
			if (Vector.InitFromString(XString + YString + ZString))
			{
				return Vector;
			}

			return FVector::ZeroVector;
		});

	FDMXOptionalTransform OptionalResult;
	const TArray<FString> SubStrings = GetVectorStringsFromMatrixStringLambda(String);
	if (SubStrings.Num() != 4)
	{
		return OptionalResult;
	}

	// From millimeters to centimeters
	const FVector U = GetVectorFromVectorStringLambda(SubStrings[0]) / 10.0;
	const FVector V = GetVectorFromVectorStringLambda(SubStrings[1]) / 10.0;
	const FVector W = GetVectorFromVectorStringLambda(SubStrings[2]) / 10.0;
	const FVector O = GetVectorFromVectorStringLambda(SubStrings[3]) / 10.0;

	FMatrix Matrix;
	Matrix.M[0][0] = U.X; 
	Matrix.M[0][1] = U.Y;
	Matrix.M[0][2] = U.Z;
	Matrix.M[0][3] = 0.0;

	Matrix.M[1][0] = V.X;
	Matrix.M[1][1] = V.Y;
	Matrix.M[1][2] = V.Z;
	Matrix.M[1][3] = 0.0;

	Matrix.M[2][0] = W.X;
	Matrix.M[2][1] = W.Y;
	Matrix.M[2][2] = W.Z;
	Matrix.M[2][3] = 0.0;

	Matrix.M[3][0] = O.X;
	Matrix.M[3][1] = -O.Y; // From GDTF's right hand to UE's left hand coordinate system
	Matrix.M[3][2] = O.Z;
	Matrix.M[3][3] = 1.0;

	FTransform Result(Matrix);

	// Mm to cm does not affect scale
	Result.SetScale3D(Result.GetScale3D() * 10.0);

	// GDTFs are facing down on the Z-Axis, but UE Actors are facing up 
	const FQuat InvertUpRotationQuaternion = FQuat(Result.GetRotation().GetAxisY(), PI);
	Result.SetRotation(Result.GetRotation() * InvertUpRotationQuaternion);

	OptionalResult = Result;
	return OptionalResult;
}

FString FDMXRuntimeUtils::ConvertTransformToGDTF4x3MatrixString(FTransform Transform)
{
	if (!ensureAlwaysMsgf(Transform.IsValid(), TEXT("Got invalid tranform when trying to generate a GDTF Matrix String from it.")))
	{
		Transform = FTransform::Identity;
	}

	// GDTFs are facing down on the Z-Axis, but UE Actors are facing up 
	const FQuat InvertUpRotationQuaternion = FQuat(Transform.GetRotation().GetAxisY(), PI);
	Transform.SetRotation(Transform.GetRotation() * InvertUpRotationQuaternion);

	FMatrix Matrix = Transform.ToMatrixWithScale();
	Matrix.M[3][1] = -Matrix.M[3][1]; // From UE's left hand to GDTF's right hand coordinate system

	FString Result;
	constexpr int32 FractionalDigits = 6; // As other soft- and hardwares

	Result += TEXT("{");
	Result += FString::SanitizeFloat(Matrix.M[0][0], FractionalDigits) + TEXT(",");
	Result += FString::SanitizeFloat(Matrix.M[0][1], FractionalDigits) + TEXT(",");
	Result += FString::SanitizeFloat(Matrix.M[0][2], FractionalDigits);
	Result += TEXT("}");

	Result += TEXT("{");
	Result += FString::SanitizeFloat(Matrix.M[1][0], FractionalDigits) + TEXT(",");
	Result += FString::SanitizeFloat(Matrix.M[1][1], FractionalDigits) + TEXT(",");
	Result += FString::SanitizeFloat(Matrix.M[1][2], FractionalDigits);
	Result += TEXT("}");

	Result += TEXT("{");
	Result += FString::SanitizeFloat(Matrix.M[2][0], FractionalDigits) + TEXT(",");
	Result += FString::SanitizeFloat(Matrix.M[2][1], FractionalDigits) + TEXT(",");
	Result += FString::SanitizeFloat(Matrix.M[2][2], FractionalDigits);
	Result += TEXT("}");

	// From centimeters to milimeters
	Result += TEXT("{");
	Result += FString::SanitizeFloat(Matrix.M[3][0] * 10.0, FractionalDigits) + TEXT(",");
	Result += FString::SanitizeFloat(Matrix.M[3][1] * 10.0, FractionalDigits) + TEXT(",");
	Result += FString::SanitizeFloat(Matrix.M[3][2] * 10.0, FractionalDigits);
	Result += TEXT("}");

	return Result;
}

FString FDMXRuntimeUtils::GenerateUniqueNameFromExisting(const TSet<FString>& InExistingNames, FString InDesiredName, uint8 MinNumDigitsInPostfix, bool bAlwaysGeneratePostfix)
{
	if (!bAlwaysGeneratePostfix && !InDesiredName.IsEmpty() && !InExistingNames.Contains(InDesiredName))
	{
		return InDesiredName;
	}

	if (!ensureAlwaysMsgf(!InDesiredName.IsEmpty(), TEXT("Empty name provided when generating unique name from exsiting. Using 'New'.")))
	{
		InDesiredName = TEXT("New");
	}

	// If the desired name is unique, just use that.
	if (Algo::Find(InExistingNames, InDesiredName) == nullptr)
	{
		return InDesiredName;
	}

	// Create a unique name
	FName UniqueName = [&InDesiredName]()
		{
			int32 Len = InDesiredName.Len();
			while (Len > 0 && FChar::IsDigit(InDesiredName[Len - 1]))
			{
				--Len;
			}

			int32 Number;
			if (!LexTryParseString(Number, *InDesiredName.RightChop(Len)))
			{
				return FName(*InDesiredName.Left(Len));
			}

			if (Len > 0 && InDesiredName[Len - 1] == TCHAR('_'))
			{
				--Len;
			}

			return FName(*InDesiredName.Left(Len), Number++);
		}();

	const auto IsUniqueLambda = [MinNumDigitsInPostfix](TArray<FString> Others, const FName& InName)
		{
			const FString NameString = InName.ToString();

			const FString& PlainNameString = InName.GetPlainNameString();
			const int32 Number = InName.GetNumber();
			const FString PossibleNameString = FString::Printf(TEXT("%s_%0*d"), *PlainNameString, MinNumDigitsInPostfix, Number);

			for (const FString& Other : Others)
			{
				if (NameString == Other ||
					PossibleNameString == Other)
				{
					return false;
				}
			}

			return true;
		};

	// Only test names that start with the plain name string to optimize for performance
	TArray<FString> ExistingNames;
	Algo::TransformIf(InExistingNames, ExistingNames,
		[UniqueNameString = UniqueName.GetPlainNameString()](const FString& ExistingName)
		{
			return ExistingName.StartsWith(UniqueNameString);
		},
		[](const FString& ExistingName)
		{
			return ExistingName;
		});

	// Sort in descending order to optimize for performance
	Algo::Sort(ExistingNames, [](const FString& NameA, const FString& NameB)
		{
			return NameA > NameB;
		});

	while (!IsUniqueLambda(ExistingNames, UniqueName))
	{
		UniqueName.SetNumber(UniqueName.GetNumber() + 1);
	}

	const int32 UniqueNumber = UniqueName.GetNumber();
	if (UniqueNumber == 0)
	{
		return UniqueName.ToString();
	}
	else
	{
		return FString::Printf(TEXT("%s_%0*d"), *UniqueName.GetPlainNameString(), MinNumDigitsInPostfix, UniqueName.GetNumber());
	}
}

FString FDMXRuntimeUtils::FindUniqueEntityName(const UDMXLibrary* InLibrary, TSubclassOf<UDMXEntity> InEntityClass, const FString& InBaseName /*= TEXT("")*/)
{
	check(InLibrary);

	// Get existing names for the current entity type
	TSet<FString> EntityNames;
	InLibrary->ForEachEntityOfType(InEntityClass, [&EntityNames](UDMXEntity* Entity)
		{
			EntityNames.Add(Entity->GetDisplayName());
		});

	FString BaseName = InBaseName;

	// If no base name was set, use the entity class name as base
	if (BaseName.IsEmpty())
	{
		BaseName = *InEntityClass->GetName();
	}

	constexpr uint8 MinNumDigitsInPostfix = 3;
	constexpr bool bAlwaysGeneratePostfix = true;
	return FDMXRuntimeUtils::GenerateUniqueNameFromExisting(EntityNames, BaseName, MinNumDigitsInPostfix, bAlwaysGeneratePostfix);
}

bool FDMXRuntimeUtils::GetNameAndIndexFromString(const FString& InString, FString& OutName, int32& OutIndex)
{
	OutName = InString.TrimEnd();

	// If there's an index at the end of the name, erase it
	int32 DigitIndex = OutName.Len();
	while (DigitIndex > 0 && OutName[DigitIndex - 1] >= '0' && OutName[DigitIndex - 1] <= '9')
	{
		--DigitIndex;
	}

	bool bHadIndex = false;
	if (DigitIndex < OutName.Len() && DigitIndex > -1)
	{
		OutIndex = FCString::Atoi(*OutName.RightChop(DigitIndex));
		OutName.LeftInline(DigitIndex);
		bHadIndex = true;
	}
	else
	{
		OutIndex = 0;
	}

	// Remove separator characters at the end of the string
	OutName.TrimEndInline();
	DigitIndex = OutName.Len(); // reuse this variable for the separator index

	while (DigitIndex > 0
		&& (OutName[DigitIndex  - 1] == '_'
		|| OutName[DigitIndex - 1] == '.'
		|| OutName[DigitIndex - 1] == '-'))
	{
		--DigitIndex;
	}

	if (DigitIndex < OutName.Len() && DigitIndex > -1)
	{
		OutName.LeftInline(DigitIndex);
	}

	return bHadIndex;
}

TMap<int32, TArray<UDMXEntityFixturePatch*>> FDMXRuntimeUtils::MapToUniverses(const TArray<UDMXEntityFixturePatch*>& AllPatches)
{
	TMap<int32, TArray<UDMXEntityFixturePatch*>> Result;
	for (UDMXEntityFixturePatch* Patch : AllPatches)
	{
		TArray<UDMXEntityFixturePatch*>& UniverseGroup = Result.FindOrAdd(Patch->GetUniverseID());
		UniverseGroup.Add(Patch);
	}
	return Result;
}

FString FDMXRuntimeUtils::GenerateUniqueNameForImportFunction(TMap<FString, uint32>& OutPotentialFunctionNamesAndCount, const FString& InBaseName)
{
	if (!InBaseName.IsEmpty() && !OutPotentialFunctionNamesAndCount.Contains(InBaseName))
	{
		return InBaseName;
	}

	FString BaseName;

	int32 Index = 0;
	if (InBaseName.IsEmpty())
	{
		BaseName = TEXT("Default name");
	}
	else
	{
		// If there's an index at the end of the name, start from there
		FDMXRuntimeUtils::GetNameAndIndexFromString(InBaseName, BaseName, Index);
	}

	FString FinalName = BaseName;

	// Generate a new Unique name and update the Unique counter
	if (uint32* CountPointer = OutPotentialFunctionNamesAndCount.Find(InBaseName))
	{
		// Calculate the number of digits in the number, adding 2 (1 extra to correctly count digits, another to account for the '_' that will be added to the name
		int32 CountLength = *CountPointer > 0 ? (int32)FGenericPlatformMath::LogX(10.0f, *CountPointer) + 2 : 2;

		// If the length of the final string will be too long, cut off the end so we can fit the number
		if (CountLength + BaseName.Len() >= NAME_SIZE)
		{
			BaseName = BaseName.Left(NAME_SIZE - CountLength - 1);
		}
		
		if (*CountPointer > 0)
		{
			FinalName = FString::Printf(TEXT("%s_%d"), *BaseName, *CountPointer);
		}
		else
		{
			FinalName = FString::Printf(TEXT("%s"), *BaseName);
		}

		*CountPointer = *CountPointer + 1;
	}

	return FinalName;
}

#undef LOCTEXT_NAMESPACE
