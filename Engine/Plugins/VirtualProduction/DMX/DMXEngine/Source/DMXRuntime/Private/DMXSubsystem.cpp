// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXSubsystem.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Async/Async.h"
#include "DMXAttribute.h"
#include "DMXConversions.h"
#include "DMXProtocolSettings.h"
#include "DMXProtocolTypes.h"
#include "DMXRuntimeUtils.h"
#include "Engine/Engine.h"
#include "Engine/ObjectLibrary.h"
#include "Engine/StreamableManager.h"
#include "EngineAnalytics.h"
#include "EngineUtils.h"
#include "Interfaces/IDMXProtocol.h"
#include "IO/DMXInputPort.h"
#include "IO/DMXOutputPort.h"
#include "IO/DMXPortManager.h"
#include "IO/DMXTrace.h"
#include "Library/DMXEntity.h"
#include "Library/DMXEntityController.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/DMXEntityReference.h"
#include "Library/DMXLibrary.h"
#include "UObject/UObjectIterator.h"

#if WITH_EDITOR
#include "AssetRegistry/AssetRegistryModule.h"
#endif // WITH_EDITOR

DECLARE_LOG_CATEGORY_CLASS(DMXSubsystemLog, Log, All);

void UDMXSubsystem::ClearDMXBuffers()
{
	// Clear port buffers
	FDMXPortManager::Get().ClearBuffers();

	// Rebuild fixture patch caches from cleared buffers, effectively clearing them as well.
	UDMXSubsystem* Subsystem = UDMXSubsystem::GetDMXSubsystem_Callable();
	if (Subsystem && Subsystem->IsValidLowLevel())
	{
		TArray<TSoftObjectPtr<UDMXLibrary>> DMXLibraries = Subsystem->GetDMXLibraries();
		for (const TSoftObjectPtr<UDMXLibrary>& Library : DMXLibraries)
		{
			if (Library.IsValid())
			{
				Library.Get()->ForEachEntityOfType<UDMXEntityFixturePatch>([](UDMXEntityFixturePatch* Patch) {
					Patch->RebuildCache();
					});
			}
		}
	}
}

void UDMXSubsystem::SendDMX(UDMXEntityFixturePatch* FixturePatch, TMap<FDMXAttributeName, int32> AttributeMap, EDMXSendResult& OutResult)
{
	OutResult = EDMXSendResult::Success;
	if (FixturePatch)
	{
		FixturePatch->SendDMX(AttributeMap);
	}
}

void UDMXSubsystem::SendDMXRaw(FDMXProtocolName SelectedProtocol, int32 RemoteUniverse, TMap<int32, uint8> ChannelToValueMap, EDMXSendResult& OutResult)
{
	// DEPRECATED 4.27
	
	for (const FDMXOutputPortSharedRef& OutputPort : FDMXPortManager::Get().GetOutputPorts())
	{
		const IDMXProtocolPtr& Protocol = OutputPort->GetProtocol();
		if (Protocol.IsValid() && Protocol->GetProtocolName() == SelectedProtocol)
		{
			// Using deprecated function in deprecated node to send to the remote universe.
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			OutputPort->SendDMXToRemoteUniverse(ChannelToValueMap, RemoteUniverse);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}

	OutResult = EDMXSendResult::Success;
}

void UDMXSubsystem::SendDMXToOutputPort(FDMXOutputPortReference OutputPortReference, TMap<int32, uint8> ChannelToValueMap, int32 LocalUniverse)
{
	const FGuid& PortGuid = OutputPortReference.GetPortGuid();
	const FDMXOutputPortSharedRef* OutputPortPtr = FDMXPortManager::Get().GetOutputPorts().FindByPredicate([PortGuid](const FDMXOutputPortSharedRef& OutputPort) {
		return OutputPort->GetPortGuid() == PortGuid;
	});

	if (OutputPortPtr)
	{
		UE_DMX_SCOPED_TRACE_SENDDMX("DMXSubsystem::SendDMXToOutputPort");
		(*OutputPortPtr)->SendDMX(LocalUniverse, ChannelToValueMap);
	}
	else
	{
		UE_LOGF(DMXSubsystemLog, Error, "Unexpected: Cannot find DMX Port, failed sending DMX with node Send DMX To Outputport.");
	}
}

void UDMXSubsystem::GetRawBuffer(FDMXProtocolName SelectedProtocol, int32 RemoteUniverse, TArray<uint8>& DMXBuffer)
{
	// DEPRECATED 4.27
	TMap<int32, uint8> ChannelToValueMap;
	
	for (const FDMXInputPortSharedRef& InputPort : FDMXPortManager::Get().GetInputPorts())
	{
		const IDMXProtocolPtr& Protocol = InputPort->GetProtocol();
		if (Protocol.IsValid() && Protocol->GetProtocolName() == SelectedProtocol)
		{
			FDMXSignalSharedPtr Signal;

			// Using deprecated function in deprecated node to get data from a remote universe.
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (InputPort->GameThreadGetDMXSignalFromRemoteUniverse(Signal, RemoteUniverse))
			{
				DMXBuffer = Signal->ChannelData;
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}

	for (const FDMXOutputPortSharedRef& OutputPort : FDMXPortManager::Get().GetOutputPorts())
	{
		const IDMXProtocolPtr& Protocol = OutputPort->GetProtocol();
		if (Protocol.IsValid() && Protocol->GetProtocolName() == SelectedProtocol)
		{
			FDMXSignalSharedPtr Signal;

			// Using deprecated function in deprecated node to get data from a remote universe.
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			constexpr bool bWhenLoopbackIsDisabled = false;
			if (OutputPort->GameThreadGetDMXSignalFromRemoteUniverse(Signal, RemoteUniverse, bWhenLoopbackIsDisabled))
			{
				DMXBuffer = Signal->ChannelData;
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
}

void UDMXSubsystem::GetDMXDataFromInputPort(FDMXInputPortReference InputPortReference, TArray<uint8>& DMXData, int32 LocalUniverse)
{
	const FGuid& PortGuid = InputPortReference.GetPortGuid();
	const FDMXInputPortSharedRef* InputPortPtr = FDMXPortManager::Get().GetInputPorts().FindByPredicate([PortGuid](const FDMXInputPortSharedRef& InputPort) {
		return InputPort->GetPortGuid() == PortGuid;
		});

	if (InputPortPtr)
	{
		const FDMXInputPortSharedRef& InputPort = *InputPortPtr;

		FDMXSignalSharedPtr Signal;
		if (InputPort->GameThreadGetDMXSignal(LocalUniverse, Signal))
		{
			DMXData = Signal->ChannelData;
		}
	}
	else
	{
		UE_LOGF(DMXSubsystemLog, Error, "Unexpected: Cannot find DMX Port, failed reading DMX in node 'Get DMX Data from Input Port'.");
	}
}

void UDMXSubsystem::GetDMXDataFromOutputPort(FDMXOutputPortReference OutputPortReference, TArray<uint8>& DMXData, int32 LocalUniverse)
{
	const FGuid& PortGuid = OutputPortReference.GetPortGuid();
	const FDMXOutputPortSharedRef* OutputPortPtr = FDMXPortManager::Get().GetOutputPorts().FindByPredicate([PortGuid](const FDMXOutputPortSharedRef& OutputPort) {
		return OutputPort->GetPortGuid() == PortGuid;
		});

	if (OutputPortPtr)
	{
		const FDMXOutputPortSharedRef& OutputPort = *OutputPortPtr;

		FDMXSignalSharedPtr Signal;
		constexpr bool bWhenLoopbackIsDisabled = false;
		if (OutputPort->GameThreadGetDMXSignal(LocalUniverse, Signal, bWhenLoopbackIsDisabled))
		{
			DMXData = Signal->ChannelData;
		}
	}
	else
	{
		UE_LOGF(DMXSubsystemLog, Error, "Unexpected: Cannot find DMX Port, failed reading DMX in node 'Get DMX Data from Input Port'.");
	}
}

bool UDMXSubsystem::SetMatrixCellValue(UDMXEntityFixturePatch* FixturePatch, FIntPoint Cell, FDMXAttributeName Attribute, int32 Value)
{
	if (FixturePatch)
	{
		return FixturePatch->SendMatrixCellValue(Cell, Attribute, Value);
	}

	return false;
}

bool UDMXSubsystem::GetMatrixCellValue(UDMXEntityFixturePatch* FixturePatch, FIntPoint Cells /* Cell X/Y */, TMap<FDMXAttributeName, int32>& AttributeValueMap)
{
	if (FixturePatch)
	{
		return FixturePatch->GetMatrixCellValues(Cells, AttributeValueMap);
	}

	return false;
}

bool UDMXSubsystem::GetMatrixCellChannelsRelative(UDMXEntityFixturePatch* FixturePatch, FIntPoint CellCoordinates /* Cell X/Y */, TMap<FDMXAttributeName, int32>& AttributeChannelMap)
{
	if (FixturePatch)
	{
		return FixturePatch->GetMatrixCellChannelsRelative(CellCoordinates, AttributeChannelMap);
	}

	return false;
}

bool UDMXSubsystem::GetMatrixCellChannelsAbsolute(UDMXEntityFixturePatch* FixturePatch, FIntPoint CellCoordinate /* Cell X/Y */, TMap<FDMXAttributeName, int32>& AttributeChannelMap)
{
	if (FixturePatch)
	{
		return FixturePatch->GetMatrixCellChannelsAbsolute(CellCoordinate, AttributeChannelMap);
	}

	return false;
}

bool UDMXSubsystem::GetMatrixProperties(UDMXEntityFixturePatch* FixturePatch, FDMXFixtureMatrix& MatrixProperties)
{
	if (FixturePatch)
	{
		return FixturePatch->GetMatrixProperties(MatrixProperties);
	}

	return false;
}

bool UDMXSubsystem::GetCellAttributes(UDMXEntityFixturePatch* FixturePatch, TArray<FDMXAttributeName>& CellAttributeNames)
{
	if (FixturePatch)
	{
		return FixturePatch->GetCellAttributes(CellAttributeNames);
	}

	return false;
}

bool UDMXSubsystem::GetMatrixCell(UDMXEntityFixturePatch* FixturePatch, FIntPoint Coordinate, FDMXCell& OutCell)
{
	if (FixturePatch)
	{
		return FixturePatch->GetMatrixCell(Coordinate, OutCell);
	}

	return false;
}

bool UDMXSubsystem::GetAllMatrixCells(UDMXEntityFixturePatch* FixturePatch, TArray<FDMXCell>& Cells)
{
	if (FixturePatch)
	{
		return FixturePatch->GetAllMatrixCells(Cells);
	}

	return false;
}

void UDMXSubsystem::PixelMappingDistributionSort(EDMXPixelMappingDistribution InDistribution, int32 InNumXPanels, int32 InNumYPanels, const TArray<int32>& InUnorderedList, TArray<int32>& OutSortedList)
{
	FDMXRuntimeUtils::PixelMappingDistributionSort(InDistribution, InNumXPanels, InNumYPanels, InUnorderedList, OutSortedList);
}

void UDMXSubsystem::GetAllFixturesOfType(const FDMXEntityFixtureTypeRef& FixtureType, TArray<UDMXEntityFixturePatch*>& OutResult)
{
	OutResult.Reset();

	if (const UDMXEntityFixtureType* FixtureTypeObj = FixtureType.GetFixtureType())
	{
		FixtureTypeObj->GetParentLibrary()->ForEachEntityOfType<UDMXEntityFixturePatch>([&](UDMXEntityFixturePatch* Fixture)
		{
			if (Fixture->GetFixtureType() == FixtureTypeObj)
			{
				OutResult.Add(Fixture);
			}
		});
	}
}

void UDMXSubsystem::GetAllFixturesOfCategory(const UDMXLibrary* DMXLibrary, FDMXFixtureCategory Category, TArray<UDMXEntityFixturePatch*>& OutResult)
{
	OutResult.Reset();

	if (DMXLibrary != nullptr)
	{
		DMXLibrary->ForEachEntityOfType<UDMXEntityFixturePatch>([&](UDMXEntityFixturePatch* FixturePatch)
		{
			if (FixturePatch->GetFixtureType() && FixturePatch->GetFixtureType()->DMXCategory == Category)
			{
				OutResult.Add(FixturePatch);
			}
		});
	}
}

void UDMXSubsystem::GetAllFixturesInUniverse(const UDMXLibrary* DMXLibrary, int32 UniverseId, TArray<UDMXEntityFixturePatch*>& OutResult)
{
	OutResult.Reset();

	if (DMXLibrary != nullptr)
	{
		DMXLibrary->ForEachEntityOfType<UDMXEntityFixturePatch>([&](UDMXEntityFixturePatch* Fixture)
		{
			if (Fixture->GetUniverseID() == UniverseId)
			{
				OutResult.Add(Fixture);
			}
		});
	}
}

void UDMXSubsystem::GetFixtureAttributes(const UDMXEntityFixturePatch* InFixturePatch, const TArray<uint8>& DMXBuffer, TMap<FDMXAttributeName, int32>& OutResult)
{
	OutResult.Reset();

	if (InFixturePatch != nullptr)
	{
		if (const UDMXEntityFixtureType* FixtureType = InFixturePatch->GetFixtureType())
		{
			const int32 StartingAddress = InFixturePatch->GetStartingChannel() - 1;

			const FDMXFixtureMode* ActiveModePtr = InFixturePatch->GetActiveMode();
			if (ActiveModePtr)
			{
				for (const FDMXFixtureFunction& Function : ActiveModePtr->Functions)
				{
					if (Function.GetLastChannel() <= InFixturePatch->GetChannelSpan())
					{
						// This function and the following ones are outside the Universe's range.
						break;
					}

					const int32 ChannelIndex = Function.Channel - 1 + StartingAddress;
					if (ChannelIndex >= DMXBuffer.Num())
					{
						continue;
					}
					const uint32 ChannelVal = UDMXEntityFixtureType::BytesToFunctionValue(Function, DMXBuffer.GetData() + ChannelIndex);

					OutResult.Add(Function.Attribute, ChannelVal);
				}
			}
			else
			{
				UE_LOGF(DMXSubsystemLog, Error, "Tried to use Fixture Patch %ls, but its Fixture Type has no Modes set up.", *InFixturePatch->Name);
				return;
			}
		}
	}
}

UDMXEntityFixtureType* UDMXSubsystem::GetFixtureType(FDMXEntityFixtureTypeRef InFixtureType)
{
	return InFixtureType.GetFixtureType();
}

UDMXEntityFixturePatch* UDMXSubsystem::GetFixturePatch(FDMXEntityFixturePatchRef InFixturePatch)
{
	return InFixturePatch.GetFixturePatch();
}

bool UDMXSubsystem::GetFunctionsMap(UDMXEntityFixturePatch* InFixturePatch, TMap<FDMXAttributeName, int32>& OutAttributesMap)
{
	if (InFixturePatch)
	{
		InFixturePatch->GetAttributeValues(OutAttributesMap);
		return true;
	}
	
	return false;
}

bool UDMXSubsystem::GetFunctionsMapForPatch(UDMXEntityFixturePatch* InFixturePatch, TMap<FDMXAttributeName, int32>& OutAttributesMap)
{
	// DEPRECATED 5.5, duplicate of GetFunctionsMap
	return GetFunctionsMap(InFixturePatch, OutAttributesMap);
}

int32 UDMXSubsystem::GetFunctionsValue(const FName FunctionAttributeName, const TMap<FDMXAttributeName, int32>& InAttributesMap)
{
	for (const TPair<FDMXAttributeName, int32>& kvp : InAttributesMap)
	{
		if(kvp.Key.Name.IsEqual(FunctionAttributeName))
		{
			const int32* Result = InAttributesMap.Find(kvp.Key);
			if (Result != nullptr)
			{
				return *Result;
			}
		}
	}	

	return 0;
}

bool UDMXSubsystem::PatchIsOfSelectedType(UDMXEntityFixturePatch* InFixturePatch, FString RefTypeValue)
{
	FDMXEntityFixtureTypeRef FixtureTypeRef;

	FDMXEntityReference::StaticStruct()
		->ImportText(*RefTypeValue, &FixtureTypeRef, nullptr, EPropertyPortFlags::PPF_None, GLog, FDMXEntityReference::StaticStruct()->GetName());

	if (FixtureTypeRef.DMXLibrary != nullptr)
	{
		UDMXEntityFixtureType* FixtureType = FixtureTypeRef.GetFixtureType();

		TArray<UDMXEntityFixturePatch*> AllPatchesOfType;
		GetAllFixturesOfType(FixtureType, AllPatchesOfType);

		if (AllPatchesOfType.Contains(InFixturePatch))
		{
			return true;
		}
	}

	return false;
}

FName UDMXSubsystem::GetAttributeLabel(FDMXAttributeName AttributeName)
{
	return AttributeName.Name;
}

/*static*/ UDMXSubsystem* UDMXSubsystem::GetDMXSubsystem_Pure()
{
	check(GEngine);
	return GEngine->GetEngineSubsystem<UDMXSubsystem>();
}

/*static*/ UDMXSubsystem* UDMXSubsystem::GetDMXSubsystem_Callable()
{
	return UDMXSubsystem::GetDMXSubsystem_Pure();
}

TArray<UDMXEntityFixturePatch*> UDMXSubsystem::GetAllFixturesWithTag(const UDMXLibrary* DMXLibrary, FName CustomTag)
{
	TArray<UDMXEntityFixturePatch*> FoundPatches;

	if (DMXLibrary != nullptr)
	{
		DMXLibrary->ForEachEntityOfType<UDMXEntityFixturePatch>([&](UDMXEntityFixturePatch* Patch)
		{
			if (Patch->GetCustomTags().Contains(CustomTag))
			{
				FoundPatches.Add(Patch);
			}
		});
	}

	return FoundPatches;
}

TArray<UDMXEntityFixturePatch*> UDMXSubsystem::GetAllFixturesInLibrary(const UDMXLibrary* DMXLibrary)
{
	TArray<UDMXEntityFixturePatch*> FoundPatches;
	
	if (DMXLibrary != nullptr)
	{
		DMXLibrary->ForEachEntityOfType<UDMXEntityFixturePatch>([&](UDMXEntityFixturePatch* Patch)
		{
			FoundPatches.Add(Patch);
		});
	}

	// Sort patches by universes and channels
	FoundPatches.Sort([](const UDMXEntityFixturePatch& FixturePatchA, const UDMXEntityFixturePatch& FixturePatchB) {

		if (FixturePatchA.GetUniverseID() < FixturePatchB.GetUniverseID())
		{
			return true;
		}

		bool bSameUniverse = FixturePatchA.GetUniverseID() == FixturePatchB.GetUniverseID();
		if (bSameUniverse)
		{
			return FixturePatchA.GetStartingChannel() <= FixturePatchB.GetStartingChannel();
		}
	
		return false;
	});

	return FoundPatches;
}

template<class TEntityType>
TEntityType* GetDMXEntityByName(const UDMXLibrary* DMXLibrary, const FString& Name)
{
	if (DMXLibrary != nullptr)
	{
		TEntityType* FoundEntity = nullptr;
		DMXLibrary->ForEachEntityOfTypeWithBreak<TEntityType>([&](TEntityType* Entity)
		{
			if (Entity->Name.Equals(Name))
			{
				FoundEntity = Entity;
				return false;
			}
			return true;
		});

		return FoundEntity;
	}

	return nullptr;
}

UDMXEntityFixturePatch* UDMXSubsystem::GetFixtureByName(const UDMXLibrary* DMXLibrary, const FString& Name)
{
	return GetDMXEntityByName<UDMXEntityFixturePatch>(DMXLibrary, Name);
}

TArray<UDMXEntityFixtureType*> UDMXSubsystem::GetAllFixtureTypesInLibrary(const UDMXLibrary* DMXLibrary)
{
	TArray<UDMXEntityFixtureType*> FoundTypes;

	if (DMXLibrary != nullptr)
	{
		DMXLibrary->ForEachEntityOfType<UDMXEntityFixtureType>([&](UDMXEntityFixtureType* Type)
		{
			FoundTypes.Add(Type);
		});
	}

	return FoundTypes;
}

UDMXEntityFixtureType* UDMXSubsystem::GetFixtureTypeByName(const UDMXLibrary* DMXLibrary, const FString& Name)
{
	return GetDMXEntityByName<UDMXEntityFixtureType>(DMXLibrary, Name);
}

TArray<UDMXEntityController*> UDMXSubsystem::GetAllControllersInLibrary(const UDMXLibrary* DMXLibrary)
{
	// DEPRECATED 4.27, controllers are no longer in use
	TArray<UDMXEntityController*> EmptyArray;
	return EmptyArray;
}

void UDMXSubsystem::GetAllUniversesInController(const UDMXLibrary* DMXLibrary, FString ControllerName, TArray<int32>& OutResult)
{
	// DEPRECATED 4.27, controllers are no longer in use
	OutResult.Reset();
}

UDMXEntityController* UDMXSubsystem::GetControllerByName(const UDMXLibrary* DMXLibrary, const FString& Name)
{
	// DEPRECATED 4.27, controllers are no longer in use
	return nullptr;
}

TArray<UDMXLibrary*> UDMXSubsystem::GetAllDMXLibraries()
{
	// DEPRECATED 5.5
	return LoadDMXLibrariesSynchronous();
}

TArray<UDMXLibrary*> UDMXSubsystem::LoadDMXLibrariesSynchronous() const
{
	TArray<TSoftObjectPtr<UDMXLibrary>> SoftDMXLibraries = GetDMXLibraries();
	TArray<UDMXLibrary*> DMXLibraries;
	Algo::TransformIf(SoftDMXLibraries, DMXLibraries,
		[](const TSoftObjectPtr<UDMXLibrary>& DMXLibrary)
		{
			return !DMXLibrary.IsNull();
		},
		[](const TSoftObjectPtr<UDMXLibrary>& DMXLibrary)
		{
			return DMXLibrary.LoadSynchronous();
		});

	return DMXLibraries;
}

TArray<TSoftObjectPtr<UDMXLibrary>> UDMXSubsystem::GetDMXLibraries() const
{
	TArray<FAssetData> AssetDataArray;
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().GetAssetsByClass(UDMXLibrary::StaticClass()->GetClassPathName(), AssetDataArray);

	TArray<TSoftObjectPtr<UDMXLibrary>> DMXLibraries;
	Algo::Transform(AssetDataArray, DMXLibraries,
		[](const FAssetData& AssetData)
		{
			return TSoftObjectPtr<UDMXLibrary>(AssetData.GetSoftObjectPath());
		});

	return DMXLibraries;
}

EDMXFixtureSignalFormat SignalFormatFromBytesNum(uint32 InBytesNum)
{
	switch (InBytesNum)
	{
	case 0:
		UE_LOGF(DMXSubsystemLog, Error, "%s called with InBytesNum = 0", __FUNCTION__);
		return EDMXFixtureSignalFormat::E8Bit;
	case 1:
		return EDMXFixtureSignalFormat::E8Bit;
	case 2:
		return EDMXFixtureSignalFormat::E16Bit;
	case 3:
		return EDMXFixtureSignalFormat::E24Bit;
	case 4:
		return EDMXFixtureSignalFormat::E32Bit;
	default: // InBytesNum is 4 or higher
		UE_LOGF(DMXSubsystemLog, Warning, "%s called with InBytesNum > 4. Only 4 bytes will be used.", __FUNCTION__);
		return EDMXFixtureSignalFormat::E32Bit;
	}
}

int32 UDMXSubsystem::BytesToInt(const TArray<uint8>& Bytes, bool bUseLSB /*= false*/) const
{
	if (Bytes.Num() == 0)
	{
		return 0;
	}

	const EDMXFixtureSignalFormat SignalFormat = SignalFormatFromBytesNum(Bytes.Num());
	return UDMXEntityFixtureType::BytesToInt(SignalFormat, bUseLSB, Bytes.GetData());
}

float UDMXSubsystem::BytesToNormalizedValue(const TArray<uint8>& Bytes, bool bUseLSB /*= false*/) const
{
	if (Bytes.Num() == 0)
	{
		return 0;
	}

	const EDMXFixtureSignalFormat SignalFormat = SignalFormatFromBytesNum(Bytes.Num());
	return UDMXEntityFixtureType::BytesToNormalizedValue(SignalFormat, bUseLSB, Bytes.GetData());
}

void UDMXSubsystem::NormalizedValueToBytes(float InValue, EDMXFixtureSignalFormat InSignalFormat, TArray<uint8>& Bytes, bool bUseLSB /*= false*/) const
{
	const uint8 NumBytes = FDMXConversions::GetSizeOfSignalFormat(InSignalFormat);
	// Make sure the array will fit the correct number of bytes
	Bytes.Reset(NumBytes);
	Bytes.AddZeroed(NumBytes);

	UDMXEntityFixtureType::NormalizedValueToBytes(InSignalFormat, bUseLSB, InValue, Bytes.GetData());
}

void UDMXSubsystem::IntValueToBytes(int32 InValue, EDMXFixtureSignalFormat InSignalFormat, TArray<uint8>& Bytes, bool bUseLSB /*= false*/)
{
	const uint8 NumBytes = FDMXConversions::GetSizeOfSignalFormat(InSignalFormat);
	// Make sure the array will fit the correct number of bytes
	Bytes.Reset(NumBytes);
	Bytes.AddZeroed(NumBytes);

	UDMXEntityFixtureType::IntToBytes(InSignalFormat, bUseLSB, InValue, Bytes.GetData());
}

float UDMXSubsystem::IntToNormalizedValue(int32 InValue, EDMXFixtureSignalFormat InSignalFormat) const
{
	return (float)(uint32)(InValue) / FDMXConversions::GetSignalFormatMaxValue(InSignalFormat);
}

float UDMXSubsystem::GetNormalizedAttributeValue(UDMXEntityFixturePatch* InFixturePatch, FDMXAttributeName InFunctionAttribute, int32 InValue) const
{
	if (InFixturePatch)
	{
		const FDMXFixtureMode* ActiveModePtr = InFixturePatch->GetActiveMode();
		if (ActiveModePtr)
		{
			// Search for a Function named InFunctionName in the Fixture Type current mode
			for (const FDMXFixtureFunction& Function : ActiveModePtr->Functions)
			{
				if (Function.Attribute == InFunctionAttribute)
				{
					return IntToNormalizedValue(InValue, Function.DataType);
				}
			}
		}
		else
		{
			UE_LOGF(DMXSubsystemLog, Error, "%s: Cannot access the Mode of Fixture Patch %ls. Either it is of fixture type none, or the fixture type has no mode.", __FUNCTION__, *InFixturePatch->Name);
		}
	}
	else
	{
		UE_LOGF(DMXSubsystemLog, Error, "%s: InFixturePatch is invalid.", __FUNCTION__);
	}

	return -1.0f;
}
