// Copyright Epic Games, Inc. All Rights Reserved.

#include "Game/DMXComponent.h"

#include "DMXProtocolSettings.h"
#include "DMXRuntimeLog.h"
#include "DMXStats.h"
#include "Library/DMXEntityFixturePatch.h"


UDMXComponent::UDMXComponent()
	: bReceiveDMXFromPatch(true)
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = true;
	PrimaryComponentTick.TickGroup = ETickingGroup::TG_PrePhysics;
	bTickInEditor = true;

#if WITH_EDITOR
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		UDMXProtocolSettings* DMXProtocolSettings = GetMutableDefault<UDMXProtocolSettings>();
		DMXProtocolSettings->GetOnAllFixturePatchesReceiveDMXInEditorEnabled().AddUObject(this, &UDMXComponent::OnAllFixturePatchesReceiveDMXInEditorEnabled);
		UDMXEntityFixturePatch::GetOnFixturePatchChanged().AddUObject(this, &UDMXComponent::OnFixturePatchPropertiesChanged);
	}
#endif
}

UDMXEntityFixturePatch* UDMXComponent::GetFixturePatch() const
{
	return FixturePatchRef.GetFixturePatch();
}

void UDMXComponent::SetFixturePatch(UDMXEntityFixturePatch* InFixturePatch)
{
	UDMXEntityFixturePatch* PreviousFixturePatch = FixturePatchRef.GetFixturePatch();

	if (InFixturePatch != PreviousFixturePatch)
	{
		// Remove the old receive binding
		if (IsValid(PreviousFixturePatch))
		{
			PreviousFixturePatch->OnFixturePatchReceivedDMXNative.RemoveAll(this);
		}

		// Set the new patch and setup the new binding
		FixturePatchRef.SetEntity(InFixturePatch);
		SetupReceiveDMXBinding();
		UpdateTickEnabled();
	}
}

void UDMXComponent::SetReceiveDMXFromPatch(bool bReceive)
{
	bReceiveDMXFromPatch = bReceive;

	SetupReceiveDMXBinding();
	UpdateTickEnabled();
}

void UDMXComponent::OnFixturePatchReceivedDMX(UDMXEntityFixturePatch* FixturePatch, const FDMXNormalizedAttributeValueMap& NormalizedValuePerAttribute)
{
	// Hotfix 5.1 - Old bindings are not properly removed when duplicating Actors that use this component @todo.
	if (FixturePatch != GetFixturePatch())
	{
		FixturePatch->OnFixturePatchReceivedDMXNative.RemoveAll(this);
		return;
	}

	if (OnFixturePatchReceived.IsBound())
	{
		FEditorScriptExecutionGuard ScriptGuard;
		OnFixturePatchReceived.Broadcast(FixturePatch, NormalizedValuePerAttribute);
	}
}

void UDMXComponent::SetupReceiveDMXBinding()
{
	UDMXEntityFixturePatch* FixturePatch = FixturePatchRef.GetFixturePatch();
	if (IsValid(FixturePatch))
	{
		const bool bReceiveDMX = [this, FixturePatch]()
		{
#if WITH_EDITOR
			if (bReceiveDMXFromPatch && GIsEditor && !GIsPlayInEditorWorld)
			{
				const UDMXProtocolSettings* ProtocolSettings = GetDefault<UDMXProtocolSettings>();
				const bool bFixturePatchReceivesDMXInEditor = FixturePatch->bReceiveDMXInEditor;
				const bool bAllFixturePatchesReceiveDMXInEditor = ProtocolSettings->ShouldAllFixturePatchesReceiveDMXInEditor();

				return bFixturePatchReceivesDMXInEditor || bAllFixturePatchesReceiveDMXInEditor;
			}
#endif
			return bReceiveDMXFromPatch;
		}();
		
		if (bReceiveDMX && !FixturePatch->OnFixturePatchReceivedDMXNative.IsBoundToObject(this))
		{
			// Enable receive DMX
			FixturePatch->OnFixturePatchReceivedDMXNative.AddUObject(this, &UDMXComponent::OnFixturePatchReceivedDMX);

			FDMXNormalizedAttributeValueMap NormalizeAttributeValues;
			FixturePatch->GetNormalizedAttributeValues(NormalizeAttributeValues);

			if (NormalizeAttributeValues.Map.Num() > 0)
			{
				OnFixturePatchReceived.Broadcast(FixturePatch, NormalizeAttributeValues);
			}
		}
		else if (!bReceiveDMX)
		{			
			// Disable receive DMX
			FixturePatch->OnFixturePatchReceivedDMXNative.RemoveAll(this);
		}
	}
}

void UDMXComponent::UpdateTickEnabled()
{
#if WITH_EDITOR
	FEditorScriptExecutionGuard ScriptGuard;
#endif

	if (IsBeingDestroyed() ||
		!OnDMXComponentTick.IsBound() ||
		!bReceiveDMXFromPatch)
	{
		SetComponentTickEnabled(false);
		return;
	}

	UDMXEntityFixturePatch* FixturePatch = FixturePatchRef.GetFixturePatch();
	if (!FixturePatch)
	{
		SetComponentTickEnabled(false);
		return;
	}

#if WITH_EDITOR
	if (GIsEditor && !GIsPlayInEditorWorld)
	{
		const UDMXProtocolSettings* ProtocolSettings = GetDefault<UDMXProtocolSettings>();
		if (!FixturePatch->bReceiveDMXInEditor && !ProtocolSettings->ShouldAllFixturePatchesReceiveDMXInEditor())
		{
			SetComponentTickEnabled(false);
			return;
		} 
	}
#endif 

	SetComponentTickEnabled(true);
}

void UDMXComponent::OnRegister()
{
	Super::OnRegister();

	SetupReceiveDMXBinding();
	UpdateTickEnabled();
}

void UDMXComponent::BeginPlay()
{
	Super::BeginPlay();

	if (UDMXEntityFixturePatch* FixturePatch = GetFixturePatch())
	{
		FDMXNormalizedAttributeValueMap NormalizeAttributeValues;
		FixturePatch->GetNormalizedAttributeValues(NormalizeAttributeValues);

		if (NormalizeAttributeValues.Map.Num() > 0)
		{
			OnFixturePatchReceived.Broadcast(FixturePatch, NormalizeAttributeValues);
		}
	}

	SetupReceiveDMXBinding();
	UpdateTickEnabled();
}

void UDMXComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if (OnDMXComponentTick.IsBound())
	{
		FEditorScriptExecutionGuard ScriptGuard;
		OnDMXComponentTick.Broadcast(DeltaTime);
	}
}

void UDMXComponent::DestroyComponent(bool bPromoteChildren)
{
	Super::DestroyComponent(bPromoteChildren);

	UDMXEntityFixturePatch* FixturePatch = FixturePatchRef.GetFixturePatch();
	if (IsValid(FixturePatch))
	{
		FixturePatch->OnFixturePatchReceivedDMXNative.RemoveAll(this);
	}
}

#if WITH_EDITOR
void UDMXComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXComponent, FixturePatchRef))
	{
		SetFixturePatch(FixturePatchRef.GetFixturePatch());
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXComponent, bReceiveDMXFromPatch))
	{
		SetupReceiveDMXBinding();
		UpdateTickEnabled();
	}
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UDMXComponent::OnAllFixturePatchesReceiveDMXInEditorEnabled(bool bEnabled)
{
	SetupReceiveDMXBinding();
	UpdateTickEnabled();
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UDMXComponent::OnFixturePatchPropertiesChanged(const UDMXEntityFixturePatch* FixturePatch)
{	
	// Look up the global setting first, which is on in most cases.
	// This is faster as GetFixturePatch can be slow.
	const UDMXProtocolSettings* ProtocolSettings = GetDefault<UDMXProtocolSettings>();
	if (ProtocolSettings->ShouldAllFixturePatchesReceiveDMXInEditor() && IsComponentTickEnabled())
	{
		return;
	}

	if (FixturePatch == GetFixturePatch())
	{
		SetupReceiveDMXBinding();
		UpdateTickEnabled();
	}
}
#endif // WITH_EDITOR
