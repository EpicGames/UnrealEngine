// Copyright Epic Games, Inc. All Rights Reserved.

#include "Library/DMXEntity.h"

#include "DMXRuntimeLog.h"
#include "Library/DMXLibrary.h"
#include "Interfaces/IDMXProtocol.h"

#include "Dom/JsonObject.h"

UDMXEntity::UDMXEntity()
{
	FPlatformMisc::CreateGuid(Id);
}

void UDMXEntity::PostInitProperties()
{
	Super::PostInitProperties();

	if (UDMXLibrary* DMXLibrary = Cast<UDMXLibrary>(GetOuter()))
	{
		ParentLibrary = DMXLibrary;
		DMXLibrary->RegisterEntity(this);
	}
	else if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		UE_LOGF(LogDMXRuntime, Fatal, "Entities must always be registered with a library")
	}
}

void UDMXEntity::Destroy()
{
	if (UDMXLibrary* Library = ParentLibrary.Get())
	{
		Library->UnregisterEntity(this);
		ParentLibrary.Reset();
		Id = FGuid();
		SetName(TEXT("Destroyed_Entity"));
	}
}

FString UDMXEntity::GetDisplayName() const
{
	return Name;
}

void UDMXEntity::SetName(const FString& InNewName)
{
	const FName UniqueName = MakeUniqueObjectName(GetOuter(), UDMXEntity::StaticClass(), *InNewName);
	Rename(*UniqueName.ToString(), GetOuter());
	Name = InNewName;
}

void UDMXEntity::SetParentLibrary(UDMXLibrary* InParent)
{
	ParentLibrary = InParent;
}

void UDMXEntity::RefreshID()
{
	FPlatformMisc::CreateGuid(Id);
}

void UDMXEntity::ReplicateID(UDMXEntity* Other)
{
	Id = Other->Id;
}
