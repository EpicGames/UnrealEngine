// Copyright Epic Games, Inc. All Rights Reserved.

#include "Library/DMXEntityFixturePatch.h"

#include "Algo/Find.h"
#include "Algo/MaxElement.h"
#include "Algo/NoneOf.h"
#include "DMXConversions.h"
#include "DMXProtocolConstants.h"
#include "DMXRuntimeLog.h"
#include "DMXRuntimeMainStreamObjectVersion.h"
#include "DMXRuntimeUtils.h"
#include "DMXStats.h"
#include "DMXTypes.h"
#include "IO/DMXInputPort.h"
#include "IO/DMXOutputPort.h"
#include "IO/DMXTrace.h"
#include "Library/DMXEntityController.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/DMXLibrary.h"
#include "Modulators/DMXModulator.h"
#include "MVR/Types/DMXMVRFixtureNode.h"
#include "UObject/UObjectGlobals.h"

DECLARE_LOG_CATEGORY_CLASS(DMXEntityFixturePatchLog, Log, All);

DECLARE_CYCLE_STAT(TEXT("FixturePatch receive DMX"), STAT_DMXFixturePatchReceiveDMX, STATGROUP_DMX);
DECLARE_CYCLE_STAT(TEXT("FixturePatch cache values"), STAT_DMXFixturePatchCacheValues, STATGROUP_DMX);

#define LOCTEXT_NAMESPACE "DMXEntityFixturePatch"

FDMXOnFixturePatchChangedDelegate UDMXEntityFixturePatch::OnFixturePatchChangedDelegate;

UDMXEntityFixturePatch::UDMXEntityFixturePatch()
	: UniverseID(1)
	, ActiveMode(0)
#if WITH_EDITORONLY_DATA
	, EditorColor(FLinearColor(1.0f, 0.0f, 1.0f))
	, bReceiveDMXInEditor(false)
#endif // WITH_EDITORONLY_DATA
{}

UDMXEntityFixturePatch* UDMXEntityFixturePatch::CreateFixturePatchInLibrary(FDMXEntityFixturePatchConstructionParams ConstructionParams, const FString& DesiredName, bool bMarkDMXLibraryDirty)
{
	UDMXEntityFixtureType* FixtureType = ConstructionParams.FixtureTypeRef.GetFixtureType();

	if (FixtureType)
	{
		UDMXLibrary* DMXLibrary = FixtureType->GetParentLibrary();
		if (ensureMsgf(DMXLibrary, TEXT("Cannot create Fixture Patch when Fixture Type's DMX Library is invalid.")))
		{
			const FString EntityName = FDMXRuntimeUtils::FindUniqueEntityName(DMXLibrary, UDMXEntityFixturePatch::StaticClass(), DesiredName);
			UDMXEntityFixturePatch* NewFixturePatch = NewObject<UDMXEntityFixturePatch>(DMXLibrary, UDMXEntityFixturePatch::StaticClass(), NAME_None, RF_Transactional);

#if WITH_EDITOR
			if (bMarkDMXLibraryDirty)
			{
				DMXLibrary->Modify();
				DMXLibrary->PreEditChange(UDMXLibrary::StaticClass()->FindPropertyByName(UDMXLibrary::GetEntitiesPropertyName()));
				NewFixturePatch->PreEditChange(nullptr);
			}
#endif
			
			NewFixturePatch->SetName(EntityName);
			NewFixturePatch->SetFixtureType(FixtureType);
			NewFixturePatch->SetUniverseID(ConstructionParams.UniverseID);
			NewFixturePatch->SetStartingChannel(ConstructionParams.StartingAddress);
			NewFixturePatch->SetActiveModeIndex(ConstructionParams.ActiveMode);

#if WITH_EDITOR
			NewFixturePatch->SetDefaultTransform(ConstructionParams.DefaultTransform);
#endif 

			if (ConstructionParams.MVRFixtureUUID.IsValid())
			{
				// Make sure the MVR UUID is truly unique across the DNX Library
				const TArray<UDMXEntityFixturePatch*> FixturePatches = DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>();
				const UDMXEntityFixturePatch* const* bOtherFixturePatchWithMVRFixtureUUIDPtr = FixturePatches.FindByPredicate([NewFixturePatch, &ConstructionParams](const UDMXEntityFixturePatch* FixturePatch)
					{
						return
							FixturePatch != NewFixturePatch &&
							FixturePatch->GetMVRFixtureUUID() == ConstructionParams.MVRFixtureUUID;
					});

				if (bOtherFixturePatchWithMVRFixtureUUIDPtr)
				{
					UE_LOGF(LogDMXRuntime, Error, "Trying to create Fixture Patch '%ls' with Unique MVR Fixture ID '%ls' in DMX Library '%ls'.", *DesiredName, *ConstructionParams.MVRFixtureUUID.ToString(), *DMXLibrary->GetName());
					UE_LOGF(LogDMXRuntime, Error, "However Fixture Patch '%ls' in the same Library already uses this Unique MVR Fixture ID.", *(*bOtherFixturePatchWithMVRFixtureUUIDPtr)->Name);
					UE_LOGF(LogDMXRuntime, Error, "Generating a new Unique MVR Fixture ID for the new Fixture Patch instead");
					ensureAlways(0);

					ConstructionParams.MVRFixtureUUID = FGuid::NewGuid();
				}
			}
			else
			{
				// If no MVR Fixture UUID is specified, generate one.
				ConstructionParams.MVRFixtureUUID = FGuid::NewGuid();
			}
			NewFixturePatch->MVRFixtureUUID = ConstructionParams.MVRFixtureUUID;

			NewFixturePatch->GenerateFixtureID();

#if WITH_EDITOR
			// Make a nice Editor Color
			const FLinearColor EditorColor = [DMXLibrary, NewFixturePatch]()
			{
				const TArray<UDMXEntityFixturePatch*> FixturePatches = DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>();
				const UDMXEntityFixturePatch* const* FixturePatchOfSameTypePtr = Algo::FindByPredicate(FixturePatches, [NewFixturePatch](const UDMXEntityFixturePatch* FixturePatch)
					{
						return
							FixturePatch != NewFixturePatch &&
							FixturePatch->GetFixtureType() == NewFixturePatch->GetFixtureType();
					});
				return FixturePatchOfSameTypePtr ? (*FixturePatchOfSameTypePtr)->EditorColor : FLinearColor::MakeRandomColor();
			}();

			NewFixturePatch->EditorColor = EditorColor;
#endif

#if WITH_EDITOR
			if (bMarkDMXLibraryDirty)
			{
				DMXLibrary->PostEditChange();
				NewFixturePatch->PostEditChange();
			}
#endif

			OnFixturePatchChangedDelegate.Broadcast(NewFixturePatch);
			return NewFixturePatch;
		}
	}

	return nullptr;
}

void UDMXEntityFixturePatch::RemoveFixturePatchFromLibrary(FDMXEntityFixturePatchRef FixturePatchRef)
{
	if (UDMXEntityFixturePatch* FixturePatch = FixturePatchRef.GetFixturePatch())
	{
		if (UDMXLibrary* DMXLibrary = FixturePatch->GetParentLibrary())
		{
			DMXLibrary->Modify();
			FixturePatch->Modify();

			FixturePatch->Destroy();
		}
	}
}

#if WITH_EDITOR
bool UDMXEntityFixturePatch::Modify(bool bAlwaysMarkDirty)
{
	if (UDMXLibrary* DMXLibrary = ParentLibrary.Get())
	{
		return DMXLibrary->Modify(bAlwaysMarkDirty) && Super::Modify(bAlwaysMarkDirty);
	}

	return Super::Modify(bAlwaysMarkDirty);
}
#endif // WITH_EDITOR

void UDMXEntityFixturePatch::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FDMXRuntimeMainStreamObjectVersion::GUID);
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FDMXRuntimeMainStreamObjectVersion::GUID) < FDMXRuntimeMainStreamObjectVersion::DMXFixturePatchHasMVRUUID)
		{
			MVRFixtureUUID = FGuid::NewGuid();
		}

#if WITH_EDITOR
		if (Ar.CustomVer(FDMXRuntimeMainStreamObjectVersion::GUID) < FDMXRuntimeMainStreamObjectVersion::DMXFixturePatchNoLongerImplementsAutoAssign)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			StartingChannel = ManualStartingAddress_DEPRECATED;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
#endif
	}
}

void UDMXEntityFixturePatch::PostInitProperties()
{
	Super::PostInitProperties();

	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		UDMXEntityFixtureType::GetOnFixtureTypeChanged().AddUObject(this, &UDMXEntityFixturePatch::OnFixtureTypeChanged);
		RebuildCache();
	}
}

void UDMXEntityFixturePatch::PostLoad()
{
	Super::PostLoad();

	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{		

#if WITH_EDITOR	
		// ~UE5.3, upgrade the patch to hold its fixture ID
		if (GetLinkerCustomVersion(FDMXRuntimeMainStreamObjectVersion::GUID) < FDMXRuntimeMainStreamObjectVersion::DMXFixturePatchHasFixtureID)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			UDMXLibrary* DMXLibrary = GetParentLibrary();
			UDMXMVRGeneralSceneDescription* GeneralSceneDescription = DMXLibrary ? DMXLibrary->GetLazyGeneralSceneDescription() : nullptr;
			UDMXMVRFixtureNode* FixtureNode = GeneralSceneDescription ? GeneralSceneDescription->FindFixtureNode(MVRFixtureUUID) : nullptr;
			if (FixtureNode)
			{
				int32 IntegralMVRFixtureID;
				if (LexTryParseString(IntegralMVRFixtureID, *FixtureNode->FixtureID))
				{
					FixtureID = IntegralMVRFixtureID;
				}
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		// ~UE5.5, adopt the DefaultTransform property from the General Scene Description
		if (GetLinkerCustomVersion(FDMXRuntimeMainStreamObjectVersion::GUID) < FDMXRuntimeMainStreamObjectVersion::DMXFixturePatchesHaveDefaultTransform)
		{
			UDMXLibrary* DMXLibrary = GetParentLibrary();
			UDMXMVRGeneralSceneDescription* GeneralSceneDescription = DMXLibrary ? DMXLibrary->GetLazyGeneralSceneDescription() : nullptr;
			UDMXMVRFixtureNode* FixtureNode = GeneralSceneDescription ? GeneralSceneDescription->FindFixtureNode(MVRFixtureUUID) : nullptr;
			if (FixtureNode)
			{
				DefaultTransform = FixtureNode->GetTransformAbsolute();
			}
		}
#endif

		// Mend invalid fixture IDs
		if (FixtureID < 1)
		{
			GenerateFixtureID();
		}

		RebuildCache();
	}
}

#if WITH_EDITOR
void UDMXEntityFixturePatch::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		RebuildCache();
		OnFixturePatchChangedDelegate.Broadcast(this);
	}
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UDMXEntityFixturePatch::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedChainEvent);

	if (PropertyChangedChainEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		RebuildCache();
		OnFixturePatchChangedDelegate.Broadcast(this);
	}
}
#endif // WITH_EDITOR

void UDMXEntityFixturePatch::Tick(float DeltaTime)
{
	const bool bDataChanged = UpdateCache();

	// Broadcast data changes
	if (bDataChanged)
	{
		if (const FDMXNormalizedAttributeValueMap* NormalizedAttributeValuesPtr = Cache.GetAllNormalizedAttributeValues())
		{
			OnFixturePatchReceivedDMXNative.Broadcast(this, *NormalizedAttributeValuesPtr);
			OnFixturePatchReceivedDMX.Broadcast(this, *NormalizedAttributeValuesPtr);
		}
	}
}

bool UDMXEntityFixturePatch::IsTickable() const
{
	return OnFixturePatchReceivedDMXNative.IsBound() || OnFixturePatchReceivedDMX.IsBound();
}

bool UDMXEntityFixturePatch::IsTickableInEditor() const
{
	const bool bHasListener = OnFixturePatchReceivedDMXNative.IsBound() || OnFixturePatchReceivedDMX.IsBound();
	if (!bHasListener)
	{
		return false;
	}

#if WITH_EDITOR
	if (GIsEditor && !GIsPlayInEditorWorld)
	{
		if (bReceiveDMXInEditor)
		{
			return true;
		}

		const UDMXProtocolSettings* ProtocolSettings = GetDefault<UDMXProtocolSettings>();
		const bool bAllFixturePatchesReceiveDMXInEditor = ProtocolSettings->ShouldAllFixturePatchesReceiveDMXInEditor();
		return bAllFixturePatchesReceiveDMXInEditor;
	}
#endif

	return true; 
}

ETickableTickType UDMXEntityFixturePatch::GetTickableTickType() const
{
	return ETickableTickType::Conditional;
}

TStatId UDMXEntityFixturePatch::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FDMXInputPort, STATGROUP_Tickables);
}

FDMXOnFixturePatchChangedDelegate& UDMXEntityFixturePatch::GetOnFixturePatchChanged()
{
	return OnFixturePatchChangedDelegate;
}

void UDMXEntityFixturePatch::SendDMX(TMap<FDMXAttributeName, int32> AttributeMap)
{		
	const FDMXFixtureMode* ModePtr = GetActiveMode();
	if (!ModePtr)
	{
		UE_LOGF(LogDMXRuntime, Error, "Tried to send DMX via Fixture Patch %ls, but its Parent Fixture Type has no Modes set up.", *GetName());
		return;
	}

	TMap<int32, uint8> DMXChannelToValueMap;
	for (const TPair<FDMXAttributeName, int32>& Elem : AttributeMap)
	{
		for (const FDMXFixtureFunction& Function : ModePtr->Functions)
		{
			const FDMXAttributeName& FunctionAttribute = Function.Attribute;
			if (FunctionAttribute == Elem.Key)
			{
				const int32 LastFunctionChannel = Function.Channel + Function.GetNumChannels() - 1;
				const bool bLastChannelExceedsChannelSpan = LastFunctionChannel > ModePtr->ChannelSpan;
				const bool bLastChannelExceedsUniverseSize = LastFunctionChannel + GetStartingChannel() - 1 > DMX_MAX_ADDRESS;

				if (!bLastChannelExceedsChannelSpan && !bLastChannelExceedsUniverseSize)
				{
					const int32 Channel = Function.Channel + GetStartingChannel() - 1;

					uint32 ChannelValue = 0;
					uint8* ChannelValueBytes = reinterpret_cast<uint8*>(&ChannelValue);
					UDMXEntityFixtureType::FunctionValueToBytes(Function, Elem.Value, ChannelValueBytes);

					const uint8 NumBytesInSignalFormat = FDMXConversions::GetSizeOfSignalFormat(Function.DataType);
					for (uint8 ChannelIt = 0; ChannelIt < NumBytesInSignalFormat; ++ChannelIt)
					{
						DMXChannelToValueMap.Add(Channel + ChannelIt, ChannelValueBytes[ChannelIt]);
					}
				}
			}
		}
	}

	// Send to the library's output ports
	if (UDMXLibrary* DMXLibrary = ParentLibrary.Get())
	{
		UE_DMX_SCOPED_TRACE_SENDDMX(DMXLibrary->GetFName());
		UE_DMX_SCOPED_TRACE_SENDDMX(*GetDisplayName());
		for (const FDMXOutputPortSharedRef& OutputPort : DMXLibrary->GetOutputPorts())
		{
			OutputPort->SendDMX(UniverseID, DMXChannelToValueMap);
		}
	}
}

void UDMXEntityFixturePatch::SendDefaultValues()
{
	constexpr bool bSendDefaultValues = true;
	SendResetDataToAllAttributes(bSendDefaultValues);
}

void UDMXEntityFixturePatch::SendZeroValues()
{
	constexpr bool bSendDefaultValues = false;
	SendResetDataToAllAttributes(bSendDefaultValues);
}

void UDMXEntityFixturePatch::RebuildCache()
{
	LastDMXSignal.Reset();
	Cache = FDMXEntityFixturePatchCache(GetStartingChannel(), GetActiveMode(), GetFixtureMatrix());
}

bool UDMXEntityFixturePatch::UpdateCache()
{
	SCOPE_CYCLE_COUNTER(STAT_DMXFixturePatchCacheValues);

	if (!Cache.HasValidProperties())
	{
		return false;
	}

	if (UDMXLibrary* DMXLibrary = ParentLibrary.Get())
	{
		// Get the lastest DMX signal
		FDMXSignalSharedPtr NewDMXSignal;
		for (const FDMXInputPortSharedRef& InputPort : DMXLibrary->GetInputPorts())
		{
			if(InputPort->GameThreadGetDMXSignal(UniverseID, NewDMXSignal))
			{
				if (!LastDMXSignal.IsValid() ||
					NewDMXSignal->Timestamp > LastDMXSignal->Timestamp)
				{
					LastDMXSignal = NewDMXSignal;
				}
			}
		}

		for (const FDMXOutputPortSharedRef& OutputPort : DMXLibrary->GetOutputPorts())
		{
			constexpr bool bGetSignalWithLoopbackDisabled = false;
			if (OutputPort->GameThreadGetDMXSignal(UniverseID, NewDMXSignal, bGetSignalWithLoopbackDisabled))
			{
				if (!LastDMXSignal.IsValid() ||
					NewDMXSignal->Timestamp > LastDMXSignal->Timestamp)
				{
					LastDMXSignal = NewDMXSignal;
				}
			}
		}

		if (LastDMXSignal.IsValid())
		{
			// Update the cache and see if data changed
			bool bDataChanged = Cache.InputDMXSignal(LastDMXSignal);

			if (bDataChanged)
			{
				// Apply modulators
				if (ParentFixtureTypeTemplate)
				{
					for (UDMXModulator* InputModulator : ParentFixtureTypeTemplate->InputModulators)
					{
						if (InputModulator)
						{
							Cache.Modulate(this, InputModulator);

							bDataChanged = true;
						}
					}
				}
			}

			return bDataChanged;
		}
	}

	return false;
}

bool UDMXEntityFixturePatch::IsValidEntity(FText& OutReason) const
{
	OutReason = FText::GetEmpty();

	if (!ParentFixtureTypeTemplate)
	{
		OutReason = LOCTEXT("InvalidReason_NullParentTemplate", "Fixture Type is null");
	}
	else if ((GetStartingChannel() + GetChannelSpan() - 1) > DMX_UNIVERSE_SIZE)
	{
		OutReason = FText::Format(
			LOCTEXT("InvalidReason_ChannelOverflow", "Channels range overflows max channel address ({0})"),
			FText::AsNumber(DMX_UNIVERSE_SIZE)
		);
	}
	else if (ParentFixtureTypeTemplate->Modes.Num() == 0)
	{
		OutReason = FText::Format(
			LOCTEXT("InvalidReason_NoModesDefined", "'{0}' cannot be assigned as its parent Fixture Type '{1}' does not define any Modes."),			
			FText::FromString(GetDisplayName()),
			FText::FromString(ParentFixtureTypeTemplate->GetDisplayName())
		);
	}	
	else if (!ParentFixtureTypeTemplate->Modes.IsValidIndex(ActiveMode))
	{
		OutReason = FText::Format(
			LOCTEXT("InvalidReason_NoActiveModes", "'{0}' cannot be assigned as its parent Fixture Type '{1}' does no longer contain Active Mode."),
			FText::FromString(GetDisplayName()),
			FText::FromString(ParentFixtureTypeTemplate->GetDisplayName())
		);
	}
	else if (
		ParentFixtureTypeTemplate->Modes[ActiveMode].Functions.Num() == 0 && 
		ParentFixtureTypeTemplate->Modes[ActiveMode].FixtureMatrixConfig.CellAttributes.Num() == 0)
	{
		OutReason = FText::Format(
			LOCTEXT("InvalidReason_NoFunctionsDefined", "'{0}' cannot be assigned as its parent Fixture Type '{1}' does not define any Functions in the Active Mode."),
			FText::FromString(GetDisplayName()),
			FText::FromString(ParentFixtureTypeTemplate->GetDisplayName()));
	}
	else if (GetChannelSpan() == 0)
	{
		OutReason = FText::Format(
			LOCTEXT("InvalidReason_ParentChannelSpanIsZero", "'{0}' cannot be assigned as its parent Fixture Type '{1}' overrides channel span with 0."),
			FText::FromString(GetDisplayName()),
			FText::FromString(ParentFixtureTypeTemplate->GetDisplayName()));
	}

	return OutReason.IsEmpty();
}

void UDMXEntityFixturePatch::ValidateActiveMode()
{
	if (ParentFixtureTypeTemplate && !ParentFixtureTypeTemplate->Modes.IsEmpty())
	{
		ActiveMode = FMath::Clamp(ActiveMode, 0, ParentFixtureTypeTemplate->Modes.Num() - 1);
	}
	else
	{
		ActiveMode = INDEX_NONE;
	}
}

const FDMXFixtureMode* UDMXEntityFixturePatch::GetActiveMode() const
{
	if (ParentFixtureTypeTemplate && 
		ParentFixtureTypeTemplate->IsValidLowLevelFast() &&
		ParentFixtureTypeTemplate->Modes.IsValidIndex(ActiveMode))
	{
		return &ParentFixtureTypeTemplate->Modes[ActiveMode];
	}

	return nullptr;
}

void UDMXEntityFixturePatch::SetFixtureType(UDMXEntityFixtureType* NewFixtureType)
{
	if (!IsValid(NewFixtureType))
	{
		ParentFixtureTypeTemplate = nullptr;
	}
	else if (NewFixtureType && NewFixtureType != ParentFixtureTypeTemplate)
	{
		ParentFixtureTypeTemplate = NewFixtureType;
		
		ActiveMode = ParentFixtureTypeTemplate->Modes.Num() > 0 ? 0 : INDEX_NONE;
	}

	RebuildCache();
}

void UDMXEntityFixturePatch::SetUniverseID(int32 NewUniverseID)
{
	UniverseID = NewUniverseID;

	RebuildCache();
}

void UDMXEntityFixturePatch::SetStartingChannel(int32 NewStartingChannel)
{
	if (NewStartingChannel == StartingChannel)
	{
		return;
	}

	StartingChannel = NewStartingChannel;

	RebuildCache();
}

int32 UDMXEntityFixturePatch::GetChannelSpan() const
{
	return Cache.GetChannelSpan();
}

int32 UDMXEntityFixturePatch::GetEndingChannel() const
{
	return GetStartingChannel() + GetChannelSpan() - 1;
}

bool UDMXEntityFixturePatch::SetActiveModeIndex(int32 NewActiveModeIndex)
{
	if (IsValid(ParentFixtureTypeTemplate))
	{
		if (ParentFixtureTypeTemplate->Modes.IsValidIndex(NewActiveModeIndex))
		{
			ActiveMode = NewActiveModeIndex;
			RebuildCache();

			return true;
		}
	}

	return false;
}

void UDMXEntityFixturePatch::GenerateFixtureID(int32 DesiredFixtureID)
{
	if (!ParentLibrary.IsValid())
	{
		FixtureID = 1;
		return;
	}

	const TArray<UDMXEntityFixturePatch*> FixturePatches = ParentLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>();

	// Try to use the desired fixture ID
	if (DesiredFixtureID > 0)
	{
		const bool bCanUseDesiredFixtureID = Algo::NoneOf(FixturePatches, [DesiredFixtureID, this](const UDMXEntityFixturePatch* Other)
			{
				return 
					Other &&
					Other != this &&
					Other->GetFixtureID() == DesiredFixtureID;
			});
		if (bCanUseDesiredFixtureID)
		{
			FixtureID = DesiredFixtureID;
			return;
		}
	}

	// Generate a new fixture ID
	const UDMXEntityFixturePatch* const* MaxFixtureIDPatchPtr = Algo::MaxElementBy(FixturePatches, [this](const UDMXEntityFixturePatch* Other)
		{
			if (Other && Other != this)
			{
				return Other->GetFixtureID();
			}
			return 0;
		});
	if (MaxFixtureIDPatchPtr)
	{
		FixtureID = (*MaxFixtureIDPatchPtr)->GetFixtureID() + 1;
	}
	else
	{
		FixtureID = 1;
	}
}

bool UDMXEntityFixturePatch::FindFixtureID(int32& OutFixtureID) const
{
	// DEPRECATED 5.5
	OutFixtureID = FixtureID;
	return true;
}

#if WITH_EDITOR
int32 UDMXEntityFixturePatch::GetRemoteUniverse() const
{	
	/** DEPRECATED 4.27 */
	UE_LOGF(LogDMXRuntime, Error, "No clear remote Universe can be deduced in DMXEntityFixturePatch::GetRemoteUniverse. Returning 0.");
	return 0;
}
#endif // WITH_EDITOR

TArray<FDMXAttributeName> UDMXEntityFixturePatch::GetAllAttributesInActiveMode() const
{
	TArray<FDMXAttributeName> AttributeNames;
	for (const FDMXFixtureFunction& Function : Cache.GetFunctions())
	{
		if (Function.GetLastChannel() <= Cache.GetChannelSpan())
		{
			AttributeNames.Add(Function.Attribute.Name);
		}
	}

	return AttributeNames;
}

TMap<FDMXAttributeName, FDMXFixtureFunction> UDMXEntityFixturePatch::GetAttributeFunctionsMap() const
{
	TMap<FDMXAttributeName, FDMXFixtureFunction> FunctionMap;
	FunctionMap.Reserve(Cache.GetFunctions().Num());

	for (const FDMXFixtureFunction& Function : Cache.GetFunctions())
	{
		if (Function.GetLastChannel() <= Cache.GetChannelSpan())
		{
			FunctionMap.Add(Function.Attribute.Name, Function);
		}
	}

	return FunctionMap;
}

TMap<FDMXAttributeName, int32> UDMXEntityFixturePatch::GetAttributeDefaultMap() const
{
	TMap<FDMXAttributeName, int32> DefaultValueMap;
	DefaultValueMap.Reserve(Cache.GetFunctions().Num());

	for (const FDMXFixtureFunction& Function : Cache.GetFunctions())
	{
		if (Function.GetLastChannel() <= Cache.GetChannelSpan())
		{
			DefaultValueMap.Add(Function.Attribute, Function.DefaultValue);
		}
	}
	
	return DefaultValueMap;
}

TMap<FDMXAttributeName, int32> UDMXEntityFixturePatch::GetAttributeChannelAssignments() const
{
	TMap<FDMXAttributeName, int32> ChannelMap;
	ChannelMap.Reserve(Cache.GetFunctions().Num());

	for (const FDMXFixtureFunction& Function : Cache.GetFunctions())
	{
		if (Function.GetLastChannel() <= Cache.GetChannelSpan())
		{
			ChannelMap.Add(Function.Attribute, Function.Channel + GetStartingChannel() - 1);
		}
	}

	return ChannelMap;
}

TMap<FDMXAttributeName, EDMXFixtureSignalFormat> UDMXEntityFixturePatch::GetAttributeSignalFormats() const
{
	TMap<FDMXAttributeName, EDMXFixtureSignalFormat> FormatMap;
	FormatMap.Reserve(Cache.GetFunctions().Num());

	for (const FDMXFixtureFunction& Function : Cache.GetFunctions())
	{
		if (Function.GetLastChannel() <= Cache.GetChannelSpan())
		{
			FormatMap.Add(Function.Attribute, Function.DataType);
		}
	}
	return FormatMap;
}

TMap<FDMXAttributeName, int32> UDMXEntityFixturePatch::ConvertRawMapToAttributeMap(const TMap<int32, uint8>& RawMap) const
{
	// DEPRECATED 4.27
	TMap<FDMXAttributeName, int32> FunctionMap;
	FunctionMap.Reserve(Cache.GetFunctions().Num());

	for (const FDMXFixtureFunction& Function : Cache.GetFunctions())
	{
		if (Function.GetLastChannel() <= Cache.GetChannelSpan())
		{
			const int32 FunctionStartingChannel = Function.Channel + (GetStartingChannel() - 1);

			// Find the function at specified channel
			if (RawMap.Contains(FunctionStartingChannel))
			{
				const uint8& RawValue(RawMap.FindRef(Function.Channel));
				const uint8 ChannelsToAdd = FDMXConversions::GetSizeOfSignalFormat(Function.DataType);

				TArray<uint8, TFixedAllocator<4>> Bytes;
				for (uint8 ChannelIt = 0; ChannelIt < ChannelsToAdd; ChannelIt++)
				{
					if (const uint8* RawVal = RawMap.Find(FunctionStartingChannel + ChannelIt))
					{
						Bytes.Add(*RawVal);
					}
				}

				const uint32 IntValue = UDMXEntityFixtureType::BytesToInt(Function.DataType, Function.bUseLSBMode, Bytes.GetData());
				FunctionMap.Add(Function.Attribute, IntValue);
			}
		}
	}

	return FunctionMap;
}

TMap<int32, uint8> UDMXEntityFixturePatch::ConvertAttributeMapToRawMap(const TMap<FDMXAttributeName, int32>& FunctionMap) const
{
	TMap<int32, uint8> RawMap;
	RawMap.Reserve(FunctionMap.Num());

	for (const TPair<FDMXAttributeName, int32>& AttributeNameValuePair : FunctionMap)
	{
		// Search for a function with Attribute == Elem.Key.Attribute
		const FDMXFixtureFunction* FunctionPtr = Cache.GetFunctions().FindByPredicate([&AttributeNameValuePair](const FDMXFixtureFunction& Function) {
			return AttributeNameValuePair.Key == Function.Attribute;
		});

		// Also check for the Function being in the valid range for the Active Mode's channel span
		if (FunctionPtr && FunctionPtr->GetLastChannel() <= Cache.GetChannelSpan())
		{
			const int32 FunctionStartingChannel = FunctionPtr->Channel + (GetStartingChannel() - 1);
			const uint8 NumChannels = FDMXConversions::GetSizeOfSignalFormat(FunctionPtr->DataType);

			const uint32 SafeValue = FMath::Max(0, AttributeNameValuePair.Value);
			const uint32 ClampedValue = FDMXConversions::ClampValueBySignalFormat(SafeValue, FunctionPtr->DataType);

			// To avoid branching in the loop, we'll decide before it on which byte to start
			// and which direction to go, depending on the Function's bit endianness.
			const int8 ByteIndexStep = FunctionPtr->bUseLSBMode ? 1 : -1;
			int8 ByteIndex = FunctionPtr->bUseLSBMode ? 0 : NumChannels - 1;

			for (uint8 ByteOffset = 0; ByteOffset < NumChannels; ++ByteOffset)
			{
				const uint8 ChannelVal = (ClampedValue >> (8 * ByteOffset)) & 0xFF;

				const int32 FinalChannel = FunctionStartingChannel + ByteIndex;
				RawMap.Add(FinalChannel, ChannelVal);

				ByteIndex += ByteIndexStep;
			}
		}
	}

	return RawMap;
}

bool UDMXEntityFixturePatch::IsMapValid(const TMap<FDMXAttributeName, int32>& FunctionMap) const
{
	for (const TPair<FDMXAttributeName, int32>& Elem : FunctionMap)
	{
		const bool bContainsAttributeName = Cache.GetFunctions().ContainsByPredicate([Elem](const FDMXFixtureFunction& Function) {
			return Function.Attribute.Name == Elem.Key;
		});

		if (!bContainsAttributeName)
		{
			return false;
		}
	}

	return true;
}

bool UDMXEntityFixturePatch::ContainsAttribute(FDMXAttributeName FunctionAttribute) const
{
	const bool bContainsAttributeName = Cache.GetFunctions().ContainsByPredicate([&FunctionAttribute](const FDMXFixtureFunction& Function) {
		return Function.Attribute.Name == FunctionAttribute;
	});

	return bContainsAttributeName;
}

TMap<FDMXAttributeName, int32> UDMXEntityFixturePatch::ConvertToValidMap(const TMap<FDMXAttributeName, int32>& FunctionMap) const
{
	TMap<FDMXAttributeName, int32> ValidMap;
	ValidMap.Reserve(FunctionMap.Num());

	for (const TPair<FDMXAttributeName, int32>& Elem : FunctionMap)
	{
		// Search for a function with Attribute == Elem.Key.Attribute
		const FDMXFixtureFunction* FunctionPtr = Cache.GetFunctions().FindByPredicate([&Elem](const FDMXFixtureFunction& Function) {
			return Elem.Key == Function.Attribute;
		});

		// Also check for the Function being in the valid range for the Active Mode's channel span
		if (FunctionPtr && FunctionPtr->GetLastChannel() <= Cache.GetChannelSpan())
		{
			ValidMap.Add(Elem.Key, Elem.Value);
		}
	}

	return ValidMap;
}

TArray<UDMXEntityController*> UDMXEntityFixturePatch::GetRelevantControllers() const
{
	// DEPRECATED 4.27
	TArray<UDMXEntityController*> EmptyArray;
	return EmptyArray;
}

const FDMXFixtureFunction* UDMXEntityFixturePatch::GetAttributeFunction(const FDMXAttributeName& Attribute) const
{
	const FDMXFixtureFunction* FixtureFunctionPtr = Cache.GetFunctions().FindByPredicate([Attribute](const FDMXFixtureFunction& Function) {
		return Function.Attribute.Name == Attribute;
	});

	if (FixtureFunctionPtr && 
		GetStartingChannel() + FixtureFunctionPtr->GetNumChannels() - 1 < DMX_UNIVERSE_SIZE)
	{
		return FixtureFunctionPtr;
	}

	return nullptr;
}

int32 UDMXEntityFixturePatch::GetAttributeValue(FDMXAttributeName Attribute, bool& bSuccess)
{
	// Update the cache if it isn't updated on tick
	if (!IsTickable())
	{
		UpdateCache();
	}

	if (const int32* ValuePtr = Cache.GetRawAttributeValue(Attribute))
	{
		bSuccess = true;
		return *ValuePtr;
	}

	return 0;
}

float UDMXEntityFixturePatch::GetNormalizedAttributeValue(FDMXAttributeName Attribute, bool& bSuccess)
{
	// Update the cache if it isn't updated on tick
	if (!IsTickable())
	{
		UpdateCache();
	}

	if (const float* ValuePtr = Cache.GetNormalizedAttributeValue(Attribute))
	{
		bSuccess = true;
		return *ValuePtr;
	}

	return 0.f;
}

void UDMXEntityFixturePatch::GetAttributesValues(TMap<FDMXAttributeName, int32>& AttributesValues)
{
	// DEPRECATED 5.5
	GetAttributeValues(AttributesValues);
}

void UDMXEntityFixturePatch::GetAttributeValues(TMap<FDMXAttributeName, int32>& AttributeValues)
{
	AttributeValues.Reset();

	// Update the cache if it isn't updated on tick
	if (!IsTickable())
	{
		UpdateCache();
	}

	if (const TMap<FDMXAttributeName, int32>* AttributeValuesPtr = Cache.GetAllRawAttributeValues())
	{
		AttributeValues = *AttributeValuesPtr;
	}
}

void UDMXEntityFixturePatch::GetNormalizedAttributesValues(FDMXNormalizedAttributeValueMap& NormalizedAttributesValues)
{
	// DEPRECATED 5.5
	GetNormalizedAttributeValues(NormalizedAttributesValues);
}

void UDMXEntityFixturePatch::GetNormalizedAttributeValues(FDMXNormalizedAttributeValueMap& NormalizedAttributeValues)
{
	// Update the cache if it isn't updated on tick
	if (!IsTickable())
	{
		UpdateCache();
	}

	if (const FDMXNormalizedAttributeValueMap* NormalizedAttributeValuesPtr = Cache.GetAllNormalizedAttributeValues())
	{
		NormalizedAttributeValues = *NormalizedAttributeValuesPtr;
	}
}

bool UDMXEntityFixturePatch::SendMatrixCellValue(const FIntPoint& CellCoordinate, const FDMXAttributeName& Attribute, int32 Value)
{
	if (UDMXLibrary* DMXLibrary = ParentLibrary.Get())
	{
		if (const FDMXFixtureMatrix* const FixtureMatrixPtr = GetFixtureMatrix())
		{
			UE_DMX_SCOPED_TRACE_SENDDMX(DMXLibrary->GetFName());
			UE_DMX_SCOPED_TRACE_SENDDMX(*GetDisplayName());

			const int32 DistributedCellIndex = Cache.GetDistributedCellIndex(CellCoordinate);
			const int32 AbsoluteMatrixStartingChannel = Cache.GetMatrixStartingChannelAbsolute();
			const int32 AbsoluteCellStartingChannel = AbsoluteMatrixStartingChannel + DistributedCellIndex * Cache.GetCellSize();

			// Iterate over the cell attributes, increment the attribute offset until the right attribute is found, then send it.
			int32 AttributeOffset = 0;
			for (const FDMXFixtureCellAttribute& CellAttribute : Cache.GetCellAttributes())
			{
				if (CellAttribute.Attribute.Name == Attribute)
				{
					const uint32 SafeValue = FMath::Max(0, Value);
					TArray<uint8> ByteArray = FDMXConversions::UnsignedInt32ToByteArray(SafeValue, CellAttribute.DataType, CellAttribute.bUseLSBMode);

					TMap<int32, uint8> DMXChannelToValueMap;
					DMXChannelToValueMap.Reserve(ByteArray.Num());
					for (int32 ByteIndex = 0; ByteIndex < CellAttribute.GetNumChannels(); ByteIndex++)
					{
						DMXChannelToValueMap.Add(AbsoluteCellStartingChannel + AttributeOffset + ByteIndex, ByteArray[ByteIndex]);
					}

					// Send to the library's output ports
					for (const FDMXOutputPortSharedRef& OutputPort : DMXLibrary->GetOutputPorts())
					{
						OutputPort->SendDMX(UniverseID, DMXChannelToValueMap);
					}

					return true;
				}

				AttributeOffset += CellAttribute.GetNumChannels();
			}
		}
	}

	return false;
}

bool UDMXEntityFixturePatch::SendMatrixCellValueWithAttributeMap(const FIntPoint& CellCoordinate, const FDMXAttributeName& Attribute, int32 Value, const TMap<FDMXAttributeName, int32>& InAttributeNameChannelMap)
{
	// DEPRECATED 4.27
	const FDMXFixtureMatrix* const FixtureMatrixPtr = GetFixtureMatrix();

	if (!FixtureMatrixPtr)
	{
		return false;
	}

	const FDMXFixtureMatrix& FixtureMatrix = *FixtureMatrixPtr;

	if (!AreCoordinatesValid(FixtureMatrix, CellCoordinate))
	{
		return false;
	}

	if (!ensure(InAttributeNameChannelMap.Num()))
	{
		return false;
	}

	TMap<int32, uint8> DMXChannelToValueMap;
	for (const FDMXFixtureCellAttribute& CellAttribute : FixtureMatrix.CellAttributes)
	{
		if (!InAttributeNameChannelMap.Contains(Attribute))
		{
			continue;
		}

		const int32 FirstChannel = InAttributeNameChannelMap[Attribute];
		const int32 LastChannel = FirstChannel + FDMXConversions::GetSizeOfSignalFormat(CellAttribute.DataType) - 1;

		TArray<uint8> ByteArr;
		ByteArr.AddZeroed(4);
		UDMXEntityFixtureType::IntToBytes(CellAttribute.DataType, CellAttribute.bUseLSBMode, Value, ByteArr.GetData());

		int32 ByteOffset = 0;
		for (int32 Channel = FirstChannel; Channel <= LastChannel; Channel++)
		{
			DMXChannelToValueMap.Add(Channel, ByteArr[ByteOffset]);
			ByteOffset++;
		}
	}

	// Send to the library's output ports
	if (UDMXLibrary* DMXLibrary = ParentLibrary.Get())
	{
		UE_DMX_SCOPED_TRACE_SENDDMX(DMXLibrary->GetFName());
		UE_DMX_SCOPED_TRACE_SENDDMX(*GetDisplayName());
		for (const FDMXOutputPortSharedRef& OutputPort : DMXLibrary->GetOutputPorts())
		{
			OutputPort->SendDMX(UniverseID, DMXChannelToValueMap);
		}
	}

	return true;
}

bool UDMXEntityFixturePatch::SendNormalizedMatrixCellValue(const FIntPoint& CellCoordinate, const FDMXAttributeName& Attribute, float Value)
{
	const FDMXFixtureMatrix* FixtureMatrixPtr = GetFixtureMatrix();

	if (!FixtureMatrixPtr)
	{
		return false;
	}

	const FDMXFixtureMatrix& FixtureMatrix = *FixtureMatrixPtr;

	const FDMXFixtureCellAttribute* const ExistingAttributePtr = FixtureMatrix.CellAttributes.FindByPredicate([&Attribute](const FDMXFixtureCellAttribute& TestedCellAttribute) {
		return TestedCellAttribute.Attribute == Attribute;
		});

	if (ExistingAttributePtr)
	{
		const FDMXFixtureCellAttribute& ExistingAttribute = *ExistingAttributePtr;

		Value = FMath::Clamp(Value, 0.0f, 1.0f);

		const uint32 UnsignedIntValue = FDMXConversions::GetSignalFormatMaxValue(ExistingAttribute.DataType) * Value;
		const int32  IntValue = FMath::Clamp(UnsignedIntValue, static_cast<uint32>(0), static_cast<uint32>(TNumericLimits<int32>::Max()));
	
		SendMatrixCellValue(CellCoordinate, Attribute, IntValue);

		return true;
	}
	
	return false;
}

bool UDMXEntityFixturePatch::GetMatrixCellValues(const FIntPoint& CellCoordinate, TMap<FDMXAttributeName, int32>& ValuePerAttribute)
{
	// Update the cache if it isn't updated on tick
	if (!IsTickable())
	{
		UpdateCache();
	}

	const int32 DistributedCellIndex = Cache.GetDistributedCellIndex(CellCoordinate);

	if (DistributedCellIndex != INDEX_NONE)
	{
		if (const TMap<FDMXAttributeName, int32>* RawMatrixAttributeValuesPtr = Cache.GetAllRawMatrixAttributeValuesFromCell(DistributedCellIndex))
		{
			ValuePerAttribute = *RawMatrixAttributeValuesPtr;
			return true;
		}
	}

	return false;
}

bool UDMXEntityFixturePatch::GetNormalizedMatrixCellValues(const FIntPoint& CellCoordinate, TMap<FDMXAttributeName, float>& NormalizedValuePerAttribute)
{
	// Update the cache if it isn't updated on tick
	if (!IsTickable())
	{
		UpdateCache();
	}

	const int32 DistributedCellIndex = Cache.GetDistributedCellIndex(CellCoordinate);
	
	if (const FDMXNormalizedAttributeValueMap* NormalizedMatrixAttributeValuesPtr = Cache.GetAllNormalizedMatrixAttributeValuesFromCell(DistributedCellIndex))
	{
		NormalizedValuePerAttribute = NormalizedMatrixAttributeValuesPtr->Map;
		return true;
	}

	return false;
}

bool UDMXEntityFixturePatch::GetMatrixCellChannelsRelative(const FIntPoint& CellCoordinate, TMap<FDMXAttributeName, int32>& AttributeChannelMap)
{
	if (Cache.IsFixtureMatrix())
	{
		const int32 DistributedCellIndex = Cache.GetDistributedCellIndex(CellCoordinate);
		const int32 FirstRelativeMatrixStartingChannel = Cache.GetMatrixStartingChannelAbsolute() - GetStartingChannel() + 1;
		const int32 RelativeMatrixStartingChannel = FirstRelativeMatrixStartingChannel + DistributedCellIndex * Cache.GetCellSize();

		int32 AttributeOffset = 0;
		for (const FDMXFixtureCellAttribute& CellAttribute : Cache.GetCellAttributes())
		{
			AttributeChannelMap.Add(CellAttribute.Attribute.Name, RelativeMatrixStartingChannel + AttributeOffset);
			AttributeOffset += CellAttribute.GetNumChannels();
		}
	}

	return true;
}

bool UDMXEntityFixturePatch::GetMatrixCellChannelsAbsolute(const FIntPoint& CellCoordinate /* Cell X/Y */, TMap<FDMXAttributeName, int32>& AttributeChannelMap)
{
	if (Cache.IsFixtureMatrix())
	{
		const int32 DistributedCellIndex = Cache.GetDistributedCellIndex(CellCoordinate);
		const int32 FirstAbsoluteMatrixStartingChannel = Cache.GetMatrixStartingChannelAbsolute();
		const int32 AbsoluteMatrixStartingChannel = FirstAbsoluteMatrixStartingChannel + DistributedCellIndex * Cache.GetCellSize();

		int32 AttributeOffset = 0;
		for (const FDMXFixtureCellAttribute& CellAttribute : Cache.GetCellAttributes())
		{
			AttributeChannelMap.Add(CellAttribute.Attribute.Name, AbsoluteMatrixStartingChannel + AttributeOffset);
			AttributeOffset += CellAttribute.GetNumChannels();
		}
		
		return true;
	}

	return false;
}

bool UDMXEntityFixturePatch::GetMatrixCellChannelsAbsoluteWithValidation(const FIntPoint& InCellCoordinate, TMap<FDMXAttributeName, int32>& OutAttributeChannelMap)
{
	if(Cache.IsFixtureMatrix())
	{
		return GetMatrixCellChannelsAbsolute(InCellCoordinate, OutAttributeChannelMap);
	}

	return false;
}

bool UDMXEntityFixturePatch::GetMatrixProperties(FDMXFixtureMatrix& MatrixProperties) const
{
	if (!ParentFixtureTypeTemplate)
	{
		return false;
	}

	const FDMXFixtureMode* ModePtr = GetActiveMode();
	if (!ModePtr)
	{
		return false;
	}

	if (!ModePtr->bFixtureMatrixEnabled)
	{
		return false;
	}

	MatrixProperties = ModePtr->FixtureMatrixConfig;

	return true;
}

bool UDMXEntityFixturePatch::GetCellAttributes(TArray<FDMXAttributeName>& CellAttributeNames)
{
	if (Cache.IsFixtureMatrix())
	{
		CellAttributeNames = Cache.GetCellAttributeNames();

		return CellAttributeNames.Num() > 0;
	}

	return false;
}

bool UDMXEntityFixturePatch::GetMatrixCell(const FIntPoint& CellCoordinate, FDMXCell& OutCell)
{
	if (Cache.IsFixtureMatrix())
	{
		const int32 DistributedCellIndex = Cache.GetDistributedCellIndex(CellCoordinate);
		if (DistributedCellIndex != INDEX_NONE)
		{
			OutCell.Coordinate = CellCoordinate;
			OutCell.CellID = DistributedCellIndex + 1;

			return true;
		}
	}
	
	return false;
}

bool UDMXEntityFixturePatch::GetAllMatrixCells(TArray<FDMXCell>& Cells)
{
	Cells.Reset();

	if (Cache.IsFixtureMatrix())
	{
		const int32 NumXCells = Cache.GetMatrixNumXCells();
		const int32 NumYCells = Cache.GetMatrixNumYCells();
		
		for (int32 CellIndexY = 0; CellIndexY < NumYCells; CellIndexY++)
		{
			for (int32 CellIndexX = 0; CellIndexX < NumXCells; CellIndexX++)
			{
				FIntPoint CellCoordinate;
				CellCoordinate.X = CellIndexX;
				CellCoordinate.Y = CellIndexY;

				const int32 DistributedCellIndex = Cache.GetDistributedCellIndex(CellIndexY * NumXCells + CellIndexX);
				if (DistributedCellIndex == INDEX_NONE)
				{
					Cells.Reset();
					return false;
				}

				FDMXCell Cell;
				Cell.Coordinate = CellCoordinate;
				Cell.CellID = DistributedCellIndex + 1;

				Cells.Add(Cell);
			}
		}

		return true;
	}

	return false;
}

void UDMXEntityFixturePatch::SendResetDataToAllAttributes(bool bUseDefaultValues)
{
	UDMXLibrary* DMXLibrary = GetParentLibrary();
	const FDMXFixtureMode* ModePtr = GetActiveMode();
	if (!DMXLibrary || !ModePtr)
	{
		return;
	}

	// Non-matrix channels
	TMap<int32, uint8> DMXChannelToValueMap;
	for (const FDMXFixtureFunction& Function : ModePtr->Functions)
	{
		const FDMXAttributeName& FunctionAttribute = Function.Attribute;

		const int32 LastFunctionChannel = Function.Channel + Function.GetNumChannels() - 1;
		const bool bLastChannelExceedsChannelSpan = LastFunctionChannel > ModePtr->ChannelSpan;
		const bool bLastChannelExceedsUniverseSize = LastFunctionChannel + GetStartingChannel() - 1 > DMX_MAX_ADDRESS;

		if (!bLastChannelExceedsChannelSpan && !bLastChannelExceedsUniverseSize)
		{
			const int32 Channel = Function.Channel + GetStartingChannel() - 1;

			uint32 Value = bUseDefaultValues ? Function.DefaultValue : 0;
			uint8* ChannelValueBytes = reinterpret_cast<uint8*>(&Value);
			UDMXEntityFixtureType::FunctionValueToBytes(Function, Value, ChannelValueBytes);

			const uint8 NumBytesInSignalFormat = FDMXConversions::GetSizeOfSignalFormat(Function.DataType);
			for (uint8 ChannelIt = 0; ChannelIt < NumBytesInSignalFormat; ++ChannelIt)
			{
				DMXChannelToValueMap.Add(Channel + ChannelIt, ChannelValueBytes[ChannelIt]);
			}
		}
	}

	// Matrix channels
	const FDMXFixtureMatrix* const FixtureMatrixPtr = GetFixtureMatrix();
	if (FixtureMatrixPtr)
	{
		TArray<FDMXCell> MatrixCells;
		GetAllMatrixCells(MatrixCells);

		for (const FDMXCell& MatrixCell : MatrixCells)
		{
			const int32 DistributedCellIndex = Cache.GetDistributedCellIndex(MatrixCell.Coordinate);
			const int32 AbsoluteMatrixStartingChannel = Cache.GetMatrixStartingChannelAbsolute();
			const int32 AbsoluteCellStartingChannel = AbsoluteMatrixStartingChannel + DistributedCellIndex * Cache.GetCellSize();

			int32 AttributeOffset = 0;
			for (const FDMXFixtureCellAttribute& CellAttribute : Cache.GetCellAttributes())
			{
				const uint32 Value = bUseDefaultValues ? CellAttribute.DefaultValue : 0;
				const TArray<uint8> ByteArray = FDMXConversions::UnsignedInt32ToByteArray(Value, CellAttribute.DataType, CellAttribute.bUseLSBMode);

				DMXChannelToValueMap.Reserve(ByteArray.Num());
				for (int32 ByteIndex = 0; ByteIndex < CellAttribute.GetNumChannels(); ByteIndex++)
				{
					DMXChannelToValueMap.Add(AbsoluteCellStartingChannel + AttributeOffset + ByteIndex, ByteArray[ByteIndex]);
				}

				AttributeOffset += CellAttribute.GetNumChannels();
			}
		}
	}

	// Send to the library's output ports
	for (const FDMXOutputPortSharedRef& OutputPort : DMXLibrary->GetOutputPorts())
	{
		OutputPort->SendDMX(UniverseID, DMXChannelToValueMap);
	}
}

void UDMXEntityFixturePatch::OnFixtureTypeChanged(const UDMXEntityFixtureType* FixtureType)
{	
	if (ParentFixtureTypeTemplate == FixtureType)
	{
		ValidateActiveMode();
		RebuildCache();

		OnFixturePatchChangedDelegate.Broadcast(this);
	}
}

const FDMXFixtureMatrix* UDMXEntityFixturePatch::GetFixtureMatrix() const
{
	if (ParentFixtureTypeTemplate)
	{
		if (const FDMXFixtureMode* ModePtr = GetActiveMode())
		{
			if (ModePtr->bFixtureMatrixEnabled)
			{
				return &ModePtr->FixtureMatrixConfig;
			}
		}
	}

	return nullptr;
}

bool UDMXEntityFixturePatch::AreCoordinatesValid(const FDMXFixtureMatrix& FixtureMatrix, const FIntPoint& Coordinate, bool bLogged)
{
	bool bValidX = Coordinate.X < FixtureMatrix.XCells && Coordinate.X >= 0;
	bool bValidY = Coordinate.Y < FixtureMatrix.YCells && Coordinate.Y >= 0;

	if (bValidX && bValidY)
	{
		return true;
	}

	if (bLogged)
	{
		UE_LOGF(DMXEntityFixturePatchLog, Error, "Invalid X Coordinate for patch (requested %d, expected in range 0-%d).", Coordinate.X, FixtureMatrix.XCells - 1);
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
