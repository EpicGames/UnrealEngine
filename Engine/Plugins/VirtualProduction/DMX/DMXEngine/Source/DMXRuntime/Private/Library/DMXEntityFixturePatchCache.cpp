// Copyright Epic Games, Inc. All Rights Reserved.

#include "Library/DMXEntityFixturePatchCache.h"
#include "Modulators/DMXModulator.h"

#include "DMXConversions.h"
#include "DMXRuntimeUtils.h"


FDMXEntityFixturePatchCache::FDMXEntityFixturePatchCache()
	: bValid(false)
	, bFixtureMatrix(false)
	, DataIndex(0)
	, DataSize(0)
	, MatrixOffset(0)
	, NumCells(0)
	, CellDataSize(0)
{}

FDMXEntityFixturePatchCache::FDMXEntityFixturePatchCache(int32 InStartingChannel, const FDMXFixtureMode* InActiveModePtr, const FDMXFixtureMatrix* InFixtureMatrixPtr)
	: bValid(true)
	, bFixtureMatrix(false)
	, DataIndex(InStartingChannel - 1)
	, DataSize(0)
	, MatrixOffset(0)
	, NumCells(0)
	, CellDataSize(0)
{
	// Not valid if the starting channel is out of data range
	if (DataIndex < 0 || DataIndex > DMX_UNIVERSE_SIZE - 1)
	{
		bValid = false;
		return;
	}

	// Not valid without a mode
	if (!InActiveModePtr)
	{
		bValid = false;
		return;
	}
	
	// Not valid when mode and matrix don't contain attributes
	if (InActiveModePtr &&
		InFixtureMatrixPtr &&
		InActiveModePtr->Functions.Num() == 0 &&
		InFixtureMatrixPtr->CellAttributes.Num() == 0)
	{
		bValid = false;
		return;
	}

	// Cache the Fixture Mode
	Mode = *InActiveModePtr;

	// Cache the Fixture Matrix
	if (InFixtureMatrixPtr &&
		InFixtureMatrixPtr->CellAttributes.Num() > 0)
	{
		FixtureMatrix = *InFixtureMatrixPtr;
		bFixtureMatrix = true;

		// Init the matrix offset
		MatrixOffset = FixtureMatrix.FirstCellChannel - 1;

		// Init num cells
		NumCells = FixtureMatrix.XCells * FixtureMatrix.YCells;

		// Init Cell Data Size
		CellDataSize = [this]()
		{
			int32 OutCellDataSize = 0;
			for (const FDMXFixtureCellAttribute& CellAttribute : FixtureMatrix.CellAttributes)
			{
				OutCellDataSize += CellAttribute.GetNumChannels();
			}
			return OutCellDataSize;
		}();

		// Initialize orderd cell indicies
		TArray<int32> UnorderedCellIndicies;
		for (int32 IndexCell = 0; IndexCell < NumCells; IndexCell++)
		{
			UnorderedCellIndicies.Add(IndexCell);
		}
		FDMXRuntimeUtils::PixelMappingDistributionSort(FixtureMatrix.PixelMappingDistribution, FixtureMatrix.XCells, FixtureMatrix.YCells, UnorderedCellIndicies, OrderedCellIndicies);
	}

	// Compute the final Data Size from Mode and Matrix
	DataSize = [this, InActiveModePtr]()
	{
		// We know functions are consecutive in front of the matrix
		int32 OutDataSize = 0;
		for (const FDMXFixtureFunction& Function : Mode.Functions)
		{
			OutDataSize += Function.GetNumChannels();
		}
		
		// Append the matrix
		if (bFixtureMatrix)
		{
			OutDataSize = FMath::Max(OutDataSize, MatrixOffset);
			OutDataSize += FixtureMatrix.XCells * FixtureMatrix.YCells * CellDataSize;
		}

		// Evaluate manual channel span
		if (!InActiveModePtr->bAutoChannelSpan)
		{
			OutDataSize = FMath::Max(OutDataSize, InActiveModePtr->ChannelSpan);
		}
		
		// Clamp DataSize to not exceed DMX_UNIVERSE_SIZE
		constexpr int32 DMXIndexMax = DMX_UNIVERSE_SIZE - 1;
		const int32 LastDataIndex = DataIndex + OutDataSize - 1;
		OutDataSize = LastDataIndex > DMXIndexMax ? OutDataSize : LastDataIndex - DataIndex + 1;

		return OutDataSize;
	}();
}

bool FDMXEntityFixturePatchCache::InputDMXSignal(const FDMXSignalSharedPtr& DMXSignal)
{
	if (bValid && DMXSignal.IsValid())
	{
		// Test if data changed
		if (CachedDMXValues.Num() < DataSize ||
			FMemory::Memcmp(CachedDMXValues.GetData(), &DMXSignal->ChannelData[DataIndex], DataSize) != 0)
		{
			// Update raw cache
			CachedDMXValues = TArray<uint8>(&DMXSignal->ChannelData[DataIndex], DataSize);

			// Update raw and normalized values per attribute cache
			CachedRawValuesPerAttribute.Reset();
			CachedNormalizedValuesPerAttribute.Map.Reset();

			for (const FDMXFixtureFunction& Function : Mode.Functions)
			{
				const int32 FunctionStartIndex = Function.Channel - 1;
				const int32 FunctionLastIndex = FunctionStartIndex + Function.GetNumChannels() - 1;
				if (FunctionLastIndex >= CachedDMXValues.Num())
				{
					break;
				}

				const uint32 IntValue = UDMXEntityFixtureType::BytesToFunctionValue(Function, CachedDMXValues.GetData() + FunctionStartIndex);
				CachedRawValuesPerAttribute.Add(Function.Attribute, IntValue);

				const float NormalizedValue = (float)IntValue / (float)FDMXConversions::GetSignalFormatMaxValue(Function.DataType);
				CachedNormalizedValuesPerAttribute.Map.Add(Function.Attribute, NormalizedValue);
			}

			// Update raw and normalized values per matrix cell cache
			CachedRawValuesPerMatrixCell.Reset();
			CachedNormalizedValuesPerMatrixCell.Reset();

			int32 CurrentCellDataIndex = MatrixOffset;
			for (int32 CellIndex = 0; CellIndex < NumCells; CellIndex++)
			{
				TMap<FDMXAttributeName, int32> CellAttributeToRawValueMap;
				CellAttributeToRawValueMap.Reserve(FixtureMatrix.CellAttributes.Num());

				FDMXNormalizedAttributeValueMap CellAttributeTNormalizedValueMap;
				CellAttributeTNormalizedValueMap.Map.Reserve(FixtureMatrix.CellAttributes.Num());

				// Add each attribute value to the map
				int32 CurrentAttributeOffset = 0;
				for (const FDMXFixtureCellAttribute& CellAttribute : FixtureMatrix.CellAttributes)
				{
					const uint8 AttributeDataSize = CellAttribute.GetNumChannels();

					const int32 FirstAttributeDataIndex = CurrentCellDataIndex + CurrentAttributeOffset;
					const int32 LastAttributeDataIndex = CurrentCellDataIndex + CurrentAttributeOffset + AttributeDataSize - 1;

					if (CachedDMXValues.IsValidIndex(FirstAttributeDataIndex) && CachedDMXValues.IsValidIndex(LastAttributeDataIndex))
					{
						const uint32 IntValue = UDMXEntityFixtureType::BytesToInt(CellAttribute.DataType, CellAttribute.bUseLSBMode, CachedDMXValues.GetData() + FirstAttributeDataIndex);
						CellAttributeToRawValueMap.Add(CellAttribute.Attribute.Name, IntValue);

						const float NormalizedValue = (float)IntValue / (float)FDMXConversions::GetSignalFormatMaxValue(CellAttribute.DataType);
						CellAttributeTNormalizedValueMap.Map.Add(CellAttribute.Attribute.Name, NormalizedValue);

						// Increment attribute offset for the next attribute
						CurrentAttributeOffset += AttributeDataSize;
					}
					else
					{
						break;
					}
				}

				CachedRawValuesPerMatrixCell.Add(MoveTemp(CellAttributeToRawValueMap));
				CachedNormalizedValuesPerMatrixCell.Add(MoveTemp(CellAttributeTNormalizedValueMap));

				// Increment the cell index for the next cell
				CurrentCellDataIndex += CellDataSize;
			}

			return true;
		}
	}

	return false;
}

void FDMXEntityFixturePatchCache::Reset()
{
	if (bValid)
	{
		CachedDMXValues.Reset();

		CachedRawValuesPerAttribute.Reset();
		CachedNormalizedValuesPerAttribute.Map.Reset();

		CachedRawValuesPerMatrixCell.Reset();
		CachedNormalizedValuesPerMatrixCell.Reset();
	}
}

void FDMXEntityFixturePatchCache::Modulate(UDMXEntityFixturePatch* FixturePatch, UDMXModulator* Modulator)
{
	if (IsValid(Modulator))
	{
		// Modulate Attributes
		Modulator->Modulate(FixturePatch, CachedNormalizedValuesPerAttribute.Map, CachedNormalizedValuesPerAttribute.Map);

		// Modulate Matrix Attributes
		if (bFixtureMatrix)
		{
			Modulator->ModulateMatrix(FixturePatch, CachedNormalizedValuesPerMatrixCell, CachedNormalizedValuesPerMatrixCell);
		}
	}
}

const int32* FDMXEntityFixturePatchCache::GetRawAttributeValue(const FDMXAttributeName& AttributeName) const
{
	if (bValid)
	{
		return CachedRawValuesPerAttribute.Find(AttributeName);
	}

	return nullptr;
}

const TMap<FDMXAttributeName, int32>* FDMXEntityFixturePatchCache::GetAllRawAttributeValues() const
{
	if (bValid)
	{
		return &CachedRawValuesPerAttribute;
	}

	return nullptr;
}

const float* FDMXEntityFixturePatchCache::GetNormalizedAttributeValue(const FDMXAttributeName& AttributeName) const
{
	if (bValid)
	{
		return CachedNormalizedValuesPerAttribute.Map.Find(AttributeName);
	}

	return nullptr;
}

const FDMXNormalizedAttributeValueMap* FDMXEntityFixturePatchCache::GetAllNormalizedAttributeValues() const
{
	if (bValid)
	{
		return &CachedNormalizedValuesPerAttribute;
	}

	return nullptr;
}

int32 FDMXEntityFixturePatchCache::GetDistributedCellIndex(const FIntPoint& CellCoordinate) const
{
	if (bFixtureMatrix)
	{
		const int32 CellIndex = FixtureMatrix.XCells * CellCoordinate.Y + CellCoordinate.X;
		const int32 DistributedCellIndex = GetDistributedCellIndex(CellIndex);

		if (OrderedCellIndicies.IsValidIndex(CellIndex))
		{
			return OrderedCellIndicies[CellIndex];
		}
	}
	return INDEX_NONE;
}

int32 FDMXEntityFixturePatchCache::GetDistributedCellIndex(int32 CellIndex) const
{
	if (bFixtureMatrix)
	{
		if (OrderedCellIndicies.IsValidIndex(CellIndex))
		{
			return OrderedCellIndicies[CellIndex];
		}
	}

	return INDEX_NONE;
}

const int32* FDMXEntityFixturePatchCache::GetRawMatrixAttributeValueFromCell(int32 CellIndex, const FDMXAttributeName& AttributeName) const
{
	if (bValid && CachedRawValuesPerMatrixCell.IsValidIndex(CellIndex))
	{
		return CachedRawValuesPerMatrixCell[CellIndex].Find(AttributeName);
	}

	return nullptr;
}

const TMap<FDMXAttributeName, int32>* FDMXEntityFixturePatchCache::GetAllRawMatrixAttributeValuesFromCell(int32 CellIndex) const
{
	if (bValid && CachedRawValuesPerMatrixCell.IsValidIndex(CellIndex))
	{
		return &CachedRawValuesPerMatrixCell[CellIndex];
	}

	return nullptr;
}

const TArray<TMap<FDMXAttributeName, int32>>* FDMXEntityFixturePatchCache::GetAllRawMatrixAttributeValues() const
{
	if (bValid)
	{
		return &CachedRawValuesPerMatrixCell;
	}

	return nullptr;
}

const float* FDMXEntityFixturePatchCache::GetNormalizedMatrixAttributeValueFromCell(int32 CellIndex, const FDMXAttributeName& AttributeName) const
{
	if (bValid && CachedNormalizedValuesPerMatrixCell.IsValidIndex(CellIndex))
	{
		return CachedNormalizedValuesPerMatrixCell[CellIndex].Map.Find(AttributeName);
	}

	return nullptr;
}

const FDMXNormalizedAttributeValueMap* FDMXEntityFixturePatchCache::GetAllNormalizedMatrixAttributeValuesFromCell(int32 CellIndex) const
{
	if (bValid && CachedNormalizedValuesPerMatrixCell.IsValidIndex(CellIndex))
	{
		return &CachedNormalizedValuesPerMatrixCell[CellIndex];
	}

	return nullptr;
}

const TArray<FDMXNormalizedAttributeValueMap>* FDMXEntityFixturePatchCache::GetAllNormalizedMatrixAttributeValues() const
{
	if (bValid)
	{
		return &CachedNormalizedValuesPerMatrixCell;
	}

	return nullptr;
}

TArray<FDMXAttributeName> FDMXEntityFixturePatchCache::GetAttributeNames() const
{
	TArray<FDMXAttributeName> AttributeNames;
	if (bValid)
	{
		for (const FDMXFixtureFunction& Function : Mode.Functions)
		{
			AttributeNames.Add(Function.Attribute.Name);
		}
	}

	return AttributeNames;
}

TArray<FDMXAttributeName> FDMXEntityFixturePatchCache::GetCellAttributeNames() const
{
	TArray<FDMXAttributeName> MatrixAttributeNames;
	if (bValid)
	{
		MatrixAttributeNames.Reserve(FixtureMatrix.CellAttributes.Num());

		for (const FDMXFixtureCellAttribute& CellAttribute : FixtureMatrix.CellAttributes)
		{
			MatrixAttributeNames.Add(CellAttribute.Attribute.Name);
		}
	}

	return MatrixAttributeNames;
}
