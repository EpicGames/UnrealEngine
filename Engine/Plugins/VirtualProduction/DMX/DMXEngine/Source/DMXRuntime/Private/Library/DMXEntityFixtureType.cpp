// Copyright Epic Games, Inc. All Rights Reserved.

#include "Library/DMXEntityFixtureType.h"

#include "DMXAttributeToDefaultPhyiscalProperties.h"
#include "DMXConversions.h"
#include "DMXProtocolSettings.h"
#include "DMXRuntimeMainStreamObjectVersion.h"
#include "DMXRuntimeUtils.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXGDTFAssetImportData.h"
#include "Library/DMXImport.h"
#include "Library/DMXImportGDTF.h"
#include "Library/DMXLibrary.h"
#include "Misc/EngineVersion.h"
#include "Misc/Paths.h"

#define LOCTEXT_NAMESPACE "DMXEntityFixtureType"

void FDMXFixtureFunction::PostSerialize(const FArchive& Ar)
{
#if WITH_EDITOR
	if (Ar.IsLoading())
	{
		UpdatePhysicalDefaultValue();

		// Upgrade Fixture Functions that were added before physical properties were added to use these properties
		if (Ar.CustomVer(FDMXRuntimeMainStreamObjectVersion::GUID) < FDMXRuntimeMainStreamObjectVersion::DMXUpgradeFixtureFunctionsToUsePhysicalProperties)
		{
			using namespace UE::DMX;
			FDMXAttributeToDefaultPhyiscalProperties::ResetToDefaultPhysicalProperties(*this);
		}
	}
#endif
}

int32 FDMXFixtureFunction::GetLastChannel() const
{
	return Channel + GetNumChannels() - 1;
}

#if WITH_EDITOR
void FDMXFixtureFunction::SetPhysicalDefaultValue(double InPhysicalDefaultValue)
{
	const double PhysicalMin = FMath::Min(PhysicalFrom, PhysicalTo);
	const double PhysicalMax = FMath::Max(PhysicalFrom, PhysicalTo);

	PhysicalDefaultValue = FMath::Clamp(InPhysicalDefaultValue, PhysicalMin, PhysicalMax);

	const double PhysicalValueRange = PhysicalTo > PhysicalFrom ? PhysicalTo - PhysicalFrom : PhysicalFrom - PhysicalTo;
	const double RelativePhysicalDefaultValue = PhysicalDefaultValue > PhysicalFrom ? PhysicalDefaultValue - PhysicalFrom : PhysicalFrom - PhysicalDefaultValue;
	const double NormalizedPhysicalDefaultValue = FMath::IsNearlyZero(PhysicalValueRange) ? 0.0 : RelativePhysicalDefaultValue / PhysicalValueRange;

	const uint32 MaxValue = FDMXConversions::GetSignalFormatMaxValue(DataType);
	DefaultValue = FMath::Lerp(0, MaxValue, NormalizedPhysicalDefaultValue);
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void FDMXFixtureFunction::SetPhysicalValueRange(double InPhysicalFrom, double InPhysicalTo)
{
	const double PhysicalValueRange = PhysicalTo > PhysicalFrom ? PhysicalTo - PhysicalFrom : PhysicalFrom - PhysicalTo;
	const double RelativePhysicalDefaultValue = PhysicalDefaultValue > PhysicalFrom ? PhysicalDefaultValue - PhysicalFrom : PhysicalFrom - PhysicalDefaultValue;
	const double NormalizedPhysicalDefaultValue = FMath::IsNearlyZero(PhysicalValueRange) ? 0.0 : RelativePhysicalDefaultValue / PhysicalValueRange;

	PhysicalFrom = InPhysicalFrom;
	PhysicalTo = InPhysicalTo;

	const double NewPhysicalDefaultValue = FMath::Lerp(PhysicalFrom, PhysicalTo, NormalizedPhysicalDefaultValue);
	
	// Set both the default and the physical default value
	SetPhysicalDefaultValue(NewPhysicalDefaultValue);
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void FDMXFixtureFunction::UpdatePhysicalDefaultValue()
{
	const uint32 MaxValue = FDMXConversions::GetSignalFormatMaxValue(DataType);
	const double NormalizedDefaultValue = static_cast<double>(DefaultValue) / MaxValue;

	const double NewPhysicalValue = FMath::Lerp(PhysicalFrom, PhysicalTo, NormalizedDefaultValue);

	constexpr double Tolerance = 0.01;
	const double RoundedPhysicalValue = FMath::RoundToDouble(NewPhysicalValue / Tolerance) * Tolerance;

	PhysicalDefaultValue = RoundedPhysicalValue;
}
#endif // WITH_EDITOR

FDMXFixtureMatrix::FDMXFixtureMatrix()
{
	// Add a default cell attribute	
	FDMXFixtureCellAttribute RedCellAttribute;
	RedCellAttribute.Attribute = FName("Red");
	CellAttributes = { RedCellAttribute };
}

int32 FDMXFixtureMatrix::GetNumChannels() const
{
	int32 NumChannelsPerCell = 0;
	for (const FDMXFixtureCellAttribute& CellAttribute : CellAttributes)
	{
		NumChannelsPerCell += CellAttribute.GetNumChannels();
	}

	return XCells * YCells * NumChannelsPerCell;
}

int32 FDMXFixtureMatrix::GetLastChannel() const
{
	const int32 NumCells = XCells * YCells;
	if (NumCells == 0)
	{
		return FirstCellChannel;
	}

	int32 NumChannelsPerCell = 0;
	for (const FDMXFixtureCellAttribute& Attribute : CellAttributes)
	{
		NumChannelsPerCell += Attribute.GetNumChannels();
	}

	return FirstCellChannel + NumCells * NumChannelsPerCell - 1;
}

#if WITH_EDITOR
int32 FDMXFixtureMode::AddOrInsertFunction(int32 IndexOfFunction, FDMXFixtureFunction InFunction)
{
	// DEPRECATED 5.0
	int32 Index = 0;
	FDMXFixtureFunction FunctionToAdd = InFunction;

	// Shift the insert function channel
	uint8 DataTypeBytes = FDMXConversions::GetSizeOfSignalFormat(InFunction.DataType);
	FunctionToAdd.Channel = InFunction.Channel + DataTypeBytes;

	if (Functions.IsValidIndex(IndexOfFunction + 1))
	{
		Index = Functions.Insert(InFunction, IndexOfFunction + 1);

		// Shift all function after this by size of insert function
		for (int32 FunctionIndex = Index + 1; FunctionIndex < Functions.Num(); FunctionIndex++)
		{
			FDMXFixtureFunction& Function = Functions[FunctionIndex];
			Function.Channel = Function.Channel + DataTypeBytes;
		}
	}
	else
	{
		InFunction.Channel = Functions.Num() > 0 ? Functions.Last().Channel + Functions.Last().GetNumChannels() : 1;
		Index = Functions.Add(InFunction);
	}

	return Index;
}
#endif // WITH_EDITOR


FDMXOnFixtureTypeChangedDelegate UDMXEntityFixtureType::OnFixtureTypeChangedDelegate;

UDMXEntityFixtureType::UDMXEntityFixtureType()
{}

UDMXEntityFixtureType* UDMXEntityFixtureType::CreateFixtureTypeInLibrary(FDMXEntityFixtureTypeConstructionParams ConstructionParams, const FString& DesiredName, bool bMarkDMXLibraryDirty)
{
	UDMXLibrary* ParentDMXLibrary = ConstructionParams.ParentDMXLibrary;
	if (ensureMsgf(IsValid(ConstructionParams.ParentDMXLibrary), TEXT("Create New Fixture Type cannot create Fixture Type when Parent Library is null.")))
	{
#if WITH_EDITOR
		if (bMarkDMXLibraryDirty)
		{
			ParentDMXLibrary->Modify();
			ParentDMXLibrary->PreEditChange(UDMXLibrary::StaticClass()->FindPropertyByName(UDMXLibrary::GetEntitiesPropertyName()));
		}
#endif

		FString EntityName = FDMXRuntimeUtils::FindUniqueEntityName(ParentDMXLibrary, UDMXEntityFixtureType::StaticClass(), DesiredName);

		UDMXEntityFixtureType* NewFixtureType = NewObject<UDMXEntityFixtureType>(ParentDMXLibrary, UDMXEntityFixtureType::StaticClass(), NAME_None, RF_Transactional);
		NewFixtureType->SetName(EntityName);
		NewFixtureType->DMXCategory = ConstructionParams.DMXCategory;
		NewFixtureType->Modes = ConstructionParams.Modes;

		// Update the channel span for each mode
		for (int32 ModeIndex = 0; ModeIndex < NewFixtureType->Modes.Num(); ModeIndex++)
		{
			NewFixtureType->UpdateChannelSpan(ModeIndex);
		}


#if WITH_EDITOR
		if (bMarkDMXLibraryDirty)
		{
			ParentDMXLibrary->PostEditChange();
		}
#endif
		OnFixtureTypeChangedDelegate.Broadcast(NewFixtureType);

		return NewFixtureType;
	}

	return nullptr;
}

void UDMXEntityFixtureType::RemoveFixtureTypeFromLibrary(FDMXEntityFixtureTypeRef FixtureTypeRef)
{
	if (UDMXEntityFixtureType* FixtureType = FixtureTypeRef.GetFixtureType())
	{
		if (UDMXLibrary* DMXLibrary = FixtureType->GetParentLibrary())
		{
			const TArray<UDMXEntityFixturePatch*> FixturePatches = DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>();
			for (UDMXEntityFixturePatch* FixturePatch : FixturePatches)
			{
				if (FixturePatch->GetFixtureType() == FixtureType)
				{
					FixturePatch->SetFixtureType(nullptr);
				}
			}

			DMXLibrary->Modify();
			FixtureType->Modify();
			FixtureType->Destroy();
		}
	}
}

void UDMXEntityFixtureType::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FDMXRuntimeMainStreamObjectVersion::GUID);
	
#if WITH_EDITOR
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FDMXRuntimeMainStreamObjectVersion::GUID) < FDMXRuntimeMainStreamObjectVersion::DMXFixtureTypeAllowMatrixInEachFixtureMode)
		{
			// For assets that were created before each mode could enable or disable the matrix, copy the deprecated bFixtureMatrixEnabled property to each mode
			for (FDMXFixtureMode& Mode : Modes)
			{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				Mode.bFixtureMatrixEnabled = bFixtureMatrixEnabled_DEPRECATED;
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
		}

		// Updgrade to use a soft object ptr for GDTF
		if (Ar.CustomVer(FDMXRuntimeMainStreamObjectVersion::GUID) < FDMXRuntimeMainStreamObjectVersion::DMXImportGDTFIsASoftObjectPtr)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (DMXImport && DMXImport->GetClass() == UDMXImportGDTF::StaticClass())
			{
				UDMXImportGDTF* DMXImportGDTF = CastChecked<UDMXImportGDTF>(DMXImport);
				GDTFSource = DMXImportGDTF;
				DMXImport = nullptr;
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR
bool UDMXEntityFixtureType::Modify(bool bAlwaysMarkDirty)
{
	if (UDMXLibrary* DMXLibrary = ParentLibrary.Get())
	{
		return DMXLibrary->Modify(bAlwaysMarkDirty) && Super::Modify(bAlwaysMarkDirty);
	}

	return Super::Modify(bAlwaysMarkDirty);
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UDMXEntityFixtureType::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		// Update the Channel Span for all Modes
		for (int32 ModeIndex = 0; ModeIndex < Modes.Num(); ModeIndex++)
		{
			AlignFunctionChannels(ModeIndex);
			UpdateChannelSpan(ModeIndex);
		}

		OnFixtureTypeChangedDelegate.Broadcast(this);
	}
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UDMXEntityFixtureType::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedChainEvent);

	const FName PropertyName = PropertyChangedChainEvent.GetPropertyName();

	if (PropertyChangedChainEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		// Update the Channel Span for all Modes
		for (int32 ModeIndex = 0; ModeIndex < Modes.Num(); ModeIndex++)
		{
			AlignFunctionChannels(ModeIndex);
			UpdateChannelSpan(ModeIndex);
		}

		OnFixtureTypeChangedDelegate.Broadcast(this);
	}
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UDMXEntityFixtureType::PostEditUndo()
{
	Super::PostEditUndo();

	OnFixtureTypeChangedDelegate.Broadcast(this);
}
#endif // WITH_EDITOR

#if WITH_EDITOR
FString UDMXEntityFixtureType::GetCleanGDTFFileNameSynchronous(bool bWithExtension) const
{
	if (GDTFSource.IsValid())
	{				
		// Return the GDTF source filename if a GDTF is set
		UDMXImportGDTF* GDTF = GDTFSource.LoadSynchronous();

		const FString GDTFFilename = FPaths::GetBaseFilename(GDTF->GetGDTFAssetImportData()->GetFilePathAndName());
		const FString Extension = bWithExtension ? TEXT(".gdtf") : TEXT("");

		return GDTFFilename + Extension;
	}
	else
	{		
		// Generate a filename if no GDTF is set
		const FString EngineVersion = FString::Printf(TEXT("%u_%u"), FEngineVersion::Current().GetMajor(), FEngineVersion::Current().GetMinor());
		const FString DateTime = FDateTime::Now().ToString(TEXT("%d_%m_%y"));
		const FString GeneratedGDTFFilename = FString::Printf(TEXT("EpicGames@UE%s_Generated_%s@%s"), *EngineVersion, *Name, *DateTime);
		const FString Extension = bWithExtension ? TEXT(".gdtf") : TEXT("");

		return GeneratedGDTFFilename + Extension;
	}
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UDMXEntityFixtureType::SetModesFromDMXImport(UDMXImport* DMXImportAsset)
{
	// DEPRECATED 5.4

	if (!IsValid(DMXImportAsset))
	{
		return;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	GDTFSource = DMXImportAsset;
	if (UDMXImportGDTFDMXModes* GDTFDMXModes = Cast<UDMXImportGDTFDMXModes>(GDTFSource->DMXModes_DEPRECATED))
	{
		// Clear existing modes
		Modes.Empty(GDTFSource->DMXModes_DEPRECATED != nullptr ? GDTFDMXModes->DMXModes.Num() : 0);

		// Used to map Functions to Attributes
		const UDMXProtocolSettings* ProtocolSettings = GetDefault<UDMXProtocolSettings>();
		// Break the Attributes' keywords into arrays of strings to be read for each Function
		TMap< FName, TArray<FString> > AttributesKeywords;
		for (const FDMXAttribute& Attribute : ProtocolSettings->Attributes)
		{
			TArray<FString> Keywords = Attribute.GetKeywords();
			
			AttributesKeywords.Emplace(Attribute.Name, MoveTemp(Keywords));
		}

		// Copy modes from asset
		for (const FDMXImportGDTFDMXMode& AssetMode : GDTFDMXModes->DMXModes)
		{
			FDMXFixtureMode& Mode = Modes[Modes.Emplace()];
			Mode.ModeName = AssetMode.Name.ToString();

			// Keep track of the Attributes used on this Mode's Functions because they must be unique
			TArray<FName> MappedAttributes;

			// Get a unique name
			TMap<FString, uint32> PotentialFunctionNamesAndCount;
			for (const FDMXImportGDTFDMXChannel& ModeChannel : AssetMode.DMXChannels)
			{
				FString FunctionName = ModeChannel.LogicalChannel.Attribute.Name.ToString();

				PotentialFunctionNamesAndCount.Add(FunctionName, 0);
			}

			int32 FunctionStartingChannel = 1;
			for (const FDMXImportGDTFDMXChannel& ModeChannel : AssetMode.DMXChannels)
			{
				FDMXFixtureFunction& Function = Mode.Functions[Mode.Functions.Emplace()];
				Function.FunctionName = FDMXRuntimeUtils::GenerateUniqueNameForImportFunction(PotentialFunctionNamesAndCount, ModeChannel.LogicalChannel.Attribute.Name.ToString());
				Function.DefaultValue = ModeChannel.Default.Value;
				Function.Channel = FunctionStartingChannel;

				// Try to auto-map the Function to an existing Attribute
				// using the Function's name and the Attributes' keywords
				if (!Function.FunctionName.IsEmpty() && AttributesKeywords.Num())
				{
					// Remove white spaces and index numbers from the name
					FString FilteredFunctionName;
					int32 IndexFromName;
					FDMXRuntimeUtils::GetNameAndIndexFromString(Function.FunctionName, FilteredFunctionName, IndexFromName);

					// Check if the Function name matches any Attribute's keywords
					for (const TPair< FName, TArray<FString> >& Keywords : AttributesKeywords)
					{
						if (MappedAttributes.Contains(Keywords.Key))
						{
							continue; // Attribute already mapped to another Function in this Mode
						}

						auto CompareStringCaseInsensitive = [&FilteredFunctionName](const FString& Keyword)
						{
							return Keyword.Equals(FilteredFunctionName, ESearchCase::IgnoreCase);
						};

						// Match the Function name against the Attribute name and its keywords
						if (CompareStringCaseInsensitive(Keywords.Key.ToString())
							|| Keywords.Value.ContainsByPredicate(CompareStringCaseInsensitive))
						{
							Function.Attribute = Keywords.Key;
							MappedAttributes.Emplace(Keywords.Key); // Mark Attribute as used in this Mode
						}
					}
				}

				// Calculate Function's number of occupied channels/addresses
				if (ModeChannel.Offset.Num() > 0)
				{
					// Compute number of used addresses in the function as the interval
					// between the lowest and highest addresses (inclusive)
					int32 AddressMin = DMX_MAX_ADDRESS;
					int32 AddressMax = 0;
					for (const int32& Address : ModeChannel.Offset)
					{
						AddressMin = FMath::Min(AddressMin, Address);
						AddressMax = FMath::Max(AddressMax, Address);
					}
					const int32 NumUsedAddresses = FMath::Clamp(AddressMax - AddressMin + 1, 1, DMX_MAX_FUNCTION_SIZE);
					FunctionStartingChannel += NumUsedAddresses;

					Function.DataType = static_cast<EDMXFixtureSignalFormat>(NumUsedAddresses - 1);

					// Offsets represent the value bytes in MSB format. If they are in reverse order,
					// it means this Function uses LSB format.
					// We need at least 2 offsets to compare. Otherwise, we leave the function as MSB,
					// which is most Fixtures' standard bit format.
					if (ModeChannel.Offset.Num() > 1)
					{
						Function.bUseLSBMode = ModeChannel.Offset[0] > ModeChannel.Offset[1];
					}
					else
					{
						Function.bUseLSBMode = false;
					}
				}
				else
				{
					FunctionStartingChannel += 1;

					Function.DataType = EDMXFixtureSignalFormat::E8Bit;
				}
			}
		}

		// Update the Channel Span for all Modes
		for (int32 ModeIndex = 0; ModeIndex < Modes.Num(); ModeIndex++)
		{
			UpdateChannelSpan(ModeIndex);
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	OnFixtureTypeChangedDelegate.Broadcast(this);
}
#endif // WITH_EDITOR

FDMXOnFixtureTypeChangedDelegate& UDMXEntityFixtureType::GetOnFixtureTypeChanged()
{
	return OnFixtureTypeChangedDelegate;
}

int32 UDMXEntityFixtureType::AddMode(FString BaseModeName)
{
	FDMXFixtureMode NewMode;

	// Make a unique name for the new mode
	TSet<FString> ModeNames;
	for (const FDMXFixtureMode& Mode : Modes)
	{
		ModeNames.Add(Mode.ModeName);
	}
	NewMode.ModeName = FDMXRuntimeUtils::GenerateUniqueNameFromExisting(ModeNames, BaseModeName);

	const int32 NewModeIndex = Modes.Add(NewMode);

	return NewModeIndex;
}

void UDMXEntityFixtureType::DuplicateModes(TArray<int32> InModeÎndiciesToDuplicate, TArray<int32>& OutNewModeIndices)
{
	int32 NumModesDuplicated = 0;
	for (int32 ModeIndex : InModeÎndiciesToDuplicate)
	{
		if (Modes.IsValidIndex(ModeIndex))
		{
			// Copy
			FDMXFixtureMode NewMode = Modes[ModeIndex];

			int32 IndexOfDuplicate = ModeIndex + 1 + NumModesDuplicated;
			if (Modes.IsValidIndex(IndexOfDuplicate))
			{
				IndexOfDuplicate = Modes.Insert(NewMode, IndexOfDuplicate);
			}
			else
			{
				IndexOfDuplicate = Modes.Add(NewMode);
			}

			OutNewModeIndices.Add(IndexOfDuplicate);

			// Make a uinque mode name by setting it
			FString UnusedString;
			SetModeName(IndexOfDuplicate, NewMode.ModeName, UnusedString);

			NumModesDuplicated++;
		}
	}
}

void UDMXEntityFixtureType::RemoveModes(const TArray<int32>& ModeIndicesToDelete)
{
	int32 NumDeletedModes = 0;
	for (const int32 ModeIndex : ModeIndicesToDelete)
	{
		const int32 DeleteAtIndex = ModeIndex - NumDeletedModes;
		if (Modes.IsValidIndex(DeleteAtIndex))
		{
			Modes.RemoveAt(DeleteAtIndex);
			NumDeletedModes++;
		}
	}
}

void UDMXEntityFixtureType::SetModeName(int32 InModeIndex, const FString& InDesiredModeName, FString& OutUniqueModeName)
{
	if (ensureMsgf(Modes.IsValidIndex(InModeIndex), TEXT("Invalid Mode Index when setting Mode Name.")))
	{
		FDMXFixtureMode& Mode = Modes[InModeIndex];

		FString OldName = Mode.ModeName;

		const FString UniqueModeNameString = [InModeIndex, InDesiredModeName, this]()
		{
			TSet<FString> OtherModeNames;
			for (int32 OtherModeIndex = 0; OtherModeIndex < Modes.Num(); OtherModeIndex++)
			{
				if (OtherModeIndex != InModeIndex)
				{
					OtherModeNames.Add(Modes[OtherModeIndex].ModeName);
				}
			}
			return FDMXRuntimeUtils::GenerateUniqueNameFromExisting(OtherModeNames, InDesiredModeName);
		}();

		Mode.ModeName = UniqueModeNameString;
		OutUniqueModeName = UniqueModeNameString;
	}
}

void UDMXEntityFixtureType::SetFixtureMatrixEnabled(int32 ModeIndex, bool bEnableMatrix)
{
	if (ensureMsgf(Modes.IsValidIndex(ModeIndex), TEXT("Trying Update Channel Span, but Mode Index is not valid.")))
	{
		FDMXFixtureMode& Mode = Modes[ModeIndex];
		FDMXFixtureMatrix& Matrix = Mode.FixtureMatrixConfig;

		if (bEnableMatrix != Mode.bFixtureMatrixEnabled)
		{
			Mode.bFixtureMatrixEnabled = bEnableMatrix;

			// Some old assets may have a 0x0 matrix stored, but we expect it to be always at least one cell.
			Mode.FixtureMatrixConfig.XCells = FMath::Max(Mode.FixtureMatrixConfig.XCells, 1);
			Mode.FixtureMatrixConfig.YCells = FMath::Max(Mode.FixtureMatrixConfig.YCells, 1);

			AlignFunctionChannels(ModeIndex);
			UpdateChannelSpan(ModeIndex);
		}
	}
}

void UDMXEntityFixtureType::UpdateChannelSpan(int32 ModeIndex)
{
	if (ensureMsgf(Modes.IsValidIndex(ModeIndex), TEXT("Trying Update Channel Span, but Mode Index is not valid.")))
	{
		FDMXFixtureMode& Mode = Modes[ModeIndex];

		if (Mode.bAutoChannelSpan)
		{
			int32 LowestChannel = TNumericLimits<int32>::Max();
			int32 HighestChannel = TNumericLimits<int32>::Min();

			for (const FDMXFixtureFunction& Function : Mode.Functions)
			{
				const int32 FirstChannelOfFunction = Function.Channel;
				LowestChannel = FirstChannelOfFunction < LowestChannel ? FirstChannelOfFunction : LowestChannel;

				const int32 LastChannelOfFunction = FirstChannelOfFunction + Function.GetNumChannels() - 1;
				HighestChannel = LastChannelOfFunction > HighestChannel ? LastChannelOfFunction : HighestChannel;
			}

			const int32 FirstChannelOfMatrix = Mode.FixtureMatrixConfig.FirstCellChannel;
			LowestChannel = FirstChannelOfMatrix < LowestChannel ? FirstChannelOfMatrix : LowestChannel;

			const int32 LastChannelOfMatrix = FirstChannelOfMatrix + Mode.FixtureMatrixConfig.GetNumChannels() - 1;
			HighestChannel = LastChannelOfMatrix > HighestChannel ? LastChannelOfMatrix : HighestChannel;

			Mode.ChannelSpan = FMath::Max(HighestChannel - LowestChannel + 1, 0);
		}
	}
}

void UDMXEntityFixtureType::AlignFunctionChannels(int32 ModeIndex)
{
	if (ensureMsgf(Modes.IsValidIndex(ModeIndex), TEXT("Invalid Mode Index when aligning the Channels of all Functions in a Mode.")))
	{
		FDMXFixtureMode& Mode = Modes[ModeIndex];

		// Align functions and matrix
		int32 NextFreeChannel = 1;
		bool bHandledMatrix = !Mode.bFixtureMatrixEnabled;
		for (FDMXFixtureFunction& Function : Mode.Functions)
		{
			if (!bHandledMatrix && 
				(Mode.FixtureMatrixConfig.FirstCellChannel <= NextFreeChannel || Mode.FixtureMatrixConfig.FirstCellChannel <= Function.Channel))
			{
				Mode.FixtureMatrixConfig.FirstCellChannel = NextFreeChannel;
				NextFreeChannel = NextFreeChannel + Mode.FixtureMatrixConfig.GetNumChannels();
				bHandledMatrix = true;
			}

			Function.Channel = NextFreeChannel;

			// Don't assign past channel 512
			NextFreeChannel = Function.GetLastChannel() + 1;
		}
	}
}

int32 UDMXEntityFixtureType::AddFunction(int32 InModeIndex)
{
	if (ensureMsgf(Modes.IsValidIndex(InModeIndex), TEXT("Invalid Mode Index when setting Mode Name.")))
	{
		FDMXFixtureMode& Mode = Modes[InModeIndex];
		FDMXFixtureFunction NewFunction;

		// Set a unique Name
		TSet<FString> FunctionNames;
		for (const FDMXFixtureFunction& Function : Mode.Functions)
		{
			FunctionNames.Add(Function.FunctionName);
		}
		NewFunction.FunctionName = FDMXRuntimeUtils::GenerateUniqueNameFromExisting(FunctionNames, LOCTEXT("DMXFixtureTypeSharedData.DefaultFunctionName", "Function").ToString());

		// Update the Channel
		NewFunction.Channel = [Mode]()
		{
			const int32 LastFunctionChannel = Mode.Functions.Num() > 0 ? Mode.Functions.Last().GetLastChannel() : 0;
			const int32 LastMatrixChannel = Mode.bFixtureMatrixEnabled ? Mode.FixtureMatrixConfig.GetLastChannel() : 0;
			return FMath::Max(LastFunctionChannel, LastMatrixChannel) + 1;
		}();

		// Add the Function and update Channel Span
		const int32 NewFunctionIndex = Mode.Functions.Add(NewFunction);
		UpdateChannelSpan(InModeIndex);

		return NewFunctionIndex;
	}

	return INDEX_NONE;
}

int32 UDMXEntityFixtureType::InsertFunction(int32 InModeIndex, int32 InInsertAtIndex, FDMXFixtureFunction& InOutNewFunction)
{
	if (ensureMsgf(Modes.IsValidIndex(InModeIndex), TEXT("Invalid Mode Index when setting Mode Name.")))
	{
		FDMXFixtureMode& Mode = Modes[InModeIndex];

		// Set a unique name
		TSet<FString> FunctionNames;
		for (const FDMXFixtureFunction& Function : Mode.Functions)
		{
			FunctionNames.Add(Function.FunctionName);
		}
		InOutNewFunction.FunctionName = FDMXRuntimeUtils::GenerateUniqueNameFromExisting(FunctionNames, InOutNewFunction.FunctionName);

		int32 NewFunctionIndex = INDEX_NONE;
		if(Mode.Functions.IsValidIndex(InInsertAtIndex))
		{
			// Add the Function, then reorder it to where it should reside
			InOutNewFunction.Channel = TNumericLimits<int32>::Max();
			const int32 TempFunctionIndex = Mode.Functions.Add(InOutNewFunction);

			ReorderFunction(InModeIndex, TempFunctionIndex, InInsertAtIndex);
			NewFunctionIndex = InInsertAtIndex;
		}
		else
		{
			// Add the Function after either the last Function or the last Matrix channel
			InOutNewFunction.Channel = [Mode]()
			{
				const int32 LastFunctionChannel = Mode.Functions.Num() > 0 ? Mode.Functions.Last().GetLastChannel() : 0;
				const int32 LastMatrixChannel = Mode.bFixtureMatrixEnabled ? Mode.FixtureMatrixConfig.GetLastChannel() : 0;
				return FMath::Max(LastFunctionChannel, LastMatrixChannel) + 1;
			}();

			NewFunctionIndex = Mode.Functions.Add(InOutNewFunction);
		}

		UpdateChannelSpan(InModeIndex);

		check(NewFunctionIndex != INDEX_NONE);
		return NewFunctionIndex;
	}

	return INDEX_NONE;
}

void UDMXEntityFixtureType::DuplicateFunctions(int32 InModeIndex, const TArray<int32>& InFunctionIndicesToDuplicate, TArray<int32>& OutNewFunctionIndices)
{
	if (Modes.IsValidIndex(InModeIndex))
	{
		FDMXFixtureMode& Mode = Modes[InModeIndex];

		int32 NumDuplicatedFunctions = 0;
		for (const int32 FunctionToDuplicateIndex : InFunctionIndicesToDuplicate)
		{
			if (ensureMsgf(Mode.Functions.IsValidIndex(FunctionToDuplicateIndex), TEXT("Trying to duplicate Function, but Function index is not valid.")))
			{
				FDMXFixtureFunction DuplicatedFunction = Mode.Functions[FunctionToDuplicateIndex];

				// Offset subsequent functions and possibly the matrix 
				DuplicatedFunction.Channel = DuplicatedFunction.GetLastChannel() + 1;
				const int32 Offset = DuplicatedFunction.GetNumChannels();
				for (FDMXFixtureFunction& Function : Mode.Functions)
				{
					if (Function.Channel >= DuplicatedFunction.Channel)
					{
						Function.Channel += Offset;
					}
				}
				if (Mode.FixtureMatrixConfig.FirstCellChannel >= DuplicatedFunction.Channel)
				{
					Mode.FixtureMatrixConfig.FirstCellChannel += Offset;
				}

				// Add the Function
				int32 IndexOfNewlyAddedFunction = FunctionToDuplicateIndex + 1 + NumDuplicatedFunctions;
				if (Mode.Functions.IsValidIndex(IndexOfNewlyAddedFunction))
				{
					IndexOfNewlyAddedFunction = Mode.Functions.Insert(DuplicatedFunction, IndexOfNewlyAddedFunction);
				}
				else
				{
					IndexOfNewlyAddedFunction = Mode.Functions.Add(DuplicatedFunction);
				}
				OutNewFunctionIndices.Add(IndexOfNewlyAddedFunction);

				// Make a uinque Function Name by setting it
				FString UnusedString;
				SetFunctionName(InModeIndex, IndexOfNewlyAddedFunction, DuplicatedFunction.FunctionName, UnusedString);

				NumDuplicatedFunctions++;
			}
		}

		if (NumDuplicatedFunctions > 0)
		{
			UpdateChannelSpan(InModeIndex);
		}
	}		
}

void UDMXEntityFixtureType::RemoveFunctions(int32 ModeIndex, TArray<int32> FunctionIndicesToDelete)
{
	if (ensureMsgf(Modes.IsValidIndex(ModeIndex), TEXT("Trying to remove Function, but Mode index is invalid.")))
	{
		FDMXFixtureMode& Mode = Modes[ModeIndex];

		// Sort to remove from last to first index
		FunctionIndicesToDelete.Sort([](int32 IndexToDeleteA, int32 IndexToDeleteB)
			{
				return IndexToDeleteA > IndexToDeleteB;
			});

		for (const int32 FunctionToRemoveIndex : FunctionIndicesToDelete)
		{
			if (ensureMsgf(Mode.Functions.IsValidIndex(FunctionToRemoveIndex), TEXT("Trying to remove Function, but Function index is invalid.")))
			{
				const FDMXFixtureFunction& FunctionToRemove = Mode.Functions[FunctionToRemoveIndex];
				const int32 Offset = FunctionToRemove.GetNumChannels();

				// Realign subsequent functions
				const int32 RealignAtIndex = FunctionToRemoveIndex + 1;
				if (Mode.Functions.IsValidIndex(RealignAtIndex))
				{
					for (int32 FunctionToReorderIndex = RealignAtIndex; FunctionToReorderIndex < Mode.Functions.Num(); FunctionToReorderIndex++)
					{
						Mode.Functions[FunctionToReorderIndex].Channel -= Offset;
					}
				}

				// Align the Matrix
				if (Mode.bFixtureMatrixEnabled)
				{
					const int32 LastRemovedFunctionChannel = FunctionToRemove.GetLastChannel();
					const int32 FirstMatrixChannel = Mode.FixtureMatrixConfig.FirstCellChannel;

					if (FirstMatrixChannel >= LastRemovedFunctionChannel)
					{
						Mode.FixtureMatrixConfig.FirstCellChannel -= Offset;
					}
				}

				Mode.Functions.RemoveAt(FunctionToRemoveIndex);
			}
		}

		UpdateChannelSpan(ModeIndex);
	}
}

void UDMXEntityFixtureType::ReorderFunction(int32 ModeIndex, int32 FunctionToReorderIndex, int32 InsertAtIndex)
{
	if (ensureMsgf(Modes.IsValidIndex(ModeIndex), TEXT("Trying to reorder Function, but Mode Index is not valid.")))
	{
		FDMXFixtureMode& Mode = Modes[ModeIndex];
		FDMXFixtureMatrix& Matrix = Mode.FixtureMatrixConfig;

		if (ensureMsgf(Mode.Functions.IsValidIndex(FunctionToReorderIndex) && Mode.Functions.IsValidIndex(InsertAtIndex), TEXT("Trying to reorder Function, but Function Indices are not valid.")))
		{
			if (FunctionToReorderIndex != InsertAtIndex)
			{
				FDMXFixtureFunction FunctionToReorder = Mode.Functions[FunctionToReorderIndex];
				const FDMXFixtureFunction& InsertAtFunction = Mode.Functions[InsertAtIndex];

				const bool bForward = FunctionToReorderIndex < InsertAtIndex;
				const int32 Offset = bForward ? -FunctionToReorder.GetNumChannels() : FunctionToReorder.GetNumChannels();

				// Align Matrix if enabled
				if (Mode.bFixtureMatrixEnabled)
				{
					const bool bInRangeOfMatrixStart = bForward ?
						Matrix.FirstCellChannel >= FunctionToReorder.Channel :
						Matrix.FirstCellChannel >= InsertAtFunction.Channel;

					const bool bInRangeOfMatrixEnd = bForward ?
						Matrix.FirstCellChannel <= InsertAtFunction.GetLastChannel() :
						Matrix.FirstCellChannel <= FunctionToReorder.GetLastChannel();

					if (bInRangeOfMatrixStart && bInRangeOfMatrixEnd)
					{
						Matrix.FirstCellChannel += Offset;
					}
				}

				// Set the Function Channel
				FunctionToReorder.Channel = bForward ? InsertAtFunction.Channel + InsertAtFunction.GetNumChannels() - FunctionToReorder.GetNumChannels() : InsertAtFunction.Channel;

				// Align affected Functions
				const int32 StartIndex = bForward ? FunctionToReorderIndex + 1 : InsertAtIndex;
				const int32 EndIndex = bForward ? InsertAtIndex : FunctionToReorderIndex - 1;
				for (int32 FunctionIndex = StartIndex; FunctionIndex <= EndIndex; FunctionIndex++)
				{
					if (Mode.Functions.IsValidIndex(FunctionIndex))
					{
						FDMXFixtureFunction& Function = Mode.Functions[FunctionIndex];
						Function.Channel += Offset;
					}
				}

				// Update Functions array order
				if (bForward && Mode.Functions.IsValidIndex(InsertAtIndex + 1))
				{
					Mode.Functions.Insert(FunctionToReorder, InsertAtIndex + 1);
				}
				else if (bForward)
				{
					Mode.Functions.Add(FunctionToReorder);
				}
				else
				{
					Mode.Functions.Insert(FunctionToReorder, InsertAtIndex);
				}
				Mode.Functions.RemoveAt(bForward ? FunctionToReorderIndex : FunctionToReorderIndex + 1);

			}
		}
	}
}

void UDMXEntityFixtureType::SetFunctionName(int32 InModeIndex, int32 InFunctionIndex, const FString& InDesiredFunctionName, FString& OutUniqueFunctionName)
{
	if (ensureMsgf(Modes.IsValidIndex(InModeIndex) && Modes[InModeIndex].Functions.IsValidIndex(InFunctionIndex), TEXT("Invalid Mode Index or Function Index when setting Function Name.")))
	{
		FDMXFixtureMode& Mode = Modes[InModeIndex];
		FDMXFixtureFunction& Function = Mode.Functions[InFunctionIndex];

		FString OldName = Function.FunctionName;

		const FString UniqueFunctionNameString = [InModeIndex, InFunctionIndex, InDesiredFunctionName, this]()
		{
			TSet<FString> OtherFunctionNames;
			for (int32 OtherFunctionIndex = 0; OtherFunctionIndex < Modes[InModeIndex].Functions.Num(); OtherFunctionIndex++)
			{
				if (OtherFunctionIndex != InFunctionIndex)
				{
					OtherFunctionNames.Add(Modes[InModeIndex].Functions[OtherFunctionIndex].FunctionName);
				}
			}
			return FDMXRuntimeUtils::GenerateUniqueNameFromExisting(OtherFunctionNames, InDesiredFunctionName);
		}();

		Function.FunctionName = UniqueFunctionNameString;
		OutUniqueFunctionName = UniqueFunctionNameString;
	}
}

void UDMXEntityFixtureType::SetFunctionStartingChannel(int32 InModeIndex, int32 InFunctionIndex, int32 InDesiredStartingChannel, int32& OutStartingChannel)
{
	if (ensureMsgf(Modes.IsValidIndex(InModeIndex) && Modes[InModeIndex].Functions.IsValidIndex(InFunctionIndex), TEXT("Invalid Mode Index or Function Index when setting Function Starting Channel.")))
	{
		FDMXFixtureMode& Mode = Modes[InModeIndex];
		FDMXFixtureFunction& Function = Mode.Functions[InFunctionIndex];

		const int32 FunctionOverNewChannelIndex = Mode.Functions.IndexOfByPredicate([InDesiredStartingChannel](const FDMXFixtureFunction& Function)
			{
				return
					Function.Channel >= InDesiredStartingChannel &&
					Function.GetLastChannel() <= InDesiredStartingChannel;
			});

		// Find the index where the Function should be moved to. If not a valid index, the Function is already at the right index.
		const int32 DesiredIndex = [&]()
		{
			if (FunctionOverNewChannelIndex != INDEX_NONE)
			{
				return FunctionOverNewChannelIndex;
			}
			else
			{
				const bool bDesiredIndexAfterLastFunction =
					Mode.Functions.Num() > 0 &&
					Mode.Functions.Last().GetLastChannel() < InDesiredStartingChannel;

				return bDesiredIndexAfterLastFunction ? Mode.Functions.Num() - 1 : INDEX_NONE;
			}
		}();

		if (Mode.Functions.IsValidIndex(DesiredIndex))
		{
			ReorderFunction(InModeIndex, InFunctionIndex, DesiredIndex);
		}
	}
}

void UDMXEntityFixtureType::ClampFunctionDefautValueByDataType(int32 ModeIndex, int32 FunctionIndex)
{
	// DEPRECATED 5.5
	if (ensureMsgf(Modes.IsValidIndex(ModeIndex), TEXT("Trying to clamp Function Default Value, but Mode Index is not valid.")))
	{
		FDMXFixtureMode& Mode = Modes[ModeIndex];

		if (ensureMsgf(Mode.Functions.IsValidIndex(FunctionIndex), TEXT("Trying to clamp Function Default Value, but Function Index is not valid.")))
		{
			FDMXFixtureFunction& Function = Mode.Functions[FunctionIndex];
			const uint32 SafeDefaultValue = FMath::Min(static_cast<int64>(TNumericLimits<uint32>::Max()), Function.DefaultValue);
			const uint32 ClampedDefaultValue = FDMXConversions::ClampValueBySignalFormat(SafeDefaultValue, Function.DataType);

			Function.DefaultValue = ClampedDefaultValue;

#if WITH_EDITOR
			Function.UpdatePhysicalDefaultValue();
#endif
		}
	}
}

void UDMXEntityFixtureType::AddCellAttribute(int32 ModeIndex)
{
	if (ensureMsgf(Modes.IsValidIndex(ModeIndex), TEXT("Trying to add a Cell Attribute, but Mode Index is not valid.")))
	{
		FDMXFixtureMode& Mode = Modes[ModeIndex];
		FDMXFixtureMatrix& Matrix = Mode.FixtureMatrixConfig;

		FDMXFixtureCellAttribute NewAttribute;
		TArray<FName> AvailableAttributes = FDMXAttributeName::GetPredefinedValues();
		NewAttribute.Attribute = AvailableAttributes.Num() > 0 ? FDMXAttributeName(AvailableAttributes[0]) : FDMXAttributeName();

		// Disable the matrix while editing it so other functions align when enabling it again
		SetFixtureMatrixEnabled(ModeIndex, false);

		Mode.FixtureMatrixConfig.CellAttributes.Add(NewAttribute);

		SetFixtureMatrixEnabled(ModeIndex, true);
		UpdateChannelSpan(ModeIndex);
	}
}

void UDMXEntityFixtureType::RemoveCellAttribute(int32 ModeIndex, int32 CellAttributeIndex)
{
	if (ensureMsgf(Modes.IsValidIndex(ModeIndex) && Modes[ModeIndex].FixtureMatrixConfig.CellAttributes.IsValidIndex(CellAttributeIndex), TEXT("Trying to remove a Cell Attribute, but Mode Index or Cell Attribute Index is invalid.")))
	{
		FDMXFixtureMode& Mode = Modes[ModeIndex];
		FDMXFixtureMatrix& Matrix = Mode.FixtureMatrixConfig;

		// Disable the matrix while editing it so other functions align when enabling it again
		SetFixtureMatrixEnabled(ModeIndex, false);

		Matrix.CellAttributes.RemoveAt(CellAttributeIndex);

		SetFixtureMatrixEnabled(ModeIndex, true);
	}
}

void UDMXEntityFixtureType::ReorderMatrix(int32 ModeIndex, int32 InsertAtFunctionIndex)
{
	if (ensureMsgf(Modes.IsValidIndex(ModeIndex), TEXT("Trying to reorder Matrix after Function, but Mode Index is not valid.")))
	{
		FDMXFixtureMode& Mode = Modes[ModeIndex];
		FDMXFixtureMatrix& Matrix = Mode.FixtureMatrixConfig;

		if (ensureMsgf(Mode.bFixtureMatrixEnabled, TEXT("Trying to reorder Matrix after Function, but Matrix is not enabled or Function Indices are not valid.")))
		{
			// Correct the insert at function index depending on direction
			const bool bReorderMatrixForward = Mode.Functions[InsertAtFunctionIndex].Channel < Matrix.FirstCellChannel;
			InsertAtFunctionIndex = bReorderMatrixForward ? InsertAtFunctionIndex : InsertAtFunctionIndex + 1;

			// Reorder channels as if the matrix was removed
			for (FDMXFixtureFunction& Function : Mode.Functions)
			{
				if (Function.Channel >= Matrix.FirstCellChannel)
				{
					Function.Channel -= Mode.FixtureMatrixConfig.GetNumChannels();
				}
			}

			// Set the new Matrix starting channel
			if (Mode.Functions.IsValidIndex(InsertAtFunctionIndex))
			{
				const FDMXFixtureFunction& InsertAtFunction = Mode.Functions[InsertAtFunctionIndex];
				Matrix.FirstCellChannel = InsertAtFunction.Channel;
			}
			else if (Mode.Functions.Num() > 0)
			{
				Matrix.FirstCellChannel = Mode.Functions.Last().Channel + Mode.Functions.Last().GetNumChannels();
			}

			// Reorder channels that now conflict to after the matrix
			for (FDMXFixtureFunction& Function : Mode.Functions)
			{
				if (Function.Channel >= Matrix.FirstCellChannel)
				{
					Function.Channel += Mode.FixtureMatrixConfig.GetNumChannels();
				}
			}
		}
	}
}

void UDMXEntityFixtureType::UpdateYCellsFromXCells(int32 ModeIndex)
{
	if (ensureMsgf(Modes.IsValidIndex(ModeIndex), TEXT("Trying to update YCells from XCells, but Mode Index is not valid.")))
	{
		FDMXFixtureMode& Mode = Modes[ModeIndex];
		FDMXFixtureMatrix& Matrix = Mode.FixtureMatrixConfig;
		if (ensureMsgf(Mode.bFixtureMatrixEnabled, TEXT("Trying to update YCells from XCells, but Fixture Matrix is not enabled.")))
		{
			int32 NumChannelsOfCell = 0;
			for (const FDMXFixtureCellAttribute& CellAttribute : Matrix.CellAttributes)
			{
				NumChannelsOfCell += CellAttribute.GetNumChannels();
			}
			Matrix.YCells = DMX_MAX_ADDRESS / (Matrix.XCells * NumChannelsOfCell);

			AlignFunctionChannels(ModeIndex);
			UpdateChannelSpan(ModeIndex);
		}
	}
}

void UDMXEntityFixtureType::UpdateXCellsFromYCells(int32 ModeIndex)
{
	if (ensureMsgf(Modes.IsValidIndex(ModeIndex), TEXT("Trying to update XCells from YCells, but Mode Index is not valid.")))
	{
		FDMXFixtureMode& Mode = Modes[ModeIndex];
		FDMXFixtureMatrix& Matrix = Mode.FixtureMatrixConfig;
		if (ensureMsgf(Mode.bFixtureMatrixEnabled, TEXT("Trying to update XCells from YCells, but Fixture Matrix is not enabled.")))
		{
			if (Matrix.GetNumChannels() > DMX_MAX_ADDRESS)
			{
				int32 NumChannelsOfCell = 0;
				for (const FDMXFixtureCellAttribute& CellAttribute : Matrix.CellAttributes)
				{
					NumChannelsOfCell += CellAttribute.GetNumChannels();
				}

				Matrix.XCells = DMX_MAX_ADDRESS / (Matrix.YCells * NumChannelsOfCell);

				AlignFunctionChannels(ModeIndex);
				UpdateChannelSpan(ModeIndex);
			}
		}
	}
}

void UDMXEntityFixtureType::FunctionValueToBytes(const FDMXFixtureFunction& InFunction, uint32 InValue, uint8* OutBytes)
{
	IntToBytes(InFunction.DataType, InFunction.bUseLSBMode, InValue, OutBytes);
}

void UDMXEntityFixtureType::IntToBytes(EDMXFixtureSignalFormat InSignalFormat, bool bUseLSB, uint32 InValue, uint8* OutBytes)
{
	// Make sure the input value is in the valid range for the data type
	InValue = FDMXConversions::ClampValueBySignalFormat(InValue, InSignalFormat);

	// Number of bytes we'll have to manipulate
	const uint8 NumBytes = FDMXConversions::GetSizeOfSignalFormat(InSignalFormat);

	if (NumBytes == 1)
	{
		OutBytes[0] = (uint8)InValue;
		return;
	}

	// To avoid branching in the loop, we'll decide before it on which byte to start
	// and which direction to go, depending on the Function's bit endianness.
	const int8 ByteIndexStep = bUseLSB ? 1 : -1;
	int8 OutByteIndex = bUseLSB ? 0 : NumBytes - 1;

	for (uint8 ValueByte = 0; ValueByte < NumBytes; ++ValueByte)
	{
		OutBytes[OutByteIndex] = InValue >> 8 * ValueByte & 0xFF;
		OutByteIndex += ByteIndexStep;
	}
}

uint32 UDMXEntityFixtureType::BytesToFunctionValue(const FDMXFixtureFunction& InFunction, const uint8* InBytes)
{
	return BytesToInt(InFunction.DataType, InFunction.bUseLSBMode, InBytes);
}

uint32 UDMXEntityFixtureType::BytesToInt(EDMXFixtureSignalFormat InSignalFormat, bool bUseLSB, const uint8* InBytes)
{
	// Number of bytes we'll read
	const uint8 NumBytes = FDMXConversions::GetSizeOfSignalFormat(InSignalFormat);

	if (NumBytes == 1)
	{
		return *InBytes;
	}

	// To avoid branching in the loop, we'll decide before it on which byte to start
	// and which direction to go, depending on the Function's bit endianness.
	const int8 ByteIndexStep = bUseLSB ? 1 : -1;
	int8 InByteIndex = bUseLSB ? 0 : NumBytes - 1;

	uint32 Result = 0;
	for (uint8 ValueByte = 0; ValueByte < NumBytes; ++ValueByte)
	{
		Result += InBytes[InByteIndex] << 8 * ValueByte;
		InByteIndex += ByteIndexStep;
	}

	return Result;
}

void UDMXEntityFixtureType::FunctionNormalizedValueToBytes(const FDMXFixtureFunction& InFunction, float InValue, uint8* OutBytes)
{
	NormalizedValueToBytes(InFunction.DataType, InFunction.bUseLSBMode, InValue, OutBytes);
}

void UDMXEntityFixtureType::NormalizedValueToBytes(EDMXFixtureSignalFormat InSignalFormat, bool bUseLSB, float InValue, uint8* OutBytes)
{
	// Make sure InValue is in the range [0.0 ... 1.0]
	InValue = FMath::Clamp(InValue, 0.0f, 1.0f);

	const uint32 IntValue = FDMXConversions::GetSignalFormatMaxValue(InSignalFormat) * InValue;

	// Get the individual bytes from the computed IntValue
	IntToBytes(InSignalFormat, bUseLSB, IntValue, OutBytes);
}

float UDMXEntityFixtureType::BytesToFunctionNormalizedValue(const FDMXFixtureFunction& InFunction, const uint8* InBytes)
{
	return BytesToNormalizedValue(InFunction.DataType, InFunction.bUseLSBMode, InBytes);
}

float UDMXEntityFixtureType::BytesToNormalizedValue(EDMXFixtureSignalFormat InSignalFormat, bool bUseLSB, const uint8* InBytes)
{
	// Get the value represented by the individual bytes
	const float Value = BytesToInt(InSignalFormat, bUseLSB, InBytes);

	// Normalize it
	return Value / FDMXConversions::GetSignalFormatMaxValue(InSignalFormat);
}

#undef LOCTEXT_NAMESPACE
