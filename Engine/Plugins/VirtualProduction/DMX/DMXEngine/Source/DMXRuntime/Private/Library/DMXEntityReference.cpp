// Copyright Epic Games, Inc. All Rights Reserved.

#include "Library/DMXEntityReference.h"
#include "Library/DMXLibrary.h"
#include "Library/DMXEntity.h"
#include "Library/DMXEntityController.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/DMXEntityFixturePatch.h"

FDMXEntityReference::FDMXEntityReference()
	: DMXLibrary(nullptr)
	, bDisplayLibraryPicker(true)
	, EntityId(0, 0, 0, 0)
{}

FDMXEntityReference::FDMXEntityReference(UDMXEntity* InEntity)
	: FDMXEntityReference()
{
	SetEntity(InEntity);
}

void FDMXEntityReference::SetEntity(UDMXEntity* NewEntity)
{
	if (NewEntity)
	{
		DMXLibrary = NewEntity->GetParentLibrary();
		EntityId = NewEntity->GetID();
		EntityType = NewEntity->GetClass();
		
		CachedEntity = NewEntity;
	}
	else
	{
		DMXLibrary = nullptr;
		EntityId.Invalidate();
		CachedEntity.Reset();
	}
}

UDMXEntity* FDMXEntityReference::GetEntity() const
{
	if (CachedEntity.IsValid() &&
		CachedEntity->GetID() == EntityId &&
		CachedEntity->GetParentLibrary() == DMXLibrary)
	{
		return CachedEntity.Get();
	}
	else if (IsValid(DMXLibrary) && EntityId.IsValid())
	{
		UDMXEntity* Entity = DMXLibrary->FindEntity(EntityId);
		if (IsValid(Entity) && Entity->GetClass()->IsChildOf(GetEntityType()))
		{
			CachedEntity = Entity;
			return Entity;
		}
		else
		{
			CachedEntity.Reset();
		}
	}
	return nullptr;
}

TSubclassOf<UDMXEntity> FDMXEntityReference::GetEntityType() const
{
	return EntityType;
}

bool FDMXEntityReference::operator==(const FDMXEntityReference& Other) const
{
	return DMXLibrary == Other.DMXLibrary && EntityId == Other.EntityId;
}

bool FDMXEntityReference::operator!=(const FDMXEntityReference& Other) const
{
	return !(*this == Other);
}

FDMXEntityControllerRef::FDMXEntityControllerRef()
{
	EntityType = UDMXEntityController::StaticClass();
}

FDMXEntityControllerRef::FDMXEntityControllerRef(UDMXEntityController* InController)
	: FDMXEntityReference(InController)
{}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UDMXEntityController* FDMXEntityControllerRef::GetController() const
{
	return Cast<UDMXEntityController>(GetEntity());
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FDMXEntityFixtureTypeRef::FDMXEntityFixtureTypeRef()
{
	EntityType = UDMXEntityFixtureType::StaticClass();
}

FDMXEntityFixtureTypeRef::FDMXEntityFixtureTypeRef(UDMXEntityFixtureType* InFixtureType)
	: FDMXEntityReference(InFixtureType)
{}

UDMXEntityFixtureType* FDMXEntityFixtureTypeRef::GetFixtureType() const
{
	return Cast<UDMXEntityFixtureType>(GetEntity());
}

FDMXEntityFixturePatchRef::FDMXEntityFixturePatchRef()
{
	EntityType = UDMXEntityFixturePatch::StaticClass();
}

FDMXEntityFixturePatchRef::FDMXEntityFixturePatchRef(UDMXEntityFixturePatch* InFixturePatch)
	: FDMXEntityReference(InFixturePatch)
{}

UDMXEntityFixturePatch* FDMXEntityFixturePatchRef::GetFixturePatch() const
{
	return Cast<UDMXEntityFixturePatch>(GetEntity());
}

//~ Type conversions extension for Entity Reference structs
PRAGMA_DISABLE_DEPRECATION_WARNINGS
UDMXEntityController* UDMXEntityReferenceConversions::Conv_ControllerRefToObj(const FDMXEntityControllerRef& InControllerRef)
{
	return InControllerRef.GetController();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

UDMXEntityFixtureType* UDMXEntityReferenceConversions::Conv_FixtureTypeRefToObj(const FDMXEntityFixtureTypeRef& InFixtureTypeRef)
{
	return InFixtureTypeRef.GetFixtureType();
}

UDMXEntityFixturePatch* UDMXEntityReferenceConversions::Conv_FixturePatchRefToObj(const FDMXEntityFixturePatchRef& InFixturePatchRef)
{
	return InFixturePatchRef.GetFixturePatch();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FDMXEntityControllerRef UDMXEntityReferenceConversions::Conv_ControllerObjToRef(UDMXEntityController* InController)
{
	return FDMXEntityControllerRef(InController);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FDMXEntityFixtureTypeRef UDMXEntityReferenceConversions::Conv_FixtureTypeObjToRef(UDMXEntityFixtureType* InFixtureType)
{
	return FDMXEntityFixtureTypeRef(InFixtureType);
}

FDMXEntityFixturePatchRef UDMXEntityReferenceConversions::Conv_FixturePatchObjToRef(UDMXEntityFixturePatch* InFixturePatch)
{
	return FDMXEntityFixturePatchRef(InFixturePatch);
}
