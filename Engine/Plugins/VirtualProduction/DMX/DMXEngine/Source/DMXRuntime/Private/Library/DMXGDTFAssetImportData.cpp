// Copyright Epic Games, Inc. All Rights Reserved.

#include "Library/DMXGDTFAssetImportData.h"

#include "Misc/FileHelper.h"
#include "Misc/Paths.h"


#if WITH_EDITOR
void UDMXGDTFAssetImportData::SetSourceFile(const FString& InFilePathAndName)
{
	FilePathAndName = InFilePathAndName;

	FAssetImportInfo Info;
	Info.Insert(FAssetImportInfo::FSourceFile(FilePathAndName));
	SourceData = Info;

	if (!FPaths::FileExists(FilePathAndName))
	{
		return;
	}

	RawSourceData.ByteArray.Reset();
	FFileHelper::LoadFileToArray(RawSourceData.ByteArray, *FilePathAndName);
}
#endif // WITH_EDITOR

void UDMXGDTFAssetImportData::PostLoad() 
{
	Super::PostLoad();

#if WITH_EDITOR
	FAssetImportInfo Info;
	Info.Insert(FAssetImportInfo::FSourceFile(FilePathAndName));
	SourceData = Info;
#endif // WITH_EDITOR
}
