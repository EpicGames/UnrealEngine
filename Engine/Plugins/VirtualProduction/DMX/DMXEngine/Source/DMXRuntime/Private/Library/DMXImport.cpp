// Copyright Epic Games, Inc. All Rights Reserved.

#include "Library/DMXImport.h"
#include "Containers/UnrealString.h"

namespace DMXImport
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FDMXColorCIE ParseColorCIE(const FString& InColor)
	{
		TArray<FString> ColorCIEArray;
		FDMXColorCIE ColorCIE;

		if (!InColor.IsEmpty())
		{
			InColor.ParseIntoArray(ColorCIEArray, TEXT(","));

			for (int32 ColorStrIndex = 0; ColorStrIndex < ColorCIEArray.Num(); ++ColorStrIndex)
			{
				if (ColorStrIndex == 0)
				{
					LexTryParseString<float>(ColorCIE.X, *ColorCIEArray[0]);
				}
				else if (ColorStrIndex == 1)
				{
					LexTryParseString<float>(ColorCIE.Y, *ColorCIEArray[1]);
				}
				else if (ColorStrIndex == 2)
				{
					LexTryParseString<uint8>(ColorCIE.YY, *ColorCIEArray[2]);
				}
			}
		}

		return MoveTemp(ColorCIE);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FMatrix ParseMatrix(FString&& InMatrixStr)
	{
		FMatrix Matrix = FMatrix::Identity;

		if (!InMatrixStr.IsEmpty())
		{
			// Remove symbols from the matrix
			FString ValuesString;
			for (int32 CharIndex = 0; CharIndex < InMatrixStr.Len(); ++CharIndex)
			{
				if (InMatrixStr[CharIndex] != '{' && InMatrixStr[CharIndex] != '}')
				{
					ValuesString.AppendChar(InMatrixStr[CharIndex]);
				}
			}

			TArray<FString> MatrixMStrArray;
			InMatrixStr.ParseIntoArray(MatrixMStrArray, TEXT(","));

			uint32 PlaneIndex = 0;
			for (int32 MatrixMIndex = 0; MatrixMIndex < MatrixMStrArray.Num(); ++MatrixMIndex)
			{
				float Value = 0.f;
				LexTryParseString(Value, *MatrixMStrArray[MatrixMIndex]);

				uint32 ValueIndex = MatrixMIndex % 4;

				if (MatrixMIndex != 0 && ValueIndex == 0)
				{
					PlaneIndex++;
				}

				Matrix.M[PlaneIndex][ValueIndex] = Value;
			}
		}

		return MoveTemp(Matrix);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
