// Copyright Epic Games, Inc. All Rights Reserved.

#include "Library/DMXImportGDTF.h"

#include "DMXGDTF.h"
#include "GDTF/DMXGDTFDescription.h"
#include "Library/DMXGDTFAssetImportData.h"

#if WITH_EDITORONLY_DATA
#include "EditorFramework/AssetImportData.h"
#endif 

UDMXImportGDTF::UDMXImportGDTF()
{
	GDTFAssetImportData = NewObject<UDMXGDTFAssetImportData>(this, TEXT("GDTFAssetImportData"), RF_Public);
}

void UDMXImportGDTF::PostLoad()
{
	Super::PostLoad();

	// Upgrade so this object always holds asset data.
	if (!GDTFAssetImportData)
	{
		GDTFAssetImportData = NewObject<UDMXGDTFAssetImportData>(this, TEXT("GDTFAssetImportData"));
	}

#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!SourceFilename_DEPRECATED.IsEmpty())
	{
		GDTFAssetImportData->SetSourceFile(SourceFilename_DEPRECATED);

		SourceFilename_DEPRECATED.Empty();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
}

UDMXGDTF* UDMXImportGDTF::LoadGDTF() const
{
	UDMXGDTF* GDTF = NewObject<UDMXGDTF>();
	if (GDTFAssetImportData)
	{
		GDTF->InitializeFromData(GDTFAssetImportData->GetRawSourceData());
	}

	return GDTF;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UDMXImportGDTFAttributeDefinitions::FindFeature(const FString& InQuery, FDMXImportGDTFFeature& OutFeature) const
{
	// DEPRECATED 5.4

    if (InQuery.IsEmpty())
    {
        return false;
    }

    FString Left;
    FString Right;
    InQuery.Split(TEXT("."), &Left, &Right);
    const FName Name_Left = FName(*Left);
    const FName Name_Right = FName(*Right);

	for (const FDMXImportGDTFFeatureGroup& FeatureGroup : FeatureGroups)
	{
		if (FeatureGroup.Name == Name_Left)
		{
			for (const FDMXImportGDTFFeature& Feature : FeatureGroup.Features)
			{
				if (Feature.Name == Name_Right)
				{
                    OutFeature = Feature;
                    return true;
				}
			}
		}
	}

    return false;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UDMXImportGDTFAttributeDefinitions::FindAtributeByName(const FName& InName, FDMXImportGDTFAttribute& OutAttribute) const
{	
	// DEPRECATED 5.4

    if (InName.IsNone())
    {
        return false;
    }

    for (const FDMXImportGDTFAttribute& Attribute : Attributes)
    {
        if (Attribute.Name == InName)
        {
            OutAttribute = Attribute;
            return true;
        }
    }

    return false;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UDMXImportGDTFWheels::FindWeelByName(const FName& InName, FDMXImportGDTFWheel& OutWheel) const
{	
	// DEPRECATED 5.4

	if (InName.IsNone())
    {
        return false;
    }

    for (const FDMXImportGDTFWheel& Wheel : Wheels)
    {
        if (Wheel.Name == InName)
        {
            OutWheel = Wheel;
			return true;
        }
    }

    return false;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UDMXImportGDTFPhysicalDescriptions::FindEmitterByName(const FName& InName, FDMXImportGDTFEmitter& OutEmitter) const
{
	// DEPRECATED 5.4

	if (InName.IsNone())
    {
		return false;
    }

    for (const FDMXImportGDTFEmitter& Emitter : Emitters)
    {
        if (Emitter.Name == InName)
        {
            OutEmitter = Emitter;
			return true;
        }
    }

	return false;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FDMXImportGDTFDMXValue::FDMXImportGDTFDMXValue(const FString& InDMXValueStr)
    : Value(0)
    , ValueSize(1)
{	
	// DEPRECATED 5.4

    if (!InDMXValueStr.IsEmpty())
    {
        if (InDMXValueStr.Equals("None"))
        {
            ValueSize = 0;
        }
        else
        {
            FString Left;
            FString Right;
            InDMXValueStr.Split(TEXT("/"), &Left, &Right);

            LexTryParseString(this->Value, *Left);
            LexTryParseString(this->ValueSize, *Right);
        }
    }
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool FDMXImportGDTFDMXChannel::ParseOffset(const FString& InOffsetStr)
{	
	// DEPRECATED 5.4

	if (InOffsetStr.IsEmpty())
	{
		return false;
	}

    Offset.Empty();
    TArray<FString> OffsetStrArray;
    InOffsetStr.ParseIntoArray(OffsetStrArray, TEXT(","));

    for (int32 OffsetIndex = 0; OffsetIndex < OffsetStrArray.Num(); ++OffsetIndex)
    {
        int32 OffsetValue;
        LexTryParseString(OffsetValue, *OffsetStrArray[OffsetIndex]);
        Offset.Add(OffsetValue);
    }
    
	return true;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TArray<FDMXImportGDTFChannelFunction> UDMXImportGDTFDMXModes::GetDMXChannelFunctions(const FDMXImportGDTFDMXMode& InMode)
{
	// DEPRECATED 5.4

	TArray<FDMXImportGDTFChannelFunction> Channels;
	for (const FDMXImportGDTFDMXChannel& ModeChannel : InMode.DMXChannels)
	{
		Channels.Add(ModeChannel.LogicalChannel.ChannelFunction);
	}
	return MoveTemp(Channels);
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
