// Copyright Epic Games, Inc. All Rights Reserved.
#include "Library/DMXLibrary.h"

#include "Algo/MaxElement.h"
#include "DMXRuntimeLog.h"
#include "DMXRuntimeMainStreamObjectVersion.h"
#include "DMXProtocolSettings.h"
#include "DMXProtocolUtils.h"
#include "Interfaces/IDMXProtocol.h"
#include "IO/DMXInputPort.h"
#include "IO/DMXOutputPort.h"
#include "IO/DMXPortManager.h"
#include "Library/DMXEntity.h"
#include "Library/DMXEntityController.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXImportGDTF.h"
#include "MVR/DMXMVRGeneralSceneDescription.h"

#include "Misc/Paths.h"

#define LOCTEXT_NAMESPACE "DMXLibrary"


FDMXOnEntityArrayChangedDelegate UDMXLibrary::OnEntitiesAddedDelegate;

FDMXOnEntityArrayChangedDelegate UDMXLibrary::OnEntitiesRemovedDelegate;

UDMXLibrary::UDMXLibrary()
{
	const FName GeneralSceneDescriptionName = FName(GetName() + TEXT("_MVRGeneralSceneDescription"));
	GeneralSceneDescription = NewObject<UDMXMVRGeneralSceneDescription>(this, GeneralSceneDescriptionName, RF_Public);
}

void UDMXLibrary::PostInitProperties()
{
	Super::PostInitProperties();

	if (!IsTemplate())
	{
		FDMXPortManager::Get().OnPortsChanged.AddUObject(this, &UDMXLibrary::UpdatePorts);
		UpdatePorts();
	}
}

void UDMXLibrary::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITOR
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FDMXRuntimeMainStreamObjectVersion::GUID) < FDMXRuntimeMainStreamObjectVersion::DMXFixturePatchHasFixtureID)
		{
			const TArray<UDMXEntityFixturePatch*> FixturePatches = GetEntitiesTypeCast<UDMXEntityFixturePatch>();
			for (UDMXEntityFixturePatch* FixturePatch : FixturePatches)
			{
				FixturePatch->GenerateFixtureID();
			}
		}
	}
#endif
}

void UDMXLibrary::PostLoad()
{
	Super::PostLoad();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
#if WITH_EDITOR
		// Upgrade from controllers to ports
		bool bNeedsUpgradeFromControllersToPorts = Entities.ContainsByPredicate([](UDMXEntity* Entity) {
			return Cast<UDMXEntityController>(Entity) != nullptr;
			});
		if (bNeedsUpgradeFromControllersToPorts)
		{
			UpgradeFromControllersToPorts();
		}

#endif 
		UpdatePorts();

		// Remove null entities
		TArray<UDMXEntity*> CachedEntities = Entities;
		for (UDMXEntity* Entity : CachedEntities)
		{
			// All entities are expected to be valid. We should never enter the condition here.
			if (!ensure(Entity))
			{
				UE_LOGF(LogDMXRuntime, Warning, "Invalid Entity found in Library %ls. Please resave the library.", *GetName());
				Entities.Remove(Entity);
			}
		}
	}
}

void UDMXLibrary::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	if (DuplicateMode != EDuplicateMode::Normal)
	{
		return;
	}

	// Make sure all Entity children have this library as their parent
	// and refresh their ID
	TArray<UDMXEntity*> ValidEntities;
	for (UDMXEntity* Entity : Entities)
	{
		// Entity could be null
		if (ensure(Entity))
		{
			Entity->RefreshID();
			ValidEntities.Add(Entity);
		}
	}

	// duplicate only valid entities
	Entities = ValidEntities;

	// Update the ports 
	UpdatePorts();
}

#if WITH_EDITOR
void UDMXLibrary::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	const FProperty* Property = PropertyChangedEvent.Property;
	const UScriptStruct* InputPortReferenceStruct = FDMXInputPortReference::StaticStruct();
	const UScriptStruct* OutputPortReferenceStruct = FDMXOutputPortReference::StaticStruct();
	const UStruct* PropertyOwnerStruct = Property ? Property->GetOwnerStruct() : nullptr;

	if (PropertyName == FDMXInputPortReference::GetEnabledFlagPropertyName() ||
		PropertyName == FDMXOutputPortReference::GetEnabledFlagPropertyName() ||
		(InputPortReferenceStruct && InputPortReferenceStruct == PropertyOwnerStruct) || 
		(OutputPortReferenceStruct && OutputPortReferenceStruct == PropertyOwnerStruct))
	{
		UpdatePorts();
	}

	// Remove any null entities from the library
	Entities.RemoveAll([](UDMXEntity* Entity)
		{
			return !IsValid(Entity);
		});
}
#endif // WITH_EDITOR

void UDMXLibrary::RegisterEntity(UDMXEntity* Entity)
{
	if (!ensureAlwaysMsgf(IsValid(Entity), TEXT("Trying to register Entity with DMX Library, but DMX Entity is not valid.")))
	{
		return;
	}

	if (!ensureAlwaysMsgf(!Entities.Contains(Entity), TEXT("Trying to register Entity %s with DMX Library, but Entity was already added."), *Entity->Name))
	{
		return;
	}

	Entities.Add(Entity);
	LastAddedEntity = Entity;

	OnEntitiesAddedDelegate.Broadcast(this, TArray<UDMXEntity*>({ Entity }));
}

void UDMXLibrary::UnregisterEntity(UDMXEntity* Entity)
{
	if (Entities.Contains(Entity))
	{
		Entities.Remove(Entity);
		OnEntitiesRemovedDelegate.Broadcast(this, TArray<UDMXEntity*>({ Entity }));
	}
}

UDMXEntity* UDMXLibrary::GetOrCreateEntityObject(const FString& InName, TSubclassOf<UDMXEntity> DMXEntityClass)
{
	if (DMXEntityClass == nullptr)
	{
		DMXEntityClass = UDMXEntity::StaticClass(); 
	}

	if (!InName.IsEmpty())
	{
		for (UDMXEntity* Entity : Entities)
		{
			if (Entity != nullptr && Entity->IsA(DMXEntityClass) && Entity->GetDisplayName() == InName)
			{
				return Entity;
			}
		}
	}

	UDMXEntity* Entity = NewObject<UDMXEntity>(this, DMXEntityClass, NAME_None, RF_Transactional);
	Entity->SetName(InName);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	AddEntity(Entity);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	return Entity;
}

UDMXEntity* UDMXLibrary::FindEntity(const FString& InSearchName) const
{
	TObjectPtr<UDMXEntity> const* Entity = Entities.FindByPredicate([&InSearchName](const UDMXEntity* InEntity)->bool
		{
			return InEntity && InEntity->GetDisplayName().Equals(InSearchName);
		});

	if (Entity != nullptr)
	{
		return *Entity;
	}
	return nullptr;
}

UDMXEntity* UDMXLibrary::FindEntity(const FGuid& Id)
{
	for (UDMXEntity* Entity : Entities)
	{
		if (IsValid(Entity) && Entity->GetID() == Id)
		{
			return Entity;
		}
	}

	return nullptr;
}

int32 UDMXLibrary::FindEntityIndex(UDMXEntity* InEntity) const
{
	return Entities.Find(InEntity);
}

void UDMXLibrary::AddEntity(UDMXEntity* InEntity)
{
	// DEPRECATED 5.0

	check(InEntity);
	check(!Entities.Contains(InEntity));
	
	Entities.Add(InEntity);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	InEntity->SetParentLibrary(this);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Check for unique Id
	for (UDMXEntity* Entity : Entities)
	{
		if (Entity && InEntity->GetID() == Entity->GetID())
		{
			InEntity->RefreshID();
			break;
		}
	}

	OnEntitiesAddedDelegate.Broadcast(this, TArray<UDMXEntity*>({ InEntity }));
	OnEntitiesUpdated_DEPRECATED.Broadcast(this);
}

void UDMXLibrary::SetEntityIndex(UDMXEntity* InEntity, const int32 NewIndex)
{
	if (NewIndex < 0)
	{
		return; 
	}

	const int32&& OldIndex = Entities.Find(InEntity);
	if (OldIndex == INDEX_NONE || OldIndex == NewIndex)
	{
		return; 
	}

	if (NewIndex == OldIndex + 1)
	{
		return; 
	}

	// If elements are close to each other, just swap them. It's the fastest operation.
	if (NewIndex == OldIndex - 1)
	{
		Entities.SwapMemory(OldIndex, NewIndex);
	}
	else
	{
		if (NewIndex >= Entities.Num())
		{
			Entities.RemoveAt(OldIndex, EAllowShrinking::No);
			Entities.Add(InEntity);
			return;
		}

		// We could use RemoveAt then Insert, but that would shift every Entity after OldIndex on RemoveAt
		// and then every Entity after NewEntityIndex for Insert. Two shifts of possibly many elements!
		// Instead, we just need to shift all entities between NewIndex and OldIndex. Still a potentially
		// huge shift, but likely smaller on most situations.

		if (NewIndex > OldIndex)
		{
			// Shifts DOWN 1 place all elements between the target indexes, as NewIndex is after all of them
			for (int32 EntityIndex = OldIndex; EntityIndex < NewIndex - 1; ++EntityIndex)
			{
				Entities[EntityIndex] = Entities[EntityIndex + 1];
			}
			Entities[NewIndex - 1] = InEntity;
		}
		else
		{
			// Shifts UP 1 place all elements between the target indexes, as NewIndex is before all of them
			for (int32 EntityIndex = OldIndex; EntityIndex > NewIndex; --EntityIndex)
			{
				Entities[EntityIndex] = Entities[EntityIndex - 1];
			}
			Entities[NewIndex] = InEntity;
		}
	}
}

void UDMXLibrary::RemoveEntity(const FString& EntityName)
{
	// DEPRECATED 5.0

	int32 EntityIndex = Entities.IndexOfByPredicate([&EntityName] (const UDMXEntity* Entity)->bool
		{
			return Entity && Entity->GetDisplayName().Equals(EntityName);
		});

	if (EntityIndex != INDEX_NONE)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Entities[EntityIndex]->SetParentLibrary(nullptr);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		Entities.RemoveAt(EntityIndex);

		OnEntitiesRemovedDelegate.Broadcast(this, TArray<UDMXEntity*>({ Entities[EntityIndex] }));
		OnEntitiesUpdated_DEPRECATED.Broadcast(this);
	}
}

void UDMXLibrary::RemoveEntity(UDMXEntity* InEntity)
{
	// DEPRECATED 5.0

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	InEntity->SetParentLibrary(nullptr);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	Entities.Remove(InEntity);

	OnEntitiesRemovedDelegate.Broadcast(this, TArray<UDMXEntity*>({ InEntity }));
	OnEntitiesUpdated_DEPRECATED.Broadcast(this);
}

void UDMXLibrary::RemoveAllEntities()
{
	// DEPRECATED 5.0

	TArray<UDMXEntity*> OldEntities = Entities;
	for (UDMXEntity* Entity : Entities)
	{
		if (Entity)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			Entity->SetParentLibrary(nullptr);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
	Entities.Empty();

	OnEntitiesRemovedDelegate.Broadcast(this, TArray<UDMXEntity*>({ OldEntities }));
	OnEntitiesUpdated_DEPRECATED.Broadcast(this);
}

const TArray<UDMXEntity*>& UDMXLibrary::GetEntities() const
{
	return Entities;
}

TArray<UDMXEntity*> UDMXLibrary::GetEntitiesOfType(TSubclassOf<UDMXEntity> InEntityClass) const
{
	return Entities.FilterByPredicate([&InEntityClass](const UDMXEntity* Entity)
		{
			return Entity && Entity->IsA(InEntityClass);
		});
}

void UDMXLibrary::ForEachEntityOfTypeWithBreak(TSubclassOf<UDMXEntity> InEntityClass, TFunction<bool(UDMXEntity*)> Predicate) const
{
	for (UDMXEntity* Entity : Entities)
	{
		if (Entity != nullptr && Entity->IsA(InEntityClass))
		{
			if (!Predicate(Entity)) 
			{ 
				break; 
			}
		}
	}
}

void UDMXLibrary::ForEachEntityOfType(TSubclassOf<UDMXEntity> InEntityClass, TFunction<void(UDMXEntity*)> Predicate) const
{
	for (UDMXEntity* Entity : Entities)
	{
		if (Entity != nullptr && Entity->IsA(InEntityClass))
		{
			Predicate(Entity);
		}
	}
}

FDMXOnEntityArrayChangedDelegate& UDMXLibrary::GetOnEntitiesAdded()
{
	return OnEntitiesAddedDelegate;
}

FDMXOnEntityArrayChangedDelegate& UDMXLibrary::GetOnEntitiesRemoved()
{
	return OnEntitiesRemovedDelegate;
}

FOnEntitiesUpdated_DEPRECATED& UDMXLibrary::GetOnEntitiesUpdated()
{
	return OnEntitiesUpdated_DEPRECATED;
}

TSet<int32> UDMXLibrary::GetAllLocalUniversesIDsInPorts() const
{
	TSet<int32> Result;
	for (const FDMXPortSharedRef& Port : GenerateAllPortsSet())
	{
		for (int32 UniverseID = Port->GetLocalUniverseStart(); UniverseID <= Port->GetLocalUniverseEnd(); UniverseID++)
		{
			if (!Result.Contains(UniverseID))
			{
				Result.Add(UniverseID);
			}
		}
	}

	return Result;
}

TSet<FDMXPortSharedRef> UDMXLibrary::GenerateAllPortsSet() const
{
	TSet<FDMXPortSharedRef> Result;
	for (const FDMXInputPortSharedRef& InputPort : InputPorts)
	{
		Result.Add(InputPort);
	}

	for (const FDMXOutputPortSharedRef& OutputPort : OutputPorts)
	{
		Result.Add(OutputPort);
	}

	return Result;
}

void UDMXLibrary::UpdatePorts()
{
	const UDMXProtocolSettings* ProtocolSettings = GetDefault<UDMXProtocolSettings>();
	check(ProtocolSettings);

	// Remove ports refs that don't exist anymore
	PortReferences.InputPortReferences.RemoveAll([ProtocolSettings](const FDMXInputPortReference& InputPortReference) {
		const FGuid& PortGuid = InputPortReference.GetPortGuid();

		bool bPortExists = ProtocolSettings->InputPortConfigs.ContainsByPredicate([PortGuid](const FDMXInputPortConfig & InputPortConfig) {
			return InputPortConfig.GetPortGuid() == PortGuid;
		});

		return !bPortExists;
	});

	PortReferences.OutputPortReferences.RemoveAll([ProtocolSettings](const FDMXOutputPortReference& OutputPortReference) {
		const FGuid& PortGuid = OutputPortReference.GetPortGuid();

		bool bPortExists = ProtocolSettings->OutputPortConfigs.ContainsByPredicate([PortGuid](const FDMXOutputPortConfig& OutputPortConfig) {
			return OutputPortConfig.GetPortGuid() == PortGuid;
		});

		return !bPortExists;
	});

	// Add port refs from newly created ports
	for (int32 IndexInputPortConfig = 0; IndexInputPortConfig < ProtocolSettings->InputPortConfigs.Num(); IndexInputPortConfig++)
	{
		const FGuid& InputPortGuid = ProtocolSettings->InputPortConfigs[IndexInputPortConfig].GetPortGuid();

		bool bInputPortExists = PortReferences.InputPortReferences.ContainsByPredicate([InputPortGuid](const FDMXInputPortReference& InputPortReference) {
			return InputPortReference.GetPortGuid() == InputPortGuid;
		});

		if (!bInputPortExists)
		{
			// Default to enabled
			bool bEnabled = true;

			if (PortReferences.InputPortReferences.IsValidIndex(IndexInputPortConfig))
			{
				PortReferences.InputPortReferences.Insert(FDMXInputPortReference(InputPortGuid, bEnabled), IndexInputPortConfig);
			}
			else
			{
				PortReferences.InputPortReferences.Add(FDMXInputPortReference(InputPortGuid, bEnabled));
			}
		}
	}

	for (int32 IndexOutputPortConfig = 0; IndexOutputPortConfig < ProtocolSettings->OutputPortConfigs.Num(); IndexOutputPortConfig++)
	{
		const FGuid& OutputPortGuid = ProtocolSettings->OutputPortConfigs[IndexOutputPortConfig].GetPortGuid();

		bool bOutputPortExists = PortReferences.OutputPortReferences.ContainsByPredicate([OutputPortGuid](const FDMXOutputPortReference& OutputPortReference) {
			return OutputPortReference.GetPortGuid() == OutputPortGuid;
			});

		if (!bOutputPortExists)
		{
			// Default to enabled
			bool bEnabled = true;

			if (PortReferences.OutputPortReferences.IsValidIndex(IndexOutputPortConfig))
			{
				PortReferences.OutputPortReferences.Insert(FDMXOutputPortReference(OutputPortGuid, bEnabled), IndexOutputPortConfig);
			}
			else
			{
				PortReferences.OutputPortReferences.Add(FDMXOutputPortReference(OutputPortGuid, bEnabled));
			}
		}
	}

	// Rebuild the arrays of actual ports
	InputPorts.Reset();
	OutputPorts.Reset();

	for (const FDMXInputPortReference& InputPortReference : PortReferences.InputPortReferences)
	{
		if (InputPortReference.IsEnabledFlagSet())
		{
			const FGuid& PortGuid = InputPortReference.GetPortGuid();
			FDMXInputPortSharedPtr InputPort = FDMXPortManager::Get().FindInputPortByGuid(PortGuid); 

			if (InputPort.IsValid())
			{
				InputPorts.Add(InputPort.ToSharedRef());
			}
		}
	}

	for (const FDMXOutputPortReference& OutputPortReference : PortReferences.OutputPortReferences)
	{
		if (OutputPortReference.IsEnabledFlagSet())
		{
			const FGuid& PortGuid = OutputPortReference.GetPortGuid();
			FDMXOutputPortSharedPtr OutputPort = FDMXPortManager::Get().FindOutputPortByGuid(PortGuid);

			if (OutputPort)
			{
				OutputPorts.Add(OutputPort.ToSharedRef());
			}
		}
	}
}

void UDMXLibrary::SetMVRGeneralSceneDescription(UDMXMVRGeneralSceneDescription* NewGeneralSceneDescription)
{
	GeneralSceneDescription = NewGeneralSceneDescription;
}

#if WITH_EDITOR
UDMXMVRGeneralSceneDescription* UDMXLibrary::UpdateGeneralSceneDescription()
{
	if (ensureAlwaysMsgf(GeneralSceneDescription, TEXT("Trying to update General Scene Description of %s, but the General Scene Description is not valid."), *GetName()))
	{
		GeneralSceneDescription->WriteDMXLibrary(*this);
		return GeneralSceneDescription;
	}

	return nullptr;
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UDMXLibrary::UpgradeFromControllersToPorts()
{	
	// This function only needs be called to upgrade projects created before 4.27

	// Only continue if an upgrade may be required
	bool bNeedsUpgradeFromControllersToPorts = Entities.ContainsByPredicate([](UDMXEntity* Entity) {
		return Cast<UDMXEntityController>(Entity) != nullptr;
	});

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (bNeedsUpgradeFromControllersToPorts)
	{
		UDMXProtocolSettings* ProtocolSettings = GetMutableDefault<UDMXProtocolSettings>();

		// Helpers to enable port references
		struct Local
		{
			static void EnableInputPortRef(UDMXLibrary* ThisLibrary, const FGuid& PortGuid)
			{
				ThisLibrary->UpdatePorts();

				FDMXInputPortReference* ExistingPortRef = ThisLibrary->PortReferences.InputPortReferences.FindByPredicate([PortGuid](const FDMXInputPortReference& InputPortRef) {
					return InputPortRef.GetPortGuid() == PortGuid;
					});

				*ExistingPortRef = FDMXInputPortReference(PortGuid, true);
			}

			static void EnableOutputPortRef(UDMXLibrary* ThisLibrary, const FGuid& PortGuid)
			{
				ThisLibrary->UpdatePorts();

				FDMXOutputPortReference* ExistingPortRef = ThisLibrary->PortReferences.OutputPortReferences.FindByPredicate([PortGuid](const FDMXOutputPortReference& OutputPortRef) {
					return OutputPortRef.GetPortGuid() == PortGuid;
					});

				*ExistingPortRef = FDMXOutputPortReference(PortGuid, true);
			}
		};

		TArray<UDMXEntity*> CachedEntities = Entities;
		for (UDMXEntity* Entity : CachedEntities)
		{
			// Clean out controllers that were in the entity array before 4.27
			// Create a corresponding port in project settings, enable only these in the Library.
			if (UDMXEntityController* VoidController = Cast<UDMXEntityController>(Entity))
			{
				// Remove the controller from the library!
				Entities.Remove(VoidController);

				// Find the best NetworkInterface IP
				FString InterfaceIPAddress_DEPRECATED = ProtocolSettings->InterfaceIPAddress_DEPRECATED;
				if (InterfaceIPAddress_DEPRECATED.IsEmpty())
				{
					TArray<TSharedPtr<FString>> LocalNetworkInterfaceCardIPs = FDMXProtocolUtils::GetLocalNetworkInterfaceCardIPs();
					if (LocalNetworkInterfaceCardIPs.Num() > 0)
					{
						InterfaceIPAddress_DEPRECATED = *LocalNetworkInterfaceCardIPs[0];
					}
				}

				FName ProtocolName = VoidController->DeviceProtocol;
				int32 GlobalUniverseOffset_DEPRECATED = ProtocolName == "Art-Net" ? ProtocolSettings->GlobalArtNetUniverseOffset_DEPRECATED : ProtocolSettings->GlobalSACNUniverseOffset_DEPRECATED;

				// Cache names of all port config to generate new port names
				TSet<FString> InputPortConfigNames;
				for (const FDMXInputPortConfig& InputPortConfig : ProtocolSettings->InputPortConfigs)
				{
					InputPortConfigNames.Add(InputPortConfig.GetPortName());
				}

				TSet<FString> OutputPortConfigNames;
				for (const FDMXOutputPortConfig& OutputPortConfig : ProtocolSettings->OutputPortConfigs)
				{
					OutputPortConfigNames.Add(OutputPortConfig.GetPortName());
				}
				
				// The 4.26 libraries held ambigous values for controllers, we mend it here to the best possible
				const int32 FixedLocalUniverseStart = VoidController->UniverseLocalStart + GlobalUniverseOffset_DEPRECATED;
				const int32 FixedLocalUniverseEnd = VoidController->UniverseLocalEnd + GlobalUniverseOffset_DEPRECATED;
				int32 FixedLocalUniverseNum = FixedLocalUniverseEnd - FixedLocalUniverseStart + 1;
				FixedLocalUniverseNum = FMath::Max(FixedLocalUniverseNum, VoidController->UniverseLocalNum);

				int32 FixedExternUniverseStart = VoidController->UniverseRemoteStart + GlobalUniverseOffset_DEPRECATED;
				const int32 FixedExternUnivereEnd = VoidController->UniverseRemoteEnd + GlobalUniverseOffset_DEPRECATED;
				int32 FixedExternUniverseNum = FixedExternUnivereEnd - FixedExternUniverseStart + 1;
				FixedExternUniverseNum = FMath::Max(FixedExternUnivereEnd, FixedExternUniverseStart + VoidController->UniverseLocalNum);

				const int32 FixedNumUniverses = FixedLocalUniverseNum;

				// Convert the controller to an input port	
				FDMXInputPortConfig* ExistingInputPortConfigPtr = ProtocolSettings->FindInputPortConfig([VoidController, &InterfaceIPAddress_DEPRECATED, &ProtocolName, FixedExternUniverseStart](FDMXInputPortConfig& InputPortConfig) {
					return
						InputPortConfig.GetProtocolName() == ProtocolName &&
						InputPortConfig.GetExternUniverseStart() == FixedExternUniverseStart;
				});

				if (ExistingInputPortConfigPtr)
				{
					FDMXInputPortConfigParams InputPortConfigParams(*ExistingInputPortConfigPtr);
					InputPortConfigParams.CommunicationType = EDMXCommunicationType::InternalOnly;
					InputPortConfigParams.LocalUniverseStart = FMath::Min(ExistingInputPortConfigPtr->GetLocalUniverseStart(), FixedLocalUniverseStart); // The lower universe start
					InputPortConfigParams.NumUniverses = FMath::Max(ExistingInputPortConfigPtr->GetNumUniverses(), FixedNumUniverses); // The higher num universes

					*ExistingInputPortConfigPtr = FDMXInputPortConfig(ExistingInputPortConfigPtr->GetPortGuid(), InputPortConfigParams);

					Local::EnableInputPortRef(this, ExistingInputPortConfigPtr->GetPortGuid());
				}
				else if (IDMXProtocol::Get(ProtocolName).IsValid())
				{
					FDMXInputPortConfigParams InputPortConfigParams;

					InputPortConfigParams.ProtocolName = ProtocolName;
					InputPortConfigParams.DeviceAddress = InterfaceIPAddress_DEPRECATED;
					InputPortConfigParams.PortName = FDMXProtocolUtils::GenerateUniqueNameFromExisting(InputPortConfigNames, TEXT("Generated_InputPort"));
					InputPortConfigParams.CommunicationType = EDMXCommunicationType::InternalOnly;
					InputPortConfigParams.LocalUniverseStart = FixedLocalUniverseStart;
					InputPortConfigParams.NumUniverses = FixedNumUniverses;
					InputPortConfigParams.ExternUniverseStart = FixedExternUniverseStart;

					FDMXInputPortConfig InputPortConfig = FDMXInputPortConfig(FGuid::NewGuid(), InputPortConfigParams);

					ProtocolSettings->InputPortConfigs.Add(InputPortConfig);

					Local::EnableInputPortRef(this, InputPortConfig.GetPortGuid());
				}

				// Convert the controller to output ports

				// Add a port for the default output
				FDMXOutputPortConfig* ExistingOutputPortConfigPtr = ProtocolSettings->FindOutputPortConfig([VoidController, &InterfaceIPAddress_DEPRECATED, &ProtocolName, FixedExternUniverseStart](FDMXOutputPortConfig& OutputPortConfig) {
					return
						OutputPortConfig.GetProtocolName() == ProtocolName &&
						OutputPortConfig.GetExternUniverseStart() == FixedExternUniverseStart;
				});

				if (ExistingOutputPortConfigPtr)
				{
					FDMXOutputPortConfigParams OutputPortConfigParams(*ExistingOutputPortConfigPtr);
					OutputPortConfigParams.CommunicationType = VoidController->DeviceProtocol == "Art-Net" ? EDMXCommunicationType::Broadcast : EDMXCommunicationType::Multicast;
					OutputPortConfigParams.bLoopbackToEngine = OutputPortConfigParams.CommunicationType == EDMXCommunicationType::Multicast ? true : false;
					OutputPortConfigParams.LocalUniverseStart = FMath::Min(ExistingOutputPortConfigPtr->GetLocalUniverseStart(), FixedLocalUniverseStart);
					OutputPortConfigParams.NumUniverses = FMath::Max(ExistingOutputPortConfigPtr->GetNumUniverses(), FixedNumUniverses);

					*ExistingOutputPortConfigPtr = FDMXOutputPortConfig(ExistingOutputPortConfigPtr->GetPortGuid(), OutputPortConfigParams);

					Local::EnableOutputPortRef(this, ExistingOutputPortConfigPtr->GetPortGuid());
				}
				else if (IDMXProtocol::Get(ProtocolName).IsValid())
				{
					FDMXOutputPortConfigParams OutputPortConfigParams;
					OutputPortConfigParams.PortName = FDMXProtocolUtils::GenerateUniqueNameFromExisting(OutputPortConfigNames, TEXT("Generated_OutputPort"));
					OutputPortConfigParams.CommunicationType = VoidController->CommunicationMode;
					OutputPortConfigParams.ProtocolName = ProtocolName;
					OutputPortConfigParams.DeviceAddress = InterfaceIPAddress_DEPRECATED;
					OutputPortConfigParams.bLoopbackToEngine = OutputPortConfigParams.CommunicationType == EDMXCommunicationType::Multicast ? true : false;
					OutputPortConfigParams.LocalUniverseStart = FixedLocalUniverseStart;
					OutputPortConfigParams.NumUniverses = FixedNumUniverses;
					OutputPortConfigParams.ExternUniverseStart = FixedExternUniverseStart;
					OutputPortConfigParams.Priority = 100;

					FDMXOutputPortConfig OutputPortConfig = FDMXOutputPortConfig(FGuid::NewGuid(), OutputPortConfigParams);

					ProtocolSettings->OutputPortConfigs.Add(OutputPortConfig);

					Local::EnableOutputPortRef(this, OutputPortConfig.GetPortGuid());
				}

				// Add ports from additional unicast ip
				if (VoidController->AdditionalUnicastIPs.Num() > 0)
				{
					ExistingOutputPortConfigPtr = ProtocolSettings->FindOutputPortConfig([VoidController, &InterfaceIPAddress_DEPRECATED, &ProtocolName](FDMXOutputPortConfig& OutputPortConfig) {
						return
							OutputPortConfig.GetDeviceAddress() == InterfaceIPAddress_DEPRECATED &&
							OutputPortConfig.GetProtocolName() == ProtocolName &&
							OutputPortConfig.GetCommunicationType() == EDMXCommunicationType::Unicast;
					});

					TArray<FDMXOutputPortDestinationAddress> DestinationAddressStructs;
					for (const FString& DestinationAddress : VoidController->AdditionalUnicastIPs)
					{
						DestinationAddressStructs.Add(FDMXOutputPortDestinationAddress(DestinationAddress));
					}

					if (ExistingOutputPortConfigPtr)
					{
						FDMXOutputPortConfigParams OutputPortConfigParams(*ExistingOutputPortConfigPtr);
						OutputPortConfigParams.CommunicationType = EDMXCommunicationType::Unicast;
						OutputPortConfigParams.DestinationAddresses = DestinationAddressStructs;
						OutputPortConfigParams.LocalUniverseStart = FMath::Min(ExistingOutputPortConfigPtr->GetLocalUniverseStart(), FixedLocalUniverseStart);
						OutputPortConfigParams.NumUniverses = FMath::Max(ExistingOutputPortConfigPtr->GetNumUniverses(), FixedNumUniverses);

						*ExistingOutputPortConfigPtr = FDMXOutputPortConfig(ExistingOutputPortConfigPtr->GetPortGuid(), OutputPortConfigParams);

						Local::EnableOutputPortRef(this, ExistingOutputPortConfigPtr->GetPortGuid());
					}
					else if (IDMXProtocol::Get(ProtocolName).IsValid())
					{
						FDMXOutputPortConfigParams OutputPortConfigParams;
						OutputPortConfigParams.PortName = FDMXProtocolUtils::GenerateUniqueNameFromExisting(OutputPortConfigNames, TEXT("Generated_OutputPort"));
						OutputPortConfigParams.CommunicationType = EDMXCommunicationType::Unicast;
						OutputPortConfigParams.ProtocolName = ProtocolName;
						OutputPortConfigParams.DeviceAddress = InterfaceIPAddress_DEPRECATED;
						OutputPortConfigParams.DestinationAddresses = DestinationAddressStructs;
						OutputPortConfigParams.bLoopbackToEngine = true;
						OutputPortConfigParams.LocalUniverseStart = FixedLocalUniverseStart;
						OutputPortConfigParams.NumUniverses = FixedNumUniverses;
						OutputPortConfigParams.ExternUniverseStart = FixedExternUniverseStart;
						OutputPortConfigParams.Priority = 100;

						FDMXOutputPortConfig OutputPortConfig = FDMXOutputPortConfig(FGuid::NewGuid(), OutputPortConfigParams);

						ProtocolSettings->OutputPortConfigs.Add(OutputPortConfig);

						Local::EnableOutputPortRef(this, OutputPortConfig.GetPortGuid());
					}
				}
			}
		}

		// Save the changes and print a log message
		ProtocolSettings->SaveConfig();		
		UE_LOGF(LogDMXRuntime, Log, "Upgraded DMX Library '%ls'. Controllers were replaced with Ports in Project Settings -> Plugins -> DMX. Please verify the library and project settings, and resave the library.", *GetName());

		// Apply the changes to the port manager
		FDMXPortManager::Get().UpdateFromProtocolSettings();

		// Apply the changes to this library
		UpdatePorts();
	}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
