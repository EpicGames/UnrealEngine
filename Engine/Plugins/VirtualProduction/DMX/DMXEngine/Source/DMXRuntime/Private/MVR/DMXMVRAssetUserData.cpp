// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVR/DMXMVRAssetUserData.h"

#include "GameFramework/Actor.h"
#include "Interfaces/Interface_AssetUserData.h"

const FName UDMXMVRAssetUserData::MVRFixtureUUIDMetaDataKey = TEXT("MVRFixtureUUID");
const FName UDMXMVRAssetUserData::FixturePatchMetaDataKey = TEXT("FixturePatch");


UDMXMVRAssetUserData* UDMXMVRAssetUserData::GetMVRAssetUserData(const AActor& Actor)
{
	if (UActorComponent* RootComponent = Actor.GetRootComponent())
	{
		return RootComponent->GetAssetUserData<UDMXMVRAssetUserData>();
	}

	return nullptr;
}

FString UDMXMVRAssetUserData::GetMVRAssetUserDataValueForKey(const AActor& Actor, const FName Key)
{
	if (UDMXMVRAssetUserData* MVRAssetUserData = GetMVRAssetUserData(Actor))
	{
		const FString* const ValuePtr = MVRAssetUserData->MetaData.Find(Key);
		return ValuePtr ? *ValuePtr : FString();
	}

	return FString();
}

bool UDMXMVRAssetUserData::SetMVRAssetUserDataValueForKey(AActor& InOutActor, FName InKey, const FString& InValue)
{
	// For AActor, the interface is actually implemented by the ActorComponent
	if (UActorComponent* RootComponent = InOutActor.GetRootComponent())
	{
		UDMXMVRAssetUserData* MVRUserData = RootComponent->GetAssetUserData<UDMXMVRAssetUserData>();
		if (!MVRUserData)
		{
			MVRUserData = NewObject<UDMXMVRAssetUserData>(&InOutActor, NAME_None, RF_Public | RF_Transactional);
			RootComponent->AddAssetUserData(MVRUserData);
		}

		// Add Datasmith meta data
		MVRUserData->MetaData.Add(InKey, InValue);
		MVRUserData->MetaData.KeySort(FNameLexicalLess());

		return true;
	}

	return false;
}
