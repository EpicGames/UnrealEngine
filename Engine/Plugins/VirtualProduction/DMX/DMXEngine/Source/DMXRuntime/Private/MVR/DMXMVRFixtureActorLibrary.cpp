// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVR/DMXMVRFixtureActorLibrary.h"

#include "AssetRegistry/AssetData.h"
#include "DMXMVRFixtureActorInterface.h"
#include "DMXRuntimeLog.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Library/DMXEntityFixturePatch.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "PreviewScene.h"
#include "Engine/SkeletalMesh.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/ScopedSlowTask.h"
#include "UObject/UObjectIterator.h"


class FDMXMVRFixtureActorAssetHierarchy
{
private:
	class FDMXVRAssetNode
		: public TSharedFromThis<FDMXVRAssetNode>
	{
	public:
		FDMXVRAssetNode(FTopLevelAssetPath InClassPath, FString InClassName)
			: ClassName(InClassName)
			, ClassPath(InClassPath)
		{}

		TSharedPtr<FDMXVRAssetNode> Parent;
		TArray<TSharedRef<FDMXVRAssetNode>> Children;

		void AppendClassPathsOfSelfAndChildren(TArray<FTopLevelAssetPath>& OutChildClassPaths) const
		{
			OutChildClassPaths.Add(ClassPath);
			for (const TSharedRef<FDMXVRAssetNode>& ChildNode : Children)
			{
				ChildNode->AppendClassPathsOfSelfAndChildren(OutChildClassPaths);
			}
		}

		FString ClassName;
		FTopLevelAssetPath ClassPath;
		FTopLevelAssetPath ParentClassPath;
		FSoftObjectPath BlueprintAssetPath;
		TArray<FString> ImplementedInterfaces;
	};

public:
	static void GetClassPathsWithInterface(const FString& InInterfaceClassPathName, TArray<FTopLevelAssetPath>& OutClassPaths)
	{
		FDMXMVRFixtureActorAssetHierarchy MVRAssetHierarchy;

		if (MVRAssetHierarchy.ObjectClassRoot.IsValid())
		{
			const TSharedRef<FDMXVRAssetNode> StartNode = MVRAssetHierarchy.ObjectClassRoot.ToSharedRef();
			MVRAssetHierarchy.GetClassPathsWithInterface(InInterfaceClassPathName, OutClassPaths, StartNode);
		}
	}

private:
	void GetClassPathsWithInterface(const FString& InInterfaceClassPathName, TArray<FTopLevelAssetPath>& OutClassPaths, const TSharedRef<FDMXVRAssetNode>& StartNode)
	{
		if (StartNode->ImplementedInterfaces.Contains(InInterfaceClassPathName))
		{
			StartNode->AppendClassPathsOfSelfAndChildren(OutClassPaths);
			return;
		}

		// Recursive on all children
		for (const TSharedRef<FDMXVRAssetNode>& ChildNode : StartNode->Children)
		{
			GetClassPathsWithInterface(InInterfaceClassPathName, OutClassPaths, ChildNode);
		}
	}

	FDMXMVRFixtureActorAssetHierarchy()
	{
		// Fetch all classes from AssetRegistry blueprint data (which covers unloaded classes), and in-memory UClasses.
		// Create a node for each one with unioned data from the AssetRegistry or UClass for that class.
		// Set parent/child pointers to create a tree, and store this tree in this->ObjectClassRoot
		TMap<FTopLevelAssetPath, TSharedPtr<FDMXVRAssetNode>> ClassPathToNode;

		// Create a node for every Blueprint class listed in the AssetRegistry and set the Blueprint fields
		// Retrieve all blueprint classes 
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		TArray<FAssetData> BlueprintList;
		FARFilter Filter;
		Filter.ClassPaths.Add(UBlueprint::StaticClass()->GetClassPathName());
		Filter.RecursiveClassPathsExclusionSet.FindOrAdd(USkeletalMesh::StaticClass()->GetClassPathName());
		Filter.bRecursiveClasses = true;

		AssetRegistryModule.Get().GetAssets(Filter, BlueprintList);

		FString ClassPathString;
		for (FAssetData& AssetData : BlueprintList)
		{
			FTopLevelAssetPath ClassPath;
			if (AssetData.GetTagValue(FBlueprintTags::GeneratedClassPath, ClassPathString))
			{
				ClassPath = FTopLevelAssetPath(FPackageName::ExportTextPathToObjectPath(ClassPathString));
			}
			if (ClassPath.IsNull())
			{
				continue;
			}

			TSharedPtr<FDMXVRAssetNode>& Node = ClassPathToNode.FindOrAdd(ClassPath);
			if (!Node.IsValid())
			{
				const FString ClassName = AssetData.AssetName.ToString();
				Node = MakeShared<FDMXVRAssetNode>(ClassPath, ClassName);
			}
			
			SetAssetDataFields(Node, AssetData);
		}

		// FindOrCreate a node for every loaded UClass, and set the UClass fields
		BuildHierarchyFromLoadedClassees(ObjectClassRoot, ClassPathToNode);

		// Set the parent and child pointers
		for (TPair<FTopLevelAssetPath, TSharedPtr<FDMXVRAssetNode>>& KVPair : ClassPathToNode)
		{
			TSharedPtr<FDMXVRAssetNode>& Node = KVPair.Value;
			if (Node == ObjectClassRoot)
			{
				// No parent expected for the root class
				continue;
			}
			TSharedPtr<FDMXVRAssetNode>* ParentNodePtr = nullptr;
			if (!Node->ParentClassPath.IsNull())
			{
				ParentNodePtr = ClassPathToNode.Find(Node->ParentClassPath);
			}
			if (!ParentNodePtr)
			{
				continue;
			}
			TSharedPtr<FDMXVRAssetNode>& ParentNode = *ParentNodePtr;
			check(ParentNode);
			ParentNode->Children.Add(Node.ToSharedRef());
		}
	}

	void BuildHierarchyFromLoadedClassees(TSharedPtr<FDMXVRAssetNode>& OutRootNode, TMap<FTopLevelAssetPath, TSharedPtr<FDMXVRAssetNode>>& InOutClassPathToNode)
	{
		for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
		{
			UClass* CurrentClass = *ClassIt;
			// Ignore deprecated and temporary trash classes.
			if (CurrentClass->HasAnyClassFlags(CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_Hidden))
			{
				continue;
			}
			TSharedPtr<FDMXVRAssetNode>& Node = InOutClassPathToNode.FindOrAdd(FTopLevelAssetPath(CurrentClass->GetPathName()));
			if (!Node.IsValid())
			{
				Node = MakeShared<FDMXVRAssetNode>(FTopLevelAssetPath(CurrentClass->GetPathName()), CurrentClass->GetPathName());
			}
			SetClassFields(Node, *CurrentClass);
		}
		TSharedPtr<FDMXVRAssetNode>* ExistingRoot = InOutClassPathToNode.Find(FTopLevelAssetPath(AActor::StaticClass()->GetPathName()));
		check(ExistingRoot && ExistingRoot->IsValid());
		OutRootNode = *ExistingRoot;
	}

	void SetClassFields(TSharedPtr<FDMXVRAssetNode>& InOutClassNode, UClass& Class)
	{
		// Fields that can als be set from FAssetData
		if (InOutClassNode->ClassPath.IsNull())
		{
			InOutClassNode->ClassPath = FTopLevelAssetPath(Class.GetPathName());
		}
		if (InOutClassNode->ParentClassPath.IsNull())
		{
			if (Class.GetSuperClass())
			{
				InOutClassNode->ParentClassPath = FTopLevelAssetPath(Class.GetSuperClass()->GetPathName());
			}
		}

		for (const FImplementedInterface& Interface : Class.Interfaces)
		{
			InOutClassNode->ImplementedInterfaces.Add(Interface.Class->GetPathName());
		}
	}

	TSharedPtr<FDMXVRAssetNode> FindNodeByGeneratedClassPath(const TSharedRef<FDMXVRAssetNode>& InRootNode, FTopLevelAssetPath InGeneratedClassPath)
	{
		if (InRootNode->ClassPath == InGeneratedClassPath)
		{
			return InRootNode;
		}

		TSharedPtr<FDMXVRAssetNode> ReturnNode;
		// Search the children recursively, one of them might have the parent.
		for (int32 ChildClassIndex = 0; !ReturnNode.IsValid() && ChildClassIndex < InRootNode->Children.Num(); ChildClassIndex++)
		{
			// Check the child, then check the return to see if it is valid. If it is valid, end the recursion.
			ReturnNode = FindNodeByGeneratedClassPath(InRootNode->Children[ChildClassIndex], InGeneratedClassPath);

			if (ReturnNode.IsValid())
			{
				return ReturnNode;
			}
		}

		return nullptr;
	}

	void SetAssetDataFields(TSharedPtr<FDMXVRAssetNode>& InOutMXVRAssetNode, const FAssetData& InAssetData)
	{
		if (InOutMXVRAssetNode->ClassPath.IsNull())
		{
			FString GeneratedClassPath;
			UClass* AssetClass = InAssetData.GetClass();
			if (AssetClass && AssetClass->IsChildOf(UBlueprintGeneratedClass::StaticClass()))
			{
				InOutMXVRAssetNode->ClassPath = InAssetData.GetSoftObjectPath().GetAssetPath();
			}
			else if (InAssetData.GetTagValue(FBlueprintTags::GeneratedClassPath, GeneratedClassPath))
			{
				InOutMXVRAssetNode->ClassPath = FTopLevelAssetPath(FPackageName::ExportTextPathToObjectPath(GeneratedClassPath));
			}
		}
		if (InOutMXVRAssetNode->ParentClassPath.IsNull())
		{
			FString ParentClassPathString;
			if (InAssetData.GetTagValue(FBlueprintTags::ParentClassPath, ParentClassPathString))
			{
				InOutMXVRAssetNode->ParentClassPath = FTopLevelAssetPath(FPackageName::ExportTextPathToObjectPath(ParentClassPathString));
			}
		}

		// Blueprint-specific fields
		InOutMXVRAssetNode->BlueprintAssetPath = InAssetData.GetSoftObjectPath();

		// Get interface class paths.
		TArray<FString> ImplementedInterfaces;
		GetImplementedInterfaceClassPathsFromAsset(InAssetData, ImplementedInterfaces);
		InOutMXVRAssetNode->ImplementedInterfaces = ImplementedInterfaces;
	}

	void GetImplementedInterfaceClassPathsFromAsset(const FAssetData& InAssetData, TArray<FString>& OutClassPaths)
	{
		if (!InAssetData.IsValid())
		{
			return;
		}

		const FString ImplementedInterfaces = InAssetData.GetTagValueRef<FString>(FBlueprintTags::ImplementedInterfaces);
		if (!ImplementedInterfaces.IsEmpty())
		{
			// Parse string like "((Interface=Class'"/Script/VPBookmark.VPBookmarkProvider"'),(Interface=Class'"/Script/VPUtilities.VPContextMenuProvider"'))"
			// We don't want to actually resolve the hard ref so do some manual parsing

			FString FullInterface;
			FString CurrentString = *ImplementedInterfaces;
			while (CurrentString.Split(TEXT("Interface="), nullptr, &FullInterface))
			{
				// Cutoff at next )
				int32 RightParen = INDEX_NONE;
				if (FullInterface.FindChar(TCHAR(')'), RightParen))
				{
					// Keep parsing
					CurrentString = FullInterface.Mid(RightParen);

					// Strip class name
					FullInterface = *FPackageName::ExportTextPathToObjectPath(FullInterface.Left(RightParen));

					// Handle quotes
					FString InterfacePath;
					const TCHAR* NewBuffer = FPropertyHelpers::ReadToken(*FullInterface, InterfacePath, true);

					if (NewBuffer)
					{
						OutClassPaths.Add(InterfacePath);
					}
				}
			}
		}
	}

	TSharedPtr<FDMXVRAssetNode> ObjectClassRoot;
};


FDMXMVRFixtureActorLibrary::FDMXMVRFixtureActorLibrary()
{
	TArray<FTopLevelAssetPath> ClassPaths;
	FDMXMVRFixtureActorAssetHierarchy::GetClassPathsWithInterface(UDMXMVRFixtureActorInterface::StaticClass()->GetPathName(), ClassPaths);
	ClassPaths.RemoveAll([](const FTopLevelAssetPath& ClassPath)
		{
			return
				ClassPath.ToString().Contains(TEXT(".SKEL_")) ||
				ClassPath.ToString().Contains(TEXT(".REINST_"));
		});

	FPreviewScene PreviewScene;
	TArray<UClass*> MVRFixtureActorClasses;
	for (const FTopLevelAssetPath& ClassPath : ClassPaths)
	{
		UClass* Class = LoadClass<UObject>(nullptr, *ClassPath.ToString());
		AActor* Actor = Class ? PreviewScene.GetWorld()->SpawnActor<AActor>(Class) : nullptr;
		
		if (Actor)
		{
			MVRActors.Add(Actor);
		}
	}
}

UClass* FDMXMVRFixtureActorLibrary::FindMostAppropriateActorClassForPatch(const UDMXEntityFixturePatch* Patch) const
{
	if (!ensureMsgf(Patch, TEXT("Invalid Fixture Patch provided when trying to find most appropriate actor class for patch. Cannot spawn Fixture Patch")))
	{
		return nullptr;
	}

	UDMXEntityFixtureType* FixtureType = Patch->GetFixtureType();
	if (!FixtureType)
	{
		// May be null. Don't ensure, just log
		UE_LOGF(LogDMXRuntime, Warning, "Cannot spawn Actor for Fixture Patch, Fixture Patch '%ls'. Fixture Patch has no valid Fixture Type.", *Patch->Name);
		return nullptr;
	}

#if WITH_EDITOR
	// Prefer the actor class set in the fixture type if available
	if (!FixtureType->ActorClassToSpawn.IsNull())
	{
		return FixtureType->ActorClassToSpawn.LoadSynchronous();
	}
#endif

	// Find the actor class with the most matching attributes
	const FDMXFixtureMode* const ModePtr = Patch->GetActiveMode();
	if (!ModePtr)
	{
		return nullptr;
	}

	TArray<FName> AttributesOfPatch;
	for (const FDMXFixtureFunction& Function : ModePtr->Functions)
	{
		AttributesOfPatch.Add(Function.Attribute.Name);
	}
	TArray<FName> MatrixAttributesOfPatch;
	if (ModePtr->bFixtureMatrixEnabled)
	{
		for (const FDMXFixtureCellAttribute& CellAttribute : ModePtr->FixtureMatrixConfig.CellAttributes)
		{
			MatrixAttributesOfPatch.Add(CellAttribute.Attribute.Name);
		}
	}

	// Best result
	AActor* BestMatch = nullptr;
	TArray<FName> BestMatchingAttributes;
	TArray<FName> BestMatchingMatrixAttributes;

	// Search through the actors
	TArray<FName> SupportedAttributes;
	TArray<FName> SupportedMatrixAttributes;
	TArray<FName> MatchingAttributes;
	TArray<FName> MatchingMatrixAttributes;
	for (auto& MVRActor : MVRActors)
	{
		// Look up attributes
		SupportedAttributes.Reset();
		SupportedMatrixAttributes.Reset();
		IDMXMVRFixtureActorInterface::Execute_OnMVRGetSupportedDMXAttributes(MVRActor, SupportedAttributes, SupportedMatrixAttributes);

		MatchingAttributes.Reset();
		MatchingMatrixAttributes.Reset();
		for (const FName& AttributeName : SupportedAttributes)
		{
			if (AttributesOfPatch.Contains(AttributeName))
			{
				MatchingAttributes.AddUnique(AttributeName);
			}
			else if (ModePtr->bFixtureMatrixEnabled && MatrixAttributesOfPatch.Contains(AttributeName))
			{
				MatchingMatrixAttributes.AddUnique(AttributeName);
			}
		}

		if (BestMatchingAttributes.Num() + BestMatchingMatrixAttributes.Num() < MatchingAttributes.Num() + MatchingMatrixAttributes.Num())
		{
			BestMatch = MVRActor;
			BestMatchingAttributes = MatchingAttributes;
			BestMatchingMatrixAttributes = MatchingMatrixAttributes;
		}
	}

	return BestMatch ? BestMatch->GetClass() : nullptr;
}

void FDMXMVRFixtureActorLibrary::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(MVRActors);
}
