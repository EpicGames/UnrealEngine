// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVR/DMXMVRGeneralSceneDescription.h"

#include "Algo/AnyOf.h"
#include "Algo/Find.h"
#include "Algo/MaxElement.h"
#include "EngineUtils.h"
#include "Game/DMXComponent.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/DMXImportGDTF.h"
#include "Library/DMXLibrary.h"
#include "MVR/DMXMVRAssetImportData.h"
#include "MVR/Types/DMXMVRChildListNode.h"
#include "MVR/Types/DMXMVRFixtureNode.h"
#include "MVR/Types/DMXMVRParametricObjectNodeBase.h"
#include "MVR/Types/DMXMVRRootNode.h"
#include "MVR/Types/DMXMVRSceneNode.h"
#include "XmlFile.h"

#define LOCTEXT_NAMESPACE "DMXMVRGeneralSceneDescription"

UDMXMVRGeneralSceneDescription::UDMXMVRGeneralSceneDescription()
{
	RootNode = CreateDefaultSubobject<UDMXMVRRootNode>("MVRRootNode");

#if WITH_EDITORONLY_DATA
	MVRAssetImportData = CreateDefaultSubobject<UDMXMVRAssetImportData>(TEXT("MVRAssetImportData"));
#endif
}

void UDMXMVRGeneralSceneDescription::GetFixtureNodes(TArray<UDMXMVRFixtureNode*>& OutFixtureNodes) const
{
	checkf(RootNode, TEXT("Unexpected: MVR General Scene Description Root Node is invalid."));

	OutFixtureNodes.Reset();
	RootNode->GetFixtureNodes(OutFixtureNodes);
}

UDMXMVRFixtureNode* UDMXMVRGeneralSceneDescription::FindFixtureNode(const FGuid& FixtureUUID) const
{
	checkf(RootNode, TEXT("Unexpected: MVR General Scene Description Root Node is invalid."));

	if (TObjectPtr<UDMXMVRParametricObjectNodeBase>* ObjectNodePtr = RootNode->FindParametricObjectNodeByUUID(FixtureUUID))
	{
		return Cast<UDMXMVRFixtureNode>(*ObjectNodePtr);
	}
	return nullptr;
}

#if WITH_EDITOR
UDMXMVRGeneralSceneDescription* UDMXMVRGeneralSceneDescription::CreateFromXmlFile(TSharedRef<FXmlFile> GeneralSceneDescriptionXml, UObject* Outer, FName Name, EObjectFlags Flags)
{
	UDMXMVRGeneralSceneDescription* GeneralSceneDescription = NewObject<UDMXMVRGeneralSceneDescription>(Outer, Name, Flags);
	if (GeneralSceneDescription->ParseGeneralSceneDescriptionXml(GeneralSceneDescriptionXml))
	{
		return GeneralSceneDescription;
	}

	return nullptr;
}
#endif // WITH_EDITOR

#if WITH_EDITOR
UDMXMVRGeneralSceneDescription* UDMXMVRGeneralSceneDescription::CreateFromDMXLibrary(const UDMXLibrary& DMXLibrary, UObject* Outer, FName Name, EObjectFlags Flags)
{
	UDMXMVRGeneralSceneDescription* GeneralSceneDescription = NewObject<UDMXMVRGeneralSceneDescription>(Outer, Name, Flags);
	GeneralSceneDescription->WriteDMXLibrary(DMXLibrary);

	return GeneralSceneDescription;
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UDMXMVRGeneralSceneDescription::WriteDMXLibraryToGeneralSceneDescription(const UDMXLibrary& DMXLibrary)
{
	// Deprecated 5.5, renamed to WriteDMXLibrary
	WriteDMXLibrary(DMXLibrary);
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UDMXMVRGeneralSceneDescription::WriteDMXLibrary(const UDMXLibrary& DMXLibrary, FDMXMVRGeneralSceneDescriptionWorldParams WorldParams)
{
	TArray<UDMXEntityFixturePatch*> FixturePatches = DMXLibrary.GetEntitiesTypeCast<UDMXEntityFixturePatch>();
	Algo::Sort(FixturePatches, [](const UDMXEntityFixturePatch* FixturePatchA, const UDMXEntityFixturePatch* FixturePatchB)
		{
			const uint64 AbsoluteAddressA = static_cast<uint64>(FixturePatchA->GetUniverseID()) * DMX_UNIVERSE_SIZE + FixturePatchA->GetStartingChannel();
			const uint64 AbsoluteAddressB = static_cast<uint64>(FixturePatchB->GetUniverseID()) * DMX_UNIVERSE_SIZE + FixturePatchB->GetStartingChannel();

			return AbsoluteAddressA <= AbsoluteAddressB;
		});
	FixturePatches.Remove(nullptr);

	// Remove Fixture Nodes no longer defined in the DMX Library
	TArray<FGuid> MVRFixtureUUIDsInUse;
	for (UDMXEntityFixturePatch* FixturePatch : FixturePatches)
	{
		MVRFixtureUUIDsInUse.Add(FixturePatch->GetMVRFixtureUUID());
	}

	TArray<UDMXMVRFixtureNode*> FixtureNodes;
	RootNode->GetFixtureNodes(FixtureNodes);
	for (UDMXMVRFixtureNode* FixtureNode : FixtureNodes)
	{
		if (!FixtureNode || !MVRFixtureUUIDsInUse.Contains(FixtureNode->UUID))
		{
			RootNode->RemoveParametricObjectNode(FixtureNode);
		}
	}

	// Write all Patches in library
	for (UDMXEntityFixturePatch* FixturePatch : FixturePatches)
	{
		if (FixturePatch)
		{
			WriteFixturePatch(*FixturePatch, FixturePatch->GetDefaultTransform());
		}
	}
	
	// Only export the library if there's no world
	if (!WorldParams.World)
	{
		return;
	}

	// Consider the world 
	const TMap<const UDMXComponent*, const AActor*> DMXComponentToActorMap = GetDMXComponentToActorMap(WorldParams.World);

	// Create fixtures and write transforms if desired
	for (const UDMXEntityFixturePatch* FixturePatchInLibrary : FixturePatches)
	{
		FGuid MultiPatchUUID;
		for (const TTuple<const UDMXComponent*, const AActor*>& DMXComponentToActorPair : DMXComponentToActorMap)
		{
			const UDMXEntityFixturePatch* FixturePatchInWorld = DMXComponentToActorPair.Key ? DMXComponentToActorPair.Key->GetFixturePatch() : nullptr;
			const AActor* Actor = DMXComponentToActorPair.Value;
			if (!FixturePatchInWorld || !Actor || FixturePatchInWorld != FixturePatchInLibrary)
			{
				continue;
			}

			if (!MultiPatchUUID.IsValid())
			{
				// Remember the MVR UUID of the first patch as multi patch UUID
				MultiPatchUUID = FixturePatchInWorld->GetMVRFixtureUUID();

				const FTransform Transform = WorldParams.bUseTransformsFromLevel ? Actor->GetTransform() : FixturePatchInLibrary->GetDefaultTransform();
				WriteFixturePatch(*FixturePatchInLibrary, Transform);
			}
			else if (WorldParams.bCreateMultiPatchFixtures)
			{
				const FTransform Transform = WorldParams.bUseTransformsFromLevel ? Actor->GetTransform() : FixturePatchInLibrary->GetDefaultTransform();
				WriteFixturePatch(*FixturePatchInLibrary, Transform, MultiPatchUUID);
			}
		}
	}
	
	// Update fixture nodes to contain newly added multi patch fixtures
	RootNode->GetFixtureNodes(FixtureNodes); 


	// Remove patches not present in the world if desired
	if (!WorldParams.bExportPatchesNotPresentInWorld)
	{
		for (const UDMXEntityFixturePatch* FixturePatchInLibrary : FixturePatches)
		{
			const bool bPatchUsedInWorldAndLibrary = Algo::AnyOf(DMXComponentToActorMap, [FixturePatchInLibrary](const TTuple<const UDMXComponent*, const AActor*>& DMXComponentToActorPair)
				{
					return DMXComponentToActorPair.Key && DMXComponentToActorPair.Key->GetFixturePatch() == FixturePatchInLibrary;
				});					

			if (!bPatchUsedInWorldAndLibrary && FixturePatchInLibrary)
			{
				for (UDMXMVRFixtureNode* FixtureNode : FixtureNodes)
				{
					if (!FixtureNode)
					{
						continue;
					}

					if (FixtureNode->UUID == FixturePatchInLibrary->GetMVRFixtureUUID() ||
						(FixtureNode->MultiPatch.IsSet() && FixtureNode->MultiPatch.GetValue() == FixturePatchInLibrary->GetMVRFixtureUUID()))
					{
						RootNode->RemoveParametricObjectNode(FixtureNode);
					}
				}
			}
		}
	}
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UDMXMVRGeneralSceneDescription::RemoveFixtureNode(const FGuid& FixtureUUID)
{
	if (!ensureMsgf(RootNode, TEXT("Unexpected: MVR General Scene Description Root Node is invalid.")))
	{
		return;
	}

	const TObjectPtr<UDMXMVRParametricObjectNodeBase>* ParametricObjectNodePtr = RootNode->FindParametricObjectNodeByUUID(FixtureUUID);
	if (ParametricObjectNodePtr && (*ParametricObjectNodePtr)->GetClass() == UDMXMVRFixtureNode::StaticClass())
	{
		RootNode->RemoveParametricObjectNode(*ParametricObjectNodePtr);
	}
}
#endif // WITH_EDITOR

#if WITH_EDITOR
bool UDMXMVRGeneralSceneDescription::CanCreateXmlFile(FText& OutReason) const
{
	TArray<UDMXMVRFixtureNode*> FixtureNodes;
	GetFixtureNodes(FixtureNodes);
	if (FixtureNodes.IsEmpty())
	{
		OutReason = LOCTEXT("CannotCreateXmlFileBecauseNoFixtures", "DMX Library does not define valid MVR fixtures.");
		return false;
	}
	return true;
}
#endif // WITH_EDITOR

#if WITH_EDITOR
TSharedPtr<FXmlFile> UDMXMVRGeneralSceneDescription::CreateXmlFile() const
{
	return RootNode ? RootNode->CreateXmlFile() : nullptr;
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UDMXMVRGeneralSceneDescription::WriteFixturePatch(const UDMXEntityFixturePatch& FixturePatch, const FTransform& Transform, const FGuid& MultiPatchUUID)
{
	TArray<UDMXMVRFixtureNode*> FixtureNodes;
	RootNode->GetFixtureNodes(FixtureNodes);

	UDMXMVRFixtureNode* MVRFixtureNode = nullptr;
	const UDMXMVRFixtureNode* const* ParentMultiPatchFixtureNodePtr = [&FixtureNodes, &MultiPatchUUID]() -> const UDMXMVRFixtureNode* const*
		{
			if (MultiPatchUUID.IsValid())
			{
				const UDMXMVRFixtureNode* const* ResultPtr = Algo::FindByPredicate(FixtureNodes, [&MultiPatchUUID](const UDMXMVRFixtureNode* Other)
					{
						return Other->UUID == MultiPatchUUID;
					});
				ensureMsgf(ResultPtr, TEXT("MultiPatch UUID provided but no corresponding node can be found"));

				return ResultPtr;
			}
			
			return nullptr;
		}(); 

	if (ParentMultiPatchFixtureNodePtr)
	{
		// Create a multipatch fixture
		UDMXMVRChildListNode& AnyChildList = RootNode->GetOrCreateFirstChildListNode();
		MVRFixtureNode = AnyChildList.CreateParametricObject<UDMXMVRFixtureNode>();

		MVRFixtureNode->Name = FixturePatch.Name;
		MVRFixtureNode->UUID = FGuid::NewGuid();
		MVRFixtureNode->MultiPatch = (*ParentMultiPatchFixtureNodePtr)->UUID;
	}
	else
	{
		const FGuid& MVRFixtureUUID = FixturePatch.GetMVRFixtureUUID();

		// Find an existing fixture
		const TObjectPtr<UDMXMVRParametricObjectNodeBase>* ParametricObjectNodePtr = RootNode->FindParametricObjectNodeByUUID(MVRFixtureUUID);
		MVRFixtureNode = ParametricObjectNodePtr ? Cast<UDMXMVRFixtureNode>(*ParametricObjectNodePtr) : nullptr;

		if (!MVRFixtureNode)
		{
			// Create a new fixture
			UDMXMVRChildListNode& AnyChildList = RootNode->GetOrCreateFirstChildListNode();
			MVRFixtureNode = AnyChildList.CreateParametricObject<UDMXMVRFixtureNode>();

			MVRFixtureNode->Name = FixturePatch.Name;
			MVRFixtureNode->Name = FixturePatch.Name;
			MVRFixtureNode->UUID = MVRFixtureUUID;
			MVRFixtureNode->FixtureID = FString::FromInt(FixturePatch.GetFixtureID());
		}
	}
	check(MVRFixtureNode)

	MVRFixtureNode->SetTransformAbsolute(Transform);
	MVRFixtureNode->SetUniverseID(FixturePatch.GetUniverseID());
	MVRFixtureNode->SetStartingChannel(FixturePatch.GetStartingChannel());

	UDMXEntityFixtureType* FixtureType = FixturePatch.GetFixtureType();
	const int32 ModeIndex = FixturePatch.GetActiveModeIndex();
	bool bSetGDTFSpec = false;

	if (FixtureType && FixtureType->Modes.IsValidIndex(ModeIndex))
	{
		// Instead refer to the generated file name
		MVRFixtureNode->GDTFMode = FixtureType->Modes[ModeIndex].ModeName;

		constexpr bool bWithExtension = false;
		MVRFixtureNode->GDTFSpec = FixtureType->GetCleanGDTFFileNameSynchronous(bWithExtension);

		bSetGDTFSpec = true;
	}

	if (!bSetGDTFSpec)
	{
		// Don't set a mode when there's no GDTF
		MVRFixtureNode->GDTFMode = TEXT("");
		MVRFixtureNode->GDTFSpec = TEXT("");
	}

	SanetizeFixtureNode(*MVRFixtureNode);
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UDMXMVRGeneralSceneDescription::SanetizeFixtureNode(UDMXMVRFixtureNode& FixtureNode)
{
	TArray<UDMXMVRFixtureNode*> FixtureNodes;
	GetFixtureNodes(FixtureNodes);

	if (FixtureNode.MultiPatch.IsSet())
	{
		FixtureNode.FixtureID = TEXT("");
		FixtureNode.CustomId.Reset();
	}
	else
	{
		const bool bInvalidFixtureID = Algo::AnyOf(FixtureNodes, [&FixtureNode](const UDMXMVRFixtureNode* Other)
			{
				return
					Other != &FixtureNode &&
					Other->FixtureID == FixtureNode.FixtureID;
			});
		if (bInvalidFixtureID)
		{
			const UDMXMVRFixtureNode* const* MaxFixtureNodePtr = Algo::MaxElementBy(FixtureNodes, [](const UDMXMVRFixtureNode* FixtureNode)
				{
					int32 IntegralFixtureID;
					return LexTryParseString(IntegralFixtureID, *FixtureNode->FixtureID) ? IntegralFixtureID : 1;
				});

			FixtureNode.FixtureID = MaxFixtureNodePtr ? (*MaxFixtureNodePtr)->FixtureID : TEXT("1");
		}

		const bool bInvalidMVRUUID = Algo::AnyOf(FixtureNodes, [&FixtureNode](const UDMXMVRFixtureNode* Other)
			{
				return
					Other != &FixtureNode &&
					Other->UUID == FixtureNode.UUID;
			});
		if (bInvalidMVRUUID)
		{
			FixtureNode.UUID = FGuid::NewGuid();
		}
	}
}
#endif // WITH_EDITOR

#if WITH_EDITOR
bool UDMXMVRGeneralSceneDescription::ParseGeneralSceneDescriptionXml(const TSharedRef<FXmlFile>& GeneralSceneDescriptionXml)
{
	checkf(RootNode, TEXT("Unexpected: MVR General Scene Description Root Node is invalid."));

	return RootNode->InitializeFromGeneralSceneDescriptionXml(GeneralSceneDescriptionXml);
}
#endif // WITH_EDITOR

#if WITH_EDITOR
TMap<const UDMXComponent*, const AActor*> UDMXMVRGeneralSceneDescription::GetDMXComponentToActorMap(const UWorld* World) const
{
	TMap<const UDMXComponent*, const AActor*> DMXComponentToActorMap;
	if (!World)
	{
		return DMXComponentToActorMap;
	}

	for (TActorIterator<AActor> It(World, AActor::StaticClass()); It; ++It)
	{
		const AActor* Actor = *It;
		if (!Actor)
		{
			continue;
		}

		const TSet<UActorComponent*> Components = Actor->GetComponents();

		TArray<const UDMXComponent*> DMXComponents;
		Algo::TransformIf(Components, DMXComponents,
			[](const UActorComponent* Component)
			{
				return Component &&
					Component->IsA(UDMXComponent::StaticClass()) &&
					CastChecked<UDMXComponent>(Component)->GetFixturePatch();
			},
			[](const UActorComponent* Component)
			{
				return CastChecked<UDMXComponent>(Component);
			});

		for (const UDMXComponent* DMXComponent : DMXComponents)
		{
			DMXComponentToActorMap.Add(TTuple<const UDMXComponent*, const AActor*>(DMXComponent, Actor));
		}
	}

	return DMXComponentToActorMap;
}
#endif // WITH_EDITOR

TArray<int32> UDMXMVRGeneralSceneDescription::GetNumericalFixtureIDsInUse(const UDMXLibrary& DMXLibrary)
{
	checkf(RootNode, TEXT("Unexpected: MVR General Scene Description Root Node is invalid."));

	TArray<UDMXEntityFixturePatch*> FixturePatches = DMXLibrary.GetEntitiesTypeCast<UDMXEntityFixturePatch>();

	TArray<FGuid> MVRFixtureUUIDsInUse;
	for (UDMXEntityFixturePatch* FixturePatch : FixturePatches)
	{
		MVRFixtureUUIDsInUse.Add(FixturePatch->GetMVRFixtureUUID());
	}

	TArray<int32> FixtureIDsInUse;
	for (const FGuid& MVRFixtureUUID : MVRFixtureUUIDsInUse)
	{
		if (TObjectPtr<UDMXMVRParametricObjectNodeBase>* ObjectNodePtr = RootNode->FindParametricObjectNodeByUUID(MVRFixtureUUID))
		{
			if (UDMXMVRFixtureNode* FixtureNode = Cast<UDMXMVRFixtureNode>(*ObjectNodePtr))
			{
				int32 FixtureIDNumerical;
				if (LexTryParseString(FixtureIDNumerical, *FixtureNode->FixtureID))
				{
					FixtureIDsInUse.Add(FixtureIDNumerical);
				}
			}
		}
	}

	FixtureIDsInUse.Sort([](int32 FixtureIDA, int32 FixtureIDB)
		{
			return FixtureIDA < FixtureIDB;
		});

	return FixtureIDsInUse;
}

#undef LOCTEXT_NAMESPACE
