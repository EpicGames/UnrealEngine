// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVR/DMXMVRSceneActor.h"

#include "Algo/Find.h"
#include "Algo/RemoveIf.h"
#include "Components/SceneComponent.h"
#include "DatasmithAssetUserData.h"
#include "DMXMVRFixtureActorInterface.h"
#include "DMXRuntimeLog.h"
#include "DMXRuntimeMainStreamObjectVersion.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "Game/DMXComponent.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXImportGDTF.h"
#include "Library/DMXLibrary.h"
#include "MVR/DMXMVRAssetUserData.h"
#include "MVR/DMXMVRFixtureActorLibrary.h"
#include "MVR/Types/DMXMVRFixtureNode.h"
#include "UObject/UObjectIterator.h"

#if WITH_EDITOR
#include "Editor.h"
#endif 


ADMXMVRSceneActor::ADMXMVRSceneActor()
{
#if WITH_EDITOR
	if (IsTemplate())
	{
		return;
	}

	FEditorDelegates::MapChange.AddUObject(this, &ADMXMVRSceneActor::OnMapChange);

	if (GEngine)
	{
		GEngine->OnLevelActorDeleted().AddUObject(this, &ADMXMVRSceneActor::OnActorDeleted);
	}

	if (GEditor)
	{
		GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetPostImport.AddUObject(this, &ADMXMVRSceneActor::OnAssetPostImport);
	}
#endif // WITH_EDITOR

	MVRSceneRoot = CreateDefaultSubobject<USceneComponent>("MVRSceneRoot");
	SetRootComponent(MVRSceneRoot);
	AddInstanceComponent(MVRSceneRoot);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS // Allow destruct complex deprecated members
ADMXMVRSceneActor::~ADMXMVRSceneActor()
{
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR
	if (IsTemplate())
	{
		return;
	}

	FEditorDelegates::MapChange.RemoveAll(this);

	if (GEngine)
	{
		GEngine->OnLevelActorDeleted().RemoveAll(this);
	}

	if (GEditor)
	{
		GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetPostImport.RemoveAll(this);
	}
#endif // WITH_EDITOR
}

void ADMXMVRSceneActor::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FDMXRuntimeMainStreamObjectVersion::GUID);
	
#if WITH_EDITOR
	if (Ar.IsLoading())
	{
		// Upgrade from actors per Fixture Type instead of per GDTF 
		if (Ar.CustomVer(FDMXRuntimeMainStreamObjectVersion::GUID) < FDMXRuntimeMainStreamObjectVersion::DMXMVRSceneActorSpawnsActorsPerFixtureType)
		{
			UpgradeToFixtureTypeToActorClasses();
		}
	}
#endif 
}

void ADMXMVRSceneActor::PostLoad()
{
	Super::PostLoad();
	EnsureMVRUUIDsForRelatedActors();
}

void ADMXMVRSceneActor::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

#if WITH_EDITOR
	// If the actor was created as a Datasmith Element, set the library from there
	const FString DMXLibraryPathString = UDatasmithAssetUserData::GetDatasmithUserDataValueForKey(this, TEXT("DMXLibraryPath"));
	if (!DMXLibraryPathString.IsEmpty() && !DMXLibrary)
	{
		const FSoftObjectPath DMXLibraryPath(DMXLibraryPathString);
		UObject* NewDMXLibraryObject = DMXLibraryPath.TryLoad();
		if (UDMXLibrary* NewDMXLibrary = Cast<UDMXLibrary>(NewDMXLibraryObject))
		{
			SetDMXLibrary(NewDMXLibrary);
		}
	}

	EnsureMVRUUIDsForRelatedActors();
#endif // WITH_EDITOR
}

#if WITH_EDITOR
void ADMXMVRSceneActor::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (PropertyAboutToChange && PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(FDMXMVRSceneFixtureTypeToActorClassPair, ActorClass))
	{
		FixtureTypeToActorClasses_PreEditChange = FixtureTypeToActorClasses;
	}
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void ADMXMVRSceneActor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(FDMXMVRSceneFixtureTypeToActorClassPair, ActorClass))
	{
		HandleDefaultActorClassForFixtureTypeChanged();
	}
}
#endif // WITH_EDITOR

UDMXEntityFixturePatch* ADMXMVRSceneActor::GetFixturePatchFromActor(AActor* Actor) const
{
	TArray<UDMXComponent*> DMXComponents;
	Actor->GetComponents<UDMXComponent>(DMXComponents);
	if (!ensureAlwaysMsgf(!DMXComponents.IsEmpty(), TEXT("'%s' implements the DMXMVRFixtureActorInterface, but has no DMX component. The DMX Component is needed to patch and identify the fixture in the MVR Scene."), *Actor->GetName()))
	{
		return nullptr;
	}
	ensureAlwaysMsgf(DMXComponents.Num() == 1, TEXT("'%s' implements the DMXMVRFixtureActorInterface, but has more than one DMX component. A single DMX component is required to clearly identify the fixture by MVR UUID in the MVR Scene."), *Actor->GetName());
	
	UDMXEntityFixturePatch* FixturePatch = DMXComponents[0]->GetFixturePatch();

	return FixturePatch;
}

void ADMXMVRSceneActor::SetFixturePatchOnActor(AActor* Actor, UDMXEntityFixturePatch* FixturePatch)
{
	if (!ensureMsgf(Actor && FixturePatch, TEXT("Trying to Set Fixture Patch on Actor, but Actor or Fixture Patch are invalid.")))
	{
		return;
	}

	TArray<UDMXComponent*> DMXComponents;
	Actor->GetComponents<UDMXComponent>(DMXComponents);
	if (!ensureAlwaysMsgf(!DMXComponents.IsEmpty(), TEXT("'%s' implements the DMXMVRFixtureActorInterface, but has no DMX component. The DMX Component is needed to patch and identify the fixture in the MVR Scene."), *Actor->GetName()))
	{
		return;
	}
	ensureAlwaysMsgf(DMXComponents.Num() == 1, TEXT("'%s' implements the DMXMVRFixtureActorInterface, but has more than one DMX component. A single DMX component is required to clearly identify the fixture by MVR UUID in the MVR Scene."), *Actor->GetName());

	DMXComponents[0]->SetFixturePatch(FixturePatch);
}


#if WITH_EDITOR
void ADMXMVRSceneActor::SetDMXLibrary(UDMXLibrary* NewDMXLibrary)
{
	if (!ensureAlwaysMsgf(!DMXLibrary, TEXT("Tried to set the DMXLibrary for %s, but it already has one set. Changing the library is not supported."), *GetName()))
	{
		return;
	}

	if (!NewDMXLibrary || NewDMXLibrary == DMXLibrary)
	{
		return;
	}
	DMXLibrary = NewDMXLibrary;

	RefreshFromDMXLibrary();
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void ADMXMVRSceneActor::RefreshFromDMXLibrary()
{
	if (!ensureMsgf(DMXLibrary, TEXT("Trying to update MVR Scene from DMX Library, but DMX Library was never set or no longer exists.")))
	{
		return;
	}

	DMXLibrary->UpdateGeneralSceneDescription();
	UDMXMVRGeneralSceneDescription* GeneralSceneDescription = DMXLibrary->GetLazyGeneralSceneDescription();
	if (!GeneralSceneDescription)
	{
		return;
	}

	const UWorld* World = GetWorld();
	if (!ensureMsgf(World, TEXT("Invalid world when trying to update MVR Scene from DMX Library.")))
	{
		return;
	}

	if (bRespawnDeletedActorsOnRefresh)
	{
		DeletedMVRFixtureUUIDs.Reset();
	}

	const TSharedRef<FDMXMVRFixtureActorLibrary> MVRFixtureActorLibrary = MakeShared<FDMXMVRFixtureActorLibrary>();
	const TArray<UDMXEntityFixturePatch*> FixturePatches = DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>();

	// Destroy actors no longer present in the DMX Library, remember any other spawned actor
	TMap<UDMXEntityFixturePatch*, AActor*> FixturePatchToSpawnedActorMap;
	for (const TSoftObjectPtr<AActor>& SoftActorPtr : RelatedActors)
	{
		if (!SoftActorPtr.IsValid())
		{
			continue;
		}
		AActor* Actor = SoftActorPtr.Get();

		UDMXEntityFixturePatch* FixturePatch = GetFixturePatchFromActor(Actor);
		if (!FixturePatch || !FixturePatches.Contains(FixturePatch))
		{
			Actor->Destroy();
			continue;
		}

		FixturePatchToSpawnedActorMap.Add(FixturePatch, Actor);
	}

	// Spawn newly added and if requested, previously deleted actors
	for (UDMXEntityFixturePatch* FixturePatch : FixturePatches)
	{
		if (!FixturePatch ||
			FixturePatchToSpawnedActorMap.Contains(FixturePatch))
		{
			continue;
		}

		if (!bRespawnDeletedActorsOnRefresh &&
			DeletedMVRFixtureUUIDs.Contains(FixturePatch->GetMVRFixtureUUID()))
		{
			continue;
		}

		const UDMXEntityFixtureType* FixtureType = FixturePatch->GetFixtureType();
		if (!FixtureType)
		{
			continue;
		}

		const FDMXMVRSceneFixtureTypeToActorClassPair* FixtureTypeToActorClassPairPtr = 
			Algo::FindBy(FixtureTypeToActorClasses, FixtureType, &FDMXMVRSceneFixtureTypeToActorClassPair::FixtureType);

		UClass* ActorClass = FixtureTypeToActorClassPairPtr ? 
			(*FixtureTypeToActorClassPairPtr).ActorClass.LoadSynchronous() : 
			MVRFixtureActorLibrary->FindMostAppropriateActorClassForPatch(FixturePatch);

		if (!ActorClass)
		{
			continue;
		}

		const FGuid& MVRFixtureUUID = FixturePatch->GetMVRFixtureUUID();
		const UDMXMVRFixtureNode* FixtureNode = GeneralSceneDescription->FindFixtureNode(MVRFixtureUUID);
		const FTransform Transform = FixtureNode ? FixtureNode->GetTransformAbsolute() : FTransform::Identity;

		SpawnMVRActor(ActorClass, FixturePatch, Transform);
	}

	// Update Transforms if requested
	if (bUpdateTransformsOnRefresh)
	{
		for (const TSoftObjectPtr<AActor>& SoftRelatedActor : RelatedActors)
		{
			if (!SoftRelatedActor.IsValid())
			{
				continue;
			}

			AActor* RelatedActor = SoftRelatedActor.Get();
			UDMXEntityFixturePatch* FixturePatch = GetFixturePatchFromActor(RelatedActor);
			if (!FixturePatch)
			{
				continue;
			}

			const FGuid& MVRFixtureUUID = FixturePatch->GetMVRFixtureUUID();
			const UDMXMVRFixtureNode* FixtureNode = GeneralSceneDescription->FindFixtureNode(MVRFixtureUUID);
			const FTransform Transform = FixtureNode ? FixtureNode->GetTransformAbsolute() : FTransform::Identity;

			RelatedActor->SetActorTransform(Transform);
		}
	}

	UpdateFixtureTypeToDefaultActorClasses(MVRFixtureActorLibrary);
}
#endif // WITH_EDITOR

#if WITH_EDITOR
TArray<AActor*> ADMXMVRSceneActor::GetActorsSpawnedForFixtureType(const UDMXEntityFixtureType* FixtureType) const
{
	// DEPRECATED 5.5
	TArray<AActor*> Result;
	if (!FixtureType)
	{
		return Result;
	}

	for (const TSoftObjectPtr<AActor>& SoftActorPtr : RelatedActors)
	{
		if (!SoftActorPtr.IsValid())
		{
			continue;
		}
		AActor* Actor = SoftActorPtr.Get();

		UDMXEntityFixturePatch* FixturePatch = GetFixturePatchFromActor(Actor);
		if (!FixturePatch || !FixturePatch->GetFixtureType())
		{
			continue;
		}

		if (FixturePatch->GetFixtureType() == FixtureType)
		{
			Result.Add(Actor);
		}
	}

	return Result;
}
#endif // WITH_EDITOR

#if WITH_EDITOR
TArray<AActor*> ADMXMVRSceneActor::GetActorsSpawnedForGDTF(const UDMXImportGDTF* GDTF) const
{
	TArray<AActor*> Result;
	if (!GDTF)
	{
		return Result;
	}

	for (const TSoftObjectPtr<AActor>& SoftActorPtr : RelatedActors)
	{
		if (!SoftActorPtr.IsValid())
		{
			continue;
		}
		AActor* Actor = SoftActorPtr.Get();

		UDMXEntityFixturePatch* FixturePatch = GetFixturePatchFromActor(Actor);
		if (!FixturePatch || !FixturePatch->GetFixtureType())
		{
			continue;
		}

		if (FixturePatch->GetFixtureType()->GDTFSource == GDTF)
		{
			Result.Add(Actor);
		}
	}

	return Result;
}
#endif // WITH_EDITOR

void ADMXMVRSceneActor::EnsureMVRUUIDsForRelatedActors()
{
	for (const TSoftObjectPtr<AActor>& RelatedActor : RelatedActors)
	{
		if (AActor* Actor = RelatedActor.Get())
		{
			const FString MVRFixtureUUID = UDMXMVRAssetUserData::GetMVRAssetUserDataValueForKey(*Actor, UDMXMVRAssetUserData::MVRFixtureUUIDMetaDataKey);
			if (MVRFixtureUUID.IsEmpty())
			{
				// Try to acquire the MVR Fixture UUID
				if (UDMXEntityFixturePatch* FixturePatch = GetFixturePatchFromActor(Actor))
				{
					UDMXMVRAssetUserData::SetMVRAssetUserDataValueForKey(*Actor, UDMXMVRAssetUserData::MVRFixtureUUIDMetaDataKey, FixturePatch->GetMVRFixtureUUID().ToString());
				}
			}
		}
	}
}

#if WITH_EDITOR
void ADMXMVRSceneActor::UpdateFixtureTypeToDefaultActorClasses(const TSharedRef<FDMXMVRFixtureActorLibrary>& MVRFixtureActorLibrary)
{
	const TArray<UDMXEntityFixturePatch*> FixturePatches = DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>();
	for (UDMXEntityFixturePatch* FixturePatch : FixturePatches)
	{
		UDMXEntityFixtureType* FixtureType = FixturePatch->GetFixtureType();
		if (FixtureType)
		{
			const FDMXMVRSceneFixtureTypeToActorClassPair* ExistingFixtureTypeToActorClassPairPtr = Algo::FindByPredicate(FixtureTypeToActorClasses, [FixtureType](const FDMXMVRSceneFixtureTypeToActorClassPair& FixtureTypeToActorClassPair)
				{
					return FixtureTypeToActorClassPair.FixtureType == FixtureType;
				});
			if (ExistingFixtureTypeToActorClassPairPtr)
			{
				continue;
			}

			UClass* ActorClass = MVRFixtureActorLibrary->FindMostAppropriateActorClassForPatch(FixturePatch);
			FDMXMVRSceneFixtureTypeToActorClassPair FixtureTypeToActorClassPair;
			FixtureTypeToActorClassPair.FixtureType = FixtureType;
			FixtureTypeToActorClassPair.ActorClass = ActorClass;

			FixtureTypeToActorClasses.Add(FixtureTypeToActorClassPair);
		}
	}
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void ADMXMVRSceneActor::OnMapChange(uint32 MapEventFlags)
{
	// Whenever a sub-level is loaded, we need to apply the fix
	if (MapEventFlags == MapChangeEventFlags::NewMap)
	{
		EnsureMVRUUIDsForRelatedActors();
	}
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void ADMXMVRSceneActor::OnActorDeleted(AActor* DeletedActor)
{
	if (DeletedActor == this)
	{
		for (const TSoftObjectPtr<AActor>& RelatedActor : RelatedActors)
		{
			if (AActor* Actor = RelatedActor.IsValid() ? RelatedActor.Get() : nullptr)
			{
				Actor->Modify();
				Actor->Destroy();
			}
		}
	}
	else
	{
		const int32 RelatedActorIndex = RelatedActors.Find(DeletedActor);
		if (RelatedActorIndex != INDEX_NONE)
		{
			// This will add this actor to the transaction if there is one currently recording
			Modify();

			const UDMXEntityFixturePatch* FixturePatch = GetFixturePatchFromActor(DeletedActor);
			if (FixturePatch)
			{
				DeletedMVRFixtureUUIDs.Add(FixturePatch->GetMVRFixtureUUID());
			}

			RelatedActors[RelatedActorIndex]->Reset();
		}
	}
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void ADMXMVRSceneActor::OnAssetPostImport(UFactory* InFactory, UObject* ActorAdded)
{
	for (TObjectIterator<AActor> It; It; ++It)
	{
		AActor* Actor = *It;

		const int32 RelatedActorIndex = RelatedActors.Find(Actor);
		if (RelatedActorIndex != INDEX_NONE)
		{
			// This will add this actor to the transaction if there is one currently recording
			Modify();

			RelatedActors[RelatedActorIndex] = Actor;
		}
	}
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void ADMXMVRSceneActor::HandleDefaultActorClassForFixtureTypeChanged()
{
	// Handle element changes, but not add/remove
	if (FixtureTypeToActorClasses_PreEditChange.Num() != FixtureTypeToActorClasses.Num())
	{
		return;
	}

	int32 IndexOfChangedElement = INDEX_NONE;
	for (const FDMXMVRSceneFixtureTypeToActorClassPair& FixtureTypeToActorClassPair : FixtureTypeToActorClasses)
	{
		IndexOfChangedElement = FixtureTypeToActorClasses_PreEditChange.IndexOfByPredicate([&FixtureTypeToActorClassPair](const FDMXMVRSceneFixtureTypeToActorClassPair& OtherFixtureTypeoActorClassPair)
			{
				return
					OtherFixtureTypeoActorClassPair.FixtureType == FixtureTypeToActorClassPair.FixtureType &&
					OtherFixtureTypeoActorClassPair.ActorClass != FixtureTypeToActorClassPair.ActorClass;
			});

		if (IndexOfChangedElement != INDEX_NONE)
		{
			break;
		}
	}

	if (IndexOfChangedElement == INDEX_NONE)
	{
		return;
	}

	const TSubclassOf<AActor> Class = FixtureTypeToActorClasses[IndexOfChangedElement].ActorClass.Get();
	if (!Class.Get())
	{
		return;
	}

	for (const TSoftObjectPtr<AActor>& RelatedActor : TArray<TSoftObjectPtr<AActor>>(RelatedActors))
	{
		AActor* Actor = RelatedActor.Get();
		if (!Actor)
		{
			continue;
		}

		UDMXEntityFixturePatch* FixturePatch = GetFixturePatchFromActor(Actor);
		if (FixturePatch && 
			FixturePatch->GetFixtureType() && 
			FixturePatch->GetFixtureType() == FixtureTypeToActorClasses[IndexOfChangedElement].FixtureType)
		{
			ReplaceMVRActor(Actor, Class);
		}
	}
}
#endif // WITH_EDITOR

#if WITH_EDITOR
AActor* ADMXMVRSceneActor::SpawnMVRActor(const TSubclassOf<AActor>&ActorClass, UDMXEntityFixturePatch* FixturePatch, const FTransform& Transform, AActor* Template)
{
	UWorld* World = GetWorld();
	if (!ensureAlwaysMsgf(World, TEXT("Trying to spawn MVR Fixture in MVR Scene, but the world is not valid.")))
	{
		return nullptr;
	}

	if (!ensureAlwaysMsgf(FixturePatch, TEXT("Trying to spawn MVR Fixture in MVR Scene, but the Fixture Patch is not valid.")))
	{
		return nullptr;
	}

	if (!FixturePatch->GetFixtureType())
	{
		UE_LOGF(LogDMXRuntime, Warning, "Cannot spawn fixture for Fixture Patch '%ls'. Fixture Patch has no Fixture Type set.", *FixturePatch->Name);
		return nullptr;
	}

	FName ActorName = *FixturePatch->Name;
	if (!IsGloballyUniqueObjectName(*FixturePatch->Name))
	{
		ActorName = MakeUniqueObjectName(World, AActor::StaticClass(), *FixturePatch->Name);
	}

	FActorSpawnParameters ActorSpawnParameters;
	ActorSpawnParameters.Template = Template;
	ActorSpawnParameters.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	ActorSpawnParameters.Name = ActorName;
	ActorSpawnParameters.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;
	AActor* NewFixtureActor = World->SpawnActor<AActor>(ActorClass, Transform, ActorSpawnParameters);
	if (!NewFixtureActor)
	{
		return nullptr;
	}
	NewFixtureActor->SetActorLabel(FixturePatch->Name);

	NewFixtureActor->RegisterAllComponents();
	USceneComponent* RootComponentOfChildActor = NewFixtureActor->GetRootComponent();
	if (!RootComponentOfChildActor)
	{
		UE_LOGF(LogDMXRuntime, Warning, "Cannot spawn MVR Fixture Actor of Class %ls, the Actor does not specifiy a root component.", *ActorClass->GetName());
		NewFixtureActor->Destroy();
		return nullptr;
	}

#if WITH_EDITOR
	// Create Property Change Events so editor objects related to the actor have a chance to update (e.g. Details, World Outliner).
	PreEditChange(ADMXMVRSceneActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ADMXMVRSceneActor, RelatedActors)));
	NewFixtureActor->PreEditChange(nullptr);
#endif 

	// Attach, set MVR Fixture UUID, set Fixture Patch, remember as a Related Actor
	RootComponentOfChildActor->SetMobility(EComponentMobility::Movable);
	RootComponentOfChildActor->AttachToComponent(MVRSceneRoot, FAttachmentTransformRules::KeepWorldTransform);
	const FGuid& MVRFixtureUUID = FixturePatch->GetMVRFixtureUUID();
	UDMXMVRAssetUserData::SetMVRAssetUserDataValueForKey(*NewFixtureActor, UDMXMVRAssetUserData::MVRFixtureUUIDMetaDataKey, MVRFixtureUUID.ToString());
	SetFixturePatchOnActor(NewFixtureActor, FixturePatch);
	RelatedActors.Add(NewFixtureActor);

	DeletedMVRFixtureUUIDs.Remove(FixturePatch->GetMVRFixtureUUID());

#if WITH_EDITOR
	PostEditChange();
	NewFixtureActor->PostEditChange();
#endif

	return NewFixtureActor;
}
#endif // WITH_EDITOR

#if WITH_EDITOR
AActor* ADMXMVRSceneActor::ReplaceMVRActor(AActor* ActorToReplace, const TSubclassOf<AActor>& ClassOfNewActor)
{
	if (!ensureAlwaysMsgf(ActorToReplace, TEXT("Trying to replace MVR Fixture in MVR Scene, but the Actor to replace is not valid.")))
	{
		return nullptr;
	}
	
	if (ActorToReplace->GetClass() == ClassOfNewActor)
	{
		// No need to replace
		return nullptr;
	}

	const FString MVRFixtureUUIDString = UDMXMVRAssetUserData::GetMVRAssetUserDataValueForKey(*ActorToReplace, UDMXMVRAssetUserData::MVRFixtureUUIDMetaDataKey);
	FGuid MVRFixtureUUID;
	if (FGuid::Parse(MVRFixtureUUIDString, MVRFixtureUUID))
	{
		// Try to find a Fixture Patch in following order:
		// By the MVR Fixture Actor Interface, it may customize the getter
		// By a DMX Component present in the Actor, it might have overriden the patch
		// By MVR Fixture UUID in the DMX Library
		UDMXEntityFixturePatch* FixturePatch = nullptr;
		if (IDMXMVRFixtureActorInterface* MVRFixtureActorInterface = Cast<IDMXMVRFixtureActorInterface>(ActorToReplace))
		{
			FixturePatch = GetFixturePatchFromActor(ActorToReplace);
		}
		
		if (!FixturePatch)
		{
			if (UActorComponent* Component = ActorToReplace->GetComponentByClass(UDMXComponent::StaticClass()))
			{
				FixturePatch = CastChecked<UDMXComponent>(Component)->GetFixturePatch();
			}
		}

		if (!FixturePatch && DMXLibrary)
		{
			const TArray<UDMXEntityFixturePatch*> FixturePatches = DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>();
			UDMXEntityFixturePatch* const* FixturePatchPtr = FixturePatches.FindByPredicate([&MVRFixtureUUID](const UDMXEntityFixturePatch* FixturePatch)
				{
					return FixturePatch->GetMVRFixtureUUID() == MVRFixtureUUID;
				});

			if (FixturePatchPtr)
			{
				FixturePatch = *FixturePatchPtr;
			}
		}

		if (AActor* NewFixtureActor = SpawnMVRActor(ClassOfNewActor, FixturePatch, ActorToReplace->GetTransform()))
		{
			RelatedActors.Remove(ActorToReplace);
			ActorToReplace->Destroy();
			return NewFixtureActor;
		}
	}

	return nullptr;
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void ADMXMVRSceneActor::UpgradeToFixtureTypeToActorClasses()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!DMXLibrary)
	{
		return;
	}

	const TArray<UDMXEntityFixtureType*> FixtureTypes = DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixtureType>();
	
	TArray<UDMXEntityFixtureType*> PreviouslyIgnoredFixtureTypes;
	for (UDMXEntityFixtureType* FixtureType : FixtureTypes)
	{
		if (!FixtureType)
		{
			continue;
		}

		const FDMXMVRSceneGDTFToActorClassPair* GDTFToActorClassPairPtr  = Algo::FindByPredicate(GDTFToDefaultActorClasses_DEPRECATED,
			[&FixtureType](const FDMXMVRSceneGDTFToActorClassPair& GDTFToActorClassPair)
			{
				return
					GDTFToActorClassPair.ActorClass &&
					!GDTFToActorClassPair.GDTF.IsNull() &&
					GDTFToActorClassPair.GDTF == FixtureType->GDTFSource;
			});

		if (GDTFToActorClassPairPtr)
		{
			FDMXMVRSceneFixtureTypeToActorClassPair FixtureTypeToActorClassPair;
			FixtureTypeToActorClassPair.FixtureType = FixtureType;
			FixtureTypeToActorClassPair.ActorClass = GDTFToActorClassPairPtr->ActorClass;

			FixtureTypeToActorClasses.Add(FixtureTypeToActorClassPair);
		}
		else
		{
			PreviouslyIgnoredFixtureTypes.Add(FixtureType);
		}
	}

	// Early out if possible to avoid any overhead
	if (PreviouslyIgnoredFixtureTypes.IsEmpty())
	{
		return;
	}

	const TSharedRef<FDMXMVRFixtureActorLibrary> MVRFixtureActorLibrary = MakeShared<FDMXMVRFixtureActorLibrary>();
	const TArray<UDMXEntityFixturePatch*> FixturePatches = DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>();

	TArray<const UDMXEntityFixtureType*> UpgradedFixtureTypes;
	for (UDMXEntityFixturePatch* FixturePatch : FixturePatches)
	{
		const UDMXEntityFixtureType* FixtureType = FixturePatch ? FixturePatch->GetFixtureType() : nullptr;
		if (!FixturePatch || !FixtureType)
		{
			continue;
		}

		if (PreviouslyIgnoredFixtureTypes.Contains(FixtureType))
		{
			// Treat previously ignored Fixture Types as deleted from the Level
			DeletedMVRFixtureUUIDs.Add(FixturePatch->GetMVRFixtureUUID());

			if (!UpgradedFixtureTypes.Contains(FixtureType))
			{
				// Upgrade to use Fixture Types which don't have a GDTF set
				FDMXMVRSceneFixtureTypeToActorClassPair FixtureTypeToActorClassPair;
				FixtureTypeToActorClassPair.FixtureType = FixtureType;
				FixtureTypeToActorClassPair.ActorClass = MVRFixtureActorLibrary->FindMostAppropriateActorClassForPatch(FixturePatch);

				FixtureTypeToActorClasses.Add(FixtureTypeToActorClassPair);

				UpgradedFixtureTypes.Add(FixtureType);
			}
		}
	}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
#endif // WITH_EDITOR
