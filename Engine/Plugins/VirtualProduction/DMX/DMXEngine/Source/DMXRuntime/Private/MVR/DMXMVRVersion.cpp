// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVR/DMXMVRVersion.h"


const FString FDMXMVRVersion::GetMajorVersionAsString() 
{ 
	return FString::FromInt(FDMXMVRVersion::MajorVersion); 
}

const FString FDMXMVRVersion::GetMinorVersionAsString() 
{ 
	return FString::FromInt(FDMXMVRVersion::MinorVersion); 
}
