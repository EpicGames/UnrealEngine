// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVR/Types/DMXMVRChildListNode.h"

#include "MVR/Types/DMXMVRFixtureNode.h"
#include "MVR/Types/DMXMVRGroupObjectNode.h"
#include "MVR/Types/DMXMVRParametricObjectNodeBase.h"

#include "XmlNode.h"
#include "Algo/Find.h"


void UDMXMVRChildListNode::InitializeFromChildListXmlNode(const FXmlNode& ChildListXmlNode)
{
	checkf(ChildListXmlNode.GetTag() == TEXT("ChildList"), TEXT("Trying to initialize a Child List Node from Xml Node, but the Node's Tag is '%s' instead of 'ChildList'."), *ChildListXmlNode.GetTag());

	ParametricObjectNodes.Reset();
	for (const FXmlNode* XmlNode : ChildListXmlNode.GetChildrenNodes())
	{
		constexpr TCHAR GroupObjectTag[] = TEXT("GroupObject");
		constexpr TCHAR FixtureTag[] = TEXT("Fixture");

		if (XmlNode->GetTag() == GroupObjectTag)
		{
			UDMXMVRGroupObjectNode* NewGroupObjectNode = CreateParametricObject<UDMXMVRGroupObjectNode>();
			NewGroupObjectNode->InitializeFromGroupObjectXmlNode(*XmlNode);
		}
		else if (XmlNode->GetTag() == FixtureTag)
		{
			UDMXMVRFixtureNode* NewFixtureNode = CreateParametricObject<UDMXMVRFixtureNode>();
			NewFixtureNode->InitializeFromFixtureXmlNode(*XmlNode);
		}
	}
}

void UDMXMVRChildListNode::CreateXmlNodeInParent(FXmlNode& ParentNode) const
{
	checkf(ParentNode.GetTag() == TEXT("layer") || ParentNode.GetTag() == TEXT("groupobject"), TEXT("Child List Nodes have to be created in a Layer Or Group Object node, but parent node is %s."), *ParentNode.GetTag());

	// Don't export empty child lists
	if (ParametricObjectNodes.IsEmpty())
	{
		return;
	}

	constexpr TCHAR Tag[] = TEXT("ChildList");
	ParentNode.AppendChildNode(Tag);
	FXmlNode* ChildListXmlNode = ParentNode.GetChildrenNodes().Last();
	check(ChildListXmlNode);

	for (const UDMXMVRParametricObjectNodeBase* ParametricObject : ParametricObjectNodes)
	{
		ParametricObject->CreateXmlNodeInParent(*ChildListXmlNode);
	}
}

void UDMXMVRChildListNode::GetFixtureNodes(TArray<UDMXMVRFixtureNode*>& OutFixtureNodes) const
{
	for (UDMXMVRParametricObjectNodeBase* Node : ParametricObjectNodes)
	{
		if (UDMXMVRFixtureNode* FixtureNode = Cast<UDMXMVRFixtureNode>(Node))
		{
			OutFixtureNodes.Add(FixtureNode);
		}
		else if (UDMXMVRGroupObjectNode* GroupObjectNode = Cast<UDMXMVRGroupObjectNode>(Node))
		{
			// For group objects, also get their Fixture Nodes
			GroupObjectNode->GetFixtureNodes(OutFixtureNodes);
		}
	}
}

bool UDMXMVRChildListNode::Contains(UDMXMVRParametricObjectNodeBase* ParametricObjectNode)
{
	for (UDMXMVRParametricObjectNodeBase* Other : ParametricObjectNodes)
	{
		if (UDMXMVRGroupObjectNode* GroupObjectNode = Cast<UDMXMVRGroupObjectNode>(Other))
		{		
			// For group objects, also test if they contain the node
			if (GroupObjectNode->Contains(ParametricObjectNode))
			{
				return true;
			}
		}

		if (Other == ParametricObjectNode)
		{
			return true;
		}
	}

	return false;
}

TObjectPtr<UDMXMVRParametricObjectNodeBase>* UDMXMVRChildListNode::FindParametricObjectNodeByUUID(const FGuid& UUID)
{
	TObjectPtr<UDMXMVRParametricObjectNodeBase>* ParametricObjectNodePtr = Algo::FindByPredicate(ParametricObjectNodes, [UUID](UDMXMVRParametricObjectNodeBase* ParametricObjectNode)
		{
			return ParametricObjectNode->UUID == UUID;
		});
	if (ParametricObjectNodePtr)
	{
		return ParametricObjectNodePtr;
	}

	for (UDMXMVRParametricObjectNodeBase* Node : ParametricObjectNodes)
	{
		if (UDMXMVRGroupObjectNode* GroupObjectNode = Cast<UDMXMVRGroupObjectNode>(Node))
		{
			ParametricObjectNodePtr = GroupObjectNode->FindParametricObjectNodeByUUID(UUID);
			if (ParametricObjectNodePtr)
			{
				return ParametricObjectNodePtr;
			}
		}
	}

	return nullptr;
}

bool UDMXMVRChildListNode::RemoveParametricObject(UDMXMVRParametricObjectNodeBase* ParametricObjectNode)
{
	const int32 NumRemovedElements = ParametricObjectNodes.RemoveSingle(ParametricObjectNode);
	if (NumRemovedElements == 1)
	{
		return true;
	}

	for (UDMXMVRParametricObjectNodeBase* Node : ParametricObjectNodes)
	{
		if (UDMXMVRGroupObjectNode* GroupObjectNode = Cast<UDMXMVRGroupObjectNode>(Node))
		{
			if (GroupObjectNode->RemoveParametricObjectNode(ParametricObjectNode))
			{
				return true;
			}
		}
	}

	return false;
}
