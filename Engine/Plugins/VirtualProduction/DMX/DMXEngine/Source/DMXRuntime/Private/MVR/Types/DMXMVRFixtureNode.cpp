// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVR/Types/DMXMVRFixtureNode.h"

#include "DMXProtocolCommon.h"
#include "DMXRuntimeLog.h"
#include "DMXRuntimeUtils.h"

#include "XmlNode.h"
#include "Misc/Paths.h"


namespace UE::DMXRuntime::DMXMVRFixture::Private
{
	/** Parses the Name from the MVR Fixture Node, returns true on success. */
	bool ParseName(const FXmlNode& InFixtureNode, FString& OutName)
	{
		constexpr TCHAR NameTag[] = TEXT("Name");
		OutName = InFixtureNode.GetAttribute(NameTag);
		if (!OutName.IsEmpty())
		{
			return true;
		}

		return false;
	}

	/** Parses the UUID from the MVR Fixture Node, returns true on success. */
	bool ParseUUID(const FXmlNode& InFixtureNode, FGuid& OutUUID)
	{
		constexpr TCHAR UUIDTag[] = TEXT("UUID");
		const FString UUIDString = InFixtureNode.GetAttribute(UUIDTag);
		if (FGuid::Parse(UUIDString, OutUUID))
		{
			return true;
		}
		return false;
	}

	/** Parses the Matrix from the MVR Fixture Node, optional, always succeeds. */
	void ParseMatrix(const FXmlNode& InFixtureNode, FDMXOptionalTransform& OutTransform)
	{
		OutTransform.Reset();

		constexpr TCHAR MatrixTag[] = TEXT("Matrix");
		const FXmlNode* MatrixNode = InFixtureNode.FindChildNode(MatrixTag);

		if (!MatrixNode)
		{
			return;
		}

		OutTransform = FDMXRuntimeUtils::ParseGDTFMatrix(MatrixNode->GetContent());
	}
	
	/** Parses the GDTFSpec from the MVR Fixture Node, returns true on success. */
	bool ParseGDTFSpec(const FXmlNode& InFixtureNode, FString& OutGDTFSpec)
	{
		constexpr TCHAR GDTFSpecTag[] = TEXT("GDTFSpec");
		const FXmlNode* GDTFSpecNode = InFixtureNode.FindChildNode(GDTFSpecTag);
		if (GDTFSpecNode)
		{
			OutGDTFSpec = GDTFSpecNode->GetContent();

			// Append the gdtf extension if it's missing.
			if (FPaths::GetExtension(OutGDTFSpec) != TEXT("gdtf"))
			{
				OutGDTFSpec += TEXT(".gdtf");
			}

			return true;
		}

		return false;
	}

	/** Parses the GDTFMode from the MVR Fixture Node, returns true on success. */
	bool ParseGDTFMode(const FXmlNode& InFixtureNode, FString& OutGDTFMode)
	{
		constexpr TCHAR GDTFModeTag[] = TEXT("GDTFMode");
		const FXmlNode* GDTFModeNode = InFixtureNode.FindChildNode(GDTFModeTag);
		if (GDTFModeNode)
		{
			OutGDTFMode = GDTFModeNode->GetContent();
			return true;
		}
		return true;
	}

	/** Parses Focus from the MVR Fixture Node, optional, always succeeds. */
	void ParseFocus(const FXmlNode& InFixtureNode, FDMXOptionalGuid& OutFocus)
	{
		OutFocus.Reset();

		constexpr TCHAR FocusTag[] = TEXT("Focus");
		const FXmlNode* FocusNode = InFixtureNode.FindChildNode(FocusTag);
		if (FocusNode)
		{
			const FString FocusString = InFixtureNode.GetContent();
			FGuid FocusGuid;
			if (FGuid::Parse(FocusString, FocusGuid))
			{
				OutFocus = FocusGuid;
			}
		};
	}

	/** Parses Cast Shadows from the MVR Fixture Node, optional, always succeeds. */
	void ParseCastShadow(const FXmlNode& InFixtureNode, FDMXOptionalBool& OutCastShadow)
	{
		OutCastShadow.Reset();

		constexpr TCHAR CastShadowTag[] = TEXT("CastShadow");
		const FXmlNode* CastShadowNode = InFixtureNode.FindChildNode(CastShadowTag);
		if (CastShadowNode)
		{
			bool bCastShadow = 
				CastShadowNode->GetContent() != TEXT("0") ||
				CastShadowNode->GetContent().Equals(TEXT("true"), ESearchCase::IgnoreCase);

			OutCastShadow = bCastShadow;
		}
	}

	/** Parses Position of from MVR Fixture Node, optional, always succeeds. */
	void ParsePosition(const FXmlNode& InFixtureNode, FDMXOptionalGuid& OutPosition)
	{
		OutPosition.Reset();

		constexpr TCHAR PositionTag[] = TEXT("Position");
		const FXmlNode* PositionNode = InFixtureNode.FindChildNode(PositionTag);
		if (PositionNode)
		{
			FGuid PositionGuid;
			if (FGuid::Parse(PositionNode->GetContent(), PositionGuid))
			{
				OutPosition = PositionGuid;
			}
		}
	}

	/** Parses the Fixture Id from the MVR Fixture Node, returns true on success. */
	bool ParseFixtureID(const FXmlNode& InFixtureNode, FString& OutFixtureId)
	{
		OutFixtureId.Reset();

		constexpr TCHAR FixtureIdTag[] = TEXT("FixtureID");
		const FXmlNode* FixtureIdNode = InFixtureNode.FindChildNode(FixtureIdTag);
		if (FixtureIdNode)
		{
			OutFixtureId = FixtureIdNode->GetContent();
			return true;
		}

		return false;
	}

	/** Parses the Unit Number from the MVR Fixture Node, returns true on success. */
	bool ParseUnitNumber(const FXmlNode& InFixtureNode, int32& OutUnitNumber)
	{
		// @todo: Temp fix for the larger fixture id/unit number mismatch
		constexpr TCHAR UnitNumberTag[] = TEXT("UnitNumber");
		const FXmlNode* UnitNumberNode = InFixtureNode.FindChildNode(UnitNumberTag);
		if (UnitNumberNode)
		{
			int32 UnitNumber;
			if (LexTryParseString(UnitNumber, *UnitNumberNode->GetContent()))
			{
				OutUnitNumber = UnitNumber;
				return true;
			}
		}
		return false;
	}

	/** Parses the Addresses from the MVR Fixture Node, defaulted, always succeeds */
	bool ParseAddresses(const FXmlNode& InFixtureNode, FDMXMVRFixtureAddresses& OutAddresses)
	{
		constexpr TCHAR AddressesTag[] = TEXT("Addresses");
		const FXmlNode* AddressesNode = InFixtureNode.FindChildNode(AddressesTag);

		constexpr TCHAR AddressTag[] = TEXT("Address");
		const FXmlNode* AddressNode = AddressesNode ? AddressesNode->FindChildNode(AddressTag) : nullptr;

		if (AddressNode)
		{
			int32 Universe = -1;
			int32 Address = -1;

			auto ParseAddressStringLambda = [](const FXmlNode& InAddressNode, int32& OutUniverse, int32& OutAddress) -> const bool
			{
				const FString InAddressString = InAddressNode.GetContent();

				int32 SeparatorIndex = INDEX_NONE;
				if (InAddressString.FindChar('.', SeparatorIndex))
				{
					if (SeparatorIndex > 0 && SeparatorIndex < InAddressString.Len() - 2)
					{
						const FString UniverseString = InAddressString.Left(SeparatorIndex);
						const FString AddressString = InAddressString.RightChop(SeparatorIndex + 1);

						const bool bValidUniverse = LexTryParseString(OutUniverse, *UniverseString) && OutUniverse >= 0 && OutUniverse <= DMX_MAX_UNIVERSE;
						const bool bValidAddress = LexTryParseString(OutAddress, *UniverseString) && OutAddress >= 0 && OutAddress <= DMX_MAX_ADDRESS;

						return bValidUniverse && bValidAddress;
					}
				}
				else if (LexTryParseString(OutAddress, *InAddressString))
				{
					static const int32 UniverseSize = 512;

					OutUniverse = OutAddress / UniverseSize;
					OutAddress = OutAddress % UniverseSize;

					return 
						OutUniverse >= 0 && 
						OutUniverse <= DMX_MAX_UNIVERSE &&
						OutAddress >= 0 && 
						OutAddress <= DMX_MAX_ADDRESS;
				}

				return false;
			}(*AddressNode, Universe, Address);
			if (ParseAddressStringLambda)
			{
				OutAddresses.Universe = Universe;
				OutAddresses.Address = Address;

				return true;
			}
		}
		return false;
	}

	/** Parses the CIE Color from the MVR Fixture Node, optional, always succeeds. */
	void ParseCIEColor(const FXmlNode& InFixtureNode, FDMXOptionalColorCIE1931& OutColorCIE1931)
	{
		OutColorCIE1931.Reset();

		constexpr TCHAR CIEColorTag[] = TEXT("CIEColor");
		const FXmlNode* CIEColorNode = InFixtureNode.FindChildNode(CIEColorTag);
		if (CIEColorNode)
		{
			const FString CIEColorString = CIEColorNode->GetContent();
			if (CIEColorString.IsEmpty())
			{
				return;
			}

			TArray<FString> CIEColorArray;
			CIEColorString.ParseIntoArray(CIEColorArray, TEXT(","));

			if (CIEColorArray.Num() != 3	)
			{
				return;
			}

			FDMXGDTFColorCIE1931xyY Result;
			bool bSuccess = LexTryParseString<float>(Result.X, *CIEColorArray[0]);
			bSuccess = bSuccess && LexTryParseString<float>(Result.Y, *CIEColorArray[1]);
			bSuccess = bSuccess && LexTryParseString<float>(Result.YY, *CIEColorArray[2]);

			if (bSuccess)
			{
				OutColorCIE1931 = Result;
			}
		}
	}

	/** Parses the Fixture Type Id from MVR Fixture Node, optional, always succeeds. */
	void ParseFixtureTypeId(const FXmlNode& InFixtureNode, FDMXOptionalInt32& OutFixtureTypeId)
	{
		OutFixtureTypeId.Reset();

		constexpr TCHAR FixtureTypeIdTag[] = TEXT("FixtureTypeId");
		const FXmlNode* FixtureTypeIdNode = InFixtureNode.FindChildNode(FixtureTypeIdTag);
		if (FixtureTypeIdNode)
		{
			int32 FixtureTypeId;
			if (LexTryParseString(FixtureTypeId, *FixtureTypeIdNode->GetContent()))
			{
				OutFixtureTypeId = FixtureTypeId;
			}
		}
	}

	/** Parses the Custom Id from MVR Fixture Node, optional, always succeeds. */
	void ParseCustomId(const FXmlNode& InFixtureNode, FDMXOptionalInt32& OutCustomId)
	{
		OutCustomId.Reset();

		constexpr TCHAR PositionTag[] = TEXT("Position");
		const FXmlNode* PositionNode = InFixtureNode.FindChildNode(PositionTag);
		if (PositionNode)
		{
			int32 FixtureTypeId;
			if (LexTryParseString(FixtureTypeId, *PositionNode->GetContent()))
			{
				OutCustomId = FixtureTypeId;
			}
		}
	}

	/** Parses the Mapping from MVR Fixture Node, optional, always succeeds. */
	void ParseMappings(const FXmlNode& InFixtureNode, TArray<FDMXMVRFixtureMapping>& OutMappings)
	{
		OutMappings.Reset();

		constexpr TCHAR MappingsTag[] = TEXT("Mappings");
		const FXmlNode* MappingsNode = InFixtureNode.FindChildNode(MappingsTag);
		if (MappingsNode)
		{
			for (const FXmlNode* MappingNode : MappingsNode->GetChildrenNodes())
			{
				FDMXMVRFixtureMapping Mapping;

				constexpr TCHAR ChildNodenName_LinkDef[] = TEXT("LinkDef");
				const FXmlNode* LinkDefNode = MappingNode->FindChildNode(ChildNodenName_LinkDef);
				if (LinkDefNode && !FGuid::Parse(LinkDefNode->GetContent(), Mapping.LinkedDef))
				{
					continue;
				}

				constexpr TCHAR ChildNodenName_UX[] = TEXT("UX");
				const FXmlNode* UXNode = InFixtureNode.FindChildNode(ChildNodenName_UX);
				int32 UX;
				if (UXNode && LexTryParseString(UX, *UXNode->GetContent()))
				{
					Mapping.UX = UX;
				}

				constexpr TCHAR ChildNodenName_UY[] = TEXT("UY");
				const FXmlNode* UYNode = InFixtureNode.FindChildNode(ChildNodenName_UY);
				int32 UY;
				if (UYNode && LexTryParseString(UY, *UYNode->GetContent()))
				{
					Mapping.UY = UY;
				}

				constexpr TCHAR ChildNodenName_OX[] = TEXT("OX");
				const FXmlNode* OXNode = InFixtureNode.FindChildNode(ChildNodenName_OX);
				int32 OX;
				if (OXNode && LexTryParseString(OX, *OXNode->GetContent()))
				{
					Mapping.OX = OX;
				}

				constexpr TCHAR ChildNodenName_OY[] = TEXT("OY");
				const FXmlNode* OYNode = InFixtureNode.FindChildNode(ChildNodenName_OY);
				int32 OY;
				if (OYNode && LexTryParseString(OY, *OYNode->GetContent()))
				{
					Mapping.OY = OY;
				}


				constexpr TCHAR ChildNodenName_RZ[] = TEXT("RZ");
				const FXmlNode* RZNode = InFixtureNode.FindChildNode(ChildNodenName_RZ);
				int32 RZ;
				if (RZNode && !LexTryParseString(RZ, *RZNode->GetContent()))
				{
					Mapping.RZ = RZ;
				}

				OutMappings.Add(Mapping);
			}
		}
	}

	/** Parses the Gobo from MVR Fixture Node, optional, always succeeds. */
	void ParseGobo(const FXmlNode& InFixtureNode, FDMXOptionalMVRFixtureGobo& OutGobo)
	{
		OutGobo.Value.Reset();

		constexpr TCHAR GoboTag[] = TEXT("Gobo");
		const FXmlNode* GoboNode = InFixtureNode.FindChildNode(GoboTag);
		if (GoboNode)
		{
			FDMXMVRFixtureGobo FixtureGobo;
			const FString& GoboResource = GoboNode->GetContent();
			FixtureGobo.Value = GoboResource;

			constexpr TCHAR RotationTag[] = TEXT("Rotation");
			const FXmlNode* GoboRotationNode = GoboNode->FindChildNode(RotationTag);
			float GoboRotation;
			if (GoboRotationNode && LexTryParseString(GoboRotation, *GoboRotationNode->GetContent()))
			{
				FixtureGobo.Rotation = GoboRotation;
			}

			OutGobo.Value = FixtureGobo;
		}
	}
}

void FDMXMVRFixtureAddresses::CreateXmlNodeInParent(FXmlNode& ParentNode) const
{
	const int32 AbsoluteChannel = Universe * DMX_UNIVERSE_SIZE + Address;
	constexpr TCHAR Tag[] = TEXT("Address");
	const FString Content = FString::FromInt(AbsoluteChannel);

	constexpr TCHAR BreakAttributeName[] = TEXT("break");
	constexpr TCHAR BreakAttributeValue[] = TEXT("0");
	const TArray<FXmlAttribute> Attributes =
	{
		FXmlAttribute(BreakAttributeName, BreakAttributeValue)
	};

	ParentNode.AppendChildNode(Tag, Content, Attributes);
}

void FDMXOptionalMVRFixtureGobo::CreateXmlNodeInParent(FXmlNode& ParentNode) const
{
	const TOptional<float>& OptionalRotation = Value.GetValue().Rotation.Value;
	const FString& Content = Value.GetValue().Value; // The resource

	if (!Content.IsEmpty() && OptionalRotation.IsSet())
	{
		// Add self
		constexpr TCHAR Tag[] = TEXT("Gobo");

		TArray<FXmlAttribute> Attributes;
		if (OptionalRotation.IsSet())
		{
			constexpr TCHAR RotationAttributeName[] = TEXT("rotation");
			constexpr int32 MinFractionalDigitsOfValue = 6; // As other soft- and hardwares
			const FString RotationAttributeValue = FString::SanitizeFloat(OptionalRotation.GetValue(), MinFractionalDigitsOfValue);

			Attributes.Add(FXmlAttribute(RotationAttributeName, RotationAttributeValue));
		}
		ParentNode.AppendChildNode(Tag, Content, Attributes);
	}
}

void FDMXMVRFixtureMapping::CreateXmlNodeInParent(FXmlNode& ParentNode) const
{
	constexpr TCHAR Tag[] = TEXT("Mapping");
	constexpr TCHAR LinkDefAttributeName[] = TEXT("linkedDef");
	const TArray<FXmlAttribute> Attributes
	{
		FXmlAttribute(LinkDefAttributeName, LinkedDef.ToString(EGuidFormats::DigitsWithHyphens))
	};
	ParentNode.AppendChildNode(Tag);
	FXmlNode* MappingXmlNode = ParentNode.GetChildrenNodes().Last();
	check(MappingXmlNode);

	if (UX.IsSet())
	{
		constexpr TCHAR UxTag[] = TEXT("ux");
		const FString ChildContent = FString::FromInt(UX.GetValue());
		MappingXmlNode->AppendChildNode(UxTag, ChildContent);
	}

	if (UY.IsSet())
	{
		constexpr TCHAR UyTag[] = TEXT("uy");
		const FString ChildContent = FString::FromInt(UY.GetValue());
		MappingXmlNode->AppendChildNode(UyTag, ChildContent);
	}

	if (OX.IsSet())
	{
		constexpr TCHAR OxTag[] = TEXT("ox");
		const FString ChildContent = FString::FromInt(OX.GetValue());
		MappingXmlNode->AppendChildNode(OxTag, ChildContent);
	}

	if (OY.IsSet())
	{
		constexpr TCHAR OyTag[] = TEXT("oy");
		const FString ChildContent = FString::FromInt(OY.GetValue());
		MappingXmlNode->AppendChildNode(OyTag, ChildContent);
	}

	if (RZ.IsSet())
	{
		constexpr TCHAR RzTag[] = TEXT("rz");
		const FString ChildContent = FString::FromInt(RZ.GetValue());
		MappingXmlNode->AppendChildNode(RzTag, ChildContent);
	}
}

bool UDMXMVRFixtureNode::InitializeFromFixtureXmlNode(const FXmlNode& FixtureXmlNode)
{
	checkf(FixtureXmlNode.GetTag() == TEXT("fixture"), TEXT("Trying to initialize a Fixture Node from Xml Node, but the Node's Tag is '%s' instead of 'Fixture'."), *FixtureXmlNode.GetTag());

	constexpr TCHAR NodeName_Fixture[] = TEXT("Fixture");
	if (!ensureAlwaysMsgf(FixtureXmlNode.GetTag() == NodeName_Fixture, TEXT("Trying to parse MVR Fixture Node, but node name doesn't match.")))
	{
		return false;
	}

	// Parse nodes. Those that don't fail are either defaulted or optional.
	using namespace UE::DMXRuntime::DMXMVRFixture::Private;
	if (!ParseName(FixtureXmlNode, Name))
	{
		// Non-optional
		UE_LOGF(LogDMXRuntime, Warning, "Cannot parse Fixture Node as MVR Fixture Name. Failed to parse MVR Node %ls.", *FixtureXmlNode.GetTag());
		return false;
	}

	if(!ParseUUID(FixtureXmlNode, UUID))
	{
		// Non-optional
		UE_LOGF(LogDMXRuntime, Warning, "Cannot parse Fixture '%ls' from MVR. Failed to parse MVR UUID.", *Name);
		return false;
	}

	ParseMatrix(FixtureXmlNode, Matrix);
	if (Matrix.IsSet() && !Matrix.GetValue().IsValid())
	{
		// Mend invalid transforms
		UE_LOGF(LogDMXRuntime, Warning, "Imported Fixture '%ls' from MVR but its Transformation Matrix is not valid, using Identity instead.", *Name);
		Matrix = FTransform::Identity;
	}

	if (!ParseGDTFSpec(FixtureXmlNode, GDTFSpec))
	{
		// Non-optional
		UE_LOGF(LogDMXRuntime, Warning, "Cannot parse Fixture '%ls' from MVR. No valid GDTF Spec specified.", *Name);
		return false;
	}

	if (!ParseGDTFMode(FixtureXmlNode, GDTFMode))
	{
		// Non-optional
		UE_LOGF(LogDMXRuntime, Warning, "Cannot parse Fixture '%ls' from MVR. No valid GDTF Mode specified.", *Name);
		return false;
	}

	ParseFocus(FixtureXmlNode, Focus);
	ParseCastShadow(FixtureXmlNode, CastShadow);
	ParsePosition(FixtureXmlNode, Position);
	
	if (!ParseFixtureID(FixtureXmlNode, FixtureID))
	{
		// Non-optional, mend with empty string
		UE_LOGF(LogDMXRuntime, Warning, "Imported Fixture '%ls' from MVR. No valid Fixture Id specified. Fixture Id set to Fixture Name.", *Name);
		FixtureID = TEXT("");
	}

	if (!ParseUnitNumber(FixtureXmlNode, UnitNumber))
	{
		// Non-optional, but defaulted by standard
		UE_LOGF(LogDMXRuntime, Warning, "Imported Fixture '%ls' from MVR. No valid Unit Number specified. Unit Number set to ''0'.", *Name);
		UnitNumber = 0;
	}

	if (!ParseAddresses(FixtureXmlNode, Addresses))
	{
		// Non-optional
		UE_LOGF(LogDMXRuntime, Warning, "Cannot parse Fixture '%ls' from MVR. No valid Addresses specified.", *Name);
		return false;
	}

	ParseCIEColor(FixtureXmlNode, CIEColor);
	ParseFixtureTypeId(FixtureXmlNode, FixtureTypeId);
	ParseCustomId(FixtureXmlNode, CustomId);
	ParseMappings(FixtureXmlNode, Mappings);
	ParseGobo(FixtureXmlNode, Gobo);

	return true;
}

void UDMXMVRFixtureNode::CreateXmlNodeInParent(FXmlNode& ParentNode) const
{
	checkf(ParentNode.GetTag() == TEXT("childlist"), TEXT("Fixture Nodes have to be created in a Child List, but parent node is %s."), *ParentNode.GetTag());

	constexpr bool bLogInvalidReason = true;
	if (!IsValid(bLogInvalidReason))
	{
		return;
	}

	constexpr TCHAR FixtureTag[] = TEXT("Fixture");
	ParentNode.AppendChildNode(FixtureTag);
	FXmlNode* FixtureXmlNode = ParentNode.GetChildrenNodes().Last();
	check(FixtureXmlNode);

	TArray<FXmlAttribute> Attributes;

	// Name attribute
	constexpr TCHAR NameAttributeName[] = TEXT("name");
	FXmlAttribute NameAttribute(NameAttributeName, Name);
	Attributes.Add(NameAttribute);

	// UUID attribute
	if (!ensureAlwaysMsgf(UUID.IsValid(), TEXT("Cannot export MVR Fixture '%s'. Invalid UUID specified."), *Name))
	{
		return;
	}

	constexpr TCHAR UUIDAttributeName[] = TEXT("uuid");
	FXmlAttribute UUIDAttribute(UUIDAttributeName, UUID.ToString(EGuidFormats::DigitsWithHyphens));
	Attributes.Add(UUIDAttribute);

	// MultiPatch attribute
	if (MultiPatch.IsSet())
	{
		constexpr TCHAR MultiPatchAttributeName[] = TEXT("multipatch");
		FXmlAttribute MultiPatchAttribute(MultiPatchAttributeName, MultiPatch.GetValue().ToString(EGuidFormats::DigitsWithHyphens));
		Attributes.Add(MultiPatchAttribute);
	}

	FixtureXmlNode->SetAttributes(Attributes);


	////////////////////
	// Child Nodes

	// Matrix 
	if (Matrix.IsSet())
	{
		constexpr TCHAR Tag[] = TEXT("Matrix");
		const FString Content = FDMXRuntimeUtils::ConvertTransformToGDTF4x3MatrixString(Matrix.GetValue());
		FixtureXmlNode->AppendChildNode(Tag, Content);
	}

	// GDTFSpec
	{
		constexpr TCHAR Tag[] = TEXT("GDTFSpec");
		FixtureXmlNode->AppendChildNode(Tag, GDTFSpec);
	}

	// GDTFMode
	{
		constexpr TCHAR Tag[] = TEXT("GDTFMode");
		FixtureXmlNode->AppendChildNode(Tag, GDTFMode);
	}

	// Focus
	if (Focus.IsSet())
	{
		constexpr TCHAR Tag[] = TEXT("Focus");
		const FString Content = Focus.GetValue().ToString(EGuidFormats::DigitsWithHyphens);
		FixtureXmlNode->AppendChildNode(Tag, Content);
	}

	// CastShadow
	if (CastShadow.IsSet())
	{
		constexpr TCHAR Tag[] = TEXT("CastShadow");
		const FString Content = CastShadow.GetValue() ? TEXT("true") : TEXT("false");
		FixtureXmlNode->AppendChildNode(Tag, Content);
	}

	// Position
	if (Position.IsSet())
	{
		constexpr TCHAR Tag[] = TEXT("Position");
		const FString Content = Position.GetValue().ToString(EGuidFormats::DigitsWithHyphens);
		FixtureXmlNode->AppendChildNode(Tag, Content);
	}

	// FixtureID
	{
		constexpr TCHAR Tag[] = TEXT("FixtureID");
		FixtureXmlNode->AppendChildNode(Tag, FixtureID);
	}

	// UnitNumber
	{
		constexpr TCHAR Tag[] = TEXT("UnitNumber");
		const FString Content = FString::FromInt(UnitNumber);
		FixtureXmlNode->AppendChildNode(Tag, Content);
	}

	// Addresses
	{
		constexpr TCHAR Tag[] = TEXT("Addresses");
		FixtureXmlNode->AppendChildNode(Tag);
		FXmlNode* AddressesXmlNode = FixtureXmlNode->GetChildrenNodes().Last();
		check(AddressesXmlNode);

		Addresses.CreateXmlNodeInParent(*AddressesXmlNode);
	}

	// CIEColor
	if (CIEColor.IsSet())
	{
		constexpr TCHAR Tag[] = TEXT("CIEColor");
		const FString Content = CIEColor.GetValue().ToString();
		FixtureXmlNode->AppendChildNode(Tag, Content);
	}

	// FixtureTypeId
	if (FixtureTypeId.IsSet())
	{
		constexpr TCHAR Tag[] = TEXT("FixtureTypeId");
		const FString Content = FString::FromInt(FixtureTypeId.GetValue());
		FixtureXmlNode->AppendChildNode(Tag, Content);
	}

	// CustomId
	if (CustomId.IsSet())
	{
		constexpr TCHAR Tag[] = TEXT("CustomId");
		const FString Content = FString::FromInt(CustomId.GetValue());
		FixtureXmlNode->AppendChildNode(Tag, Content);
	}

	// Mappings
	if (Mappings.Num() > 0)
	{
		constexpr TCHAR Tag[] = TEXT("Mappings");
		FixtureXmlNode->AppendChildNode(Tag);
		FXmlNode* MappingsXmlNode = FixtureXmlNode->GetChildrenNodes().Last();
		check(MappingsXmlNode);

		for (const FDMXMVRFixtureMapping& Mapping : Mappings)
		{
			Mapping.CreateXmlNodeInParent(*MappingsXmlNode);
		}
	}

	// Gobo
	if (Gobo.Value.IsSet())
	{
		constexpr TCHAR Tag[] = TEXT("Gobo");
		FixtureXmlNode->AppendChildNode(Tag);
		FXmlNode* GoboXmlNode = FixtureXmlNode->GetChildrenNodes().Last();
		check(GoboXmlNode);

		Gobo.CreateXmlNodeInParent(*GoboXmlNode);
	}
}

bool UDMXMVRFixtureNode::IsValid(bool bLogInvalidReason) const
{
	const FString BeautifulName = Name.IsEmpty() ? TEXT("unnamed") : Name;

	if (!UUID.IsValid())
	{
		if (bLogInvalidReason)
		{
			UE_LOGF(LogDMXRuntime, Warning, "Fixture '%ls' is not a valid MVR Fixture, its Fixture UUID is invalid.", *BeautifulName);
		}
		return false;
	}
	
	if (GDTFSpec.IsEmpty())
	{
		if (bLogInvalidReason)
		{
			UE_LOGF(LogDMXRuntime, Warning, "Fixture '%ls' is not a valid MVR Fixture, its GDTFSpec is empty.", *BeautifulName);
		}
		return false;
	}

	if (GDTFMode.IsEmpty())
	{
		if (bLogInvalidReason)
		{
			UE_LOGF(LogDMXRuntime, Warning, "Fixture '%ls' is not a valid MVR Fixture, its GDTFMode is empty.", *BeautifulName);
		}
		return false;
	}

	if (Addresses.Universe < 0 || Addresses.Universe > DMX_MAX_UNIVERSE)
	{		
		if (bLogInvalidReason)
		{
			UE_LOGF(LogDMXRuntime, Warning, "Fixture '%ls' is not a valid MVR Fixture, its Universe is < 0 or larger than the Range supported by UE.", *BeautifulName);
		}
		return false;
	}

	if (Addresses.Address < 1 || Addresses.Address > DMX_MAX_ADDRESS)
	{
		if (bLogInvalidReason)
		{
			UE_LOGF(LogDMXRuntime, Warning, "Fixture '%ls' is not a valid MVR Fixture, its Address is < 1 or > 512.", *BeautifulName);
		}
		return false;
	}

	return true;
}
