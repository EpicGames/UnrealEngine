// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVR/Types/DMXMVRGroupObjectNode.h"

#include "DMXRuntimeLog.h"
#include "DMXRuntimeUtils.h"
#include "MVR/Types/DMXMVRChildListNode.h"
#include "MVR/Types/DMXMVRFixtureNode.h"
#include "MVR/Types/DMXMVRParametricObjectNodeBase.h"

#include "XmlNode.h"


UDMXMVRGroupObjectNode::UDMXMVRGroupObjectNode()
{
	ChildListNode = CreateDefaultSubobject<UDMXMVRChildListNode>("ChildListNode");
}

void UDMXMVRGroupObjectNode::InitializeFromGroupObjectXmlNode(const FXmlNode& GroupObjectXmlNode)
{
	checkf(GroupObjectXmlNode.GetTag() == TEXT("groupobject"), TEXT("Trying to initialize a Group Object Node from Xml Node, but the Node's Tag is '%s' instead of 'GroupObject'."), *GroupObjectXmlNode.GetTag());
	checkf(ChildListNode, TEXT("Default Subobject 'ChildListNode' is invalid, this is not expected."));

	// Init self
	constexpr TCHAR AttributeName_UUID[] = TEXT("UUID");
	const FString UUIDString = GroupObjectXmlNode.GetAttribute(AttributeName_UUID);
	if (!FGuid::Parse(UUIDString, UUID))
	{
		UE_LOGF(LogDMXRuntime, Warning, "Found invalid UUID of Group Object when parsing MVR File. Skipping Group Object.")
		return;
	}
	constexpr TCHAR AttributeName_Name[] = TEXT("Name");
	const FString NameString = GroupObjectXmlNode.GetAttribute(AttributeName_Name);
	if (!NameString.IsEmpty())
	{
		Name.Value = NameString;
	}

	static const FString AttributeName_Matrix = TEXT("Matrix");
	const FString MatrixString = GroupObjectXmlNode.GetAttribute(AttributeName_Matrix);
	Matrix = FDMXRuntimeUtils::ParseGDTFMatrix(MatrixString);

	// Init Child List Node
	static const FString NodeName_ChildList = TEXT("ChildList");
	const FXmlNode* ChildListXmlNode = GroupObjectXmlNode.FindChildNode(NodeName_ChildList);

	if (ChildListXmlNode)
	{
		ChildListNode->InitializeFromChildListXmlNode(*ChildListXmlNode);
	}
}

void UDMXMVRGroupObjectNode::CreateXmlNodeInParent(FXmlNode& ParentNode) const
{
	checkf(ParentNode.GetTag() == TEXT("childlist"), TEXT("Group Object Nodes have to be created in a Child List node, but parent node is %s."), *ParentNode.GetTag());
	
	// Don't export empty group objects
	if (!ChildListNode || ChildListNode->GetParametricObjectNodes().IsEmpty())
	{
		return;
	}

	constexpr TCHAR GroupObjectTag[] = TEXT("GroupObject");
	ParentNode.AppendChildNode(GroupObjectTag);
	FXmlNode* GroupObjectXmlNode = ParentNode.GetChildrenNodes().Last();
	check(GroupObjectXmlNode);

	if (ChildListNode)
	{
		ChildListNode->CreateXmlNodeInParent(*GroupObjectXmlNode);
	}
}

void UDMXMVRGroupObjectNode::GetFixtureNodes(TArray<UDMXMVRFixtureNode*>& OutFixtureNodes) const
{
	checkf(ChildListNode, TEXT("Default Subobject 'ChildListNode' is invalid, this is not expected."));

	ChildListNode->GetFixtureNodes(OutFixtureNodes);
}

bool UDMXMVRGroupObjectNode::Contains(UDMXMVRParametricObjectNodeBase* ParametricObjectNode)
{
	checkf(ChildListNode, TEXT("Default Subobject 'ChildListNode' is invalid, this is not expected."));

	for (UDMXMVRParametricObjectNodeBase* Other : ChildListNode->GetParametricObjectNodes())
	{
		if (UDMXMVRGroupObjectNode* GroupObjectNode = Cast<UDMXMVRGroupObjectNode>(Other))
		{
			if (GroupObjectNode->Contains(ParametricObjectNode))
			{
				return true;
			}
		}

		if (Other == ParametricObjectNode)
		{
			return true;
		}
	}

	return false;
}

TObjectPtr<UDMXMVRParametricObjectNodeBase>* UDMXMVRGroupObjectNode::FindParametricObjectNodeByUUID(const FGuid& InUUID)
{
	checkf(ChildListNode, TEXT("Default Subobject 'ChildListNode' is invalid, this is not expected."));

	return ChildListNode->FindParametricObjectNodeByUUID(InUUID);
}

bool UDMXMVRGroupObjectNode::RemoveParametricObjectNode(UDMXMVRParametricObjectNodeBase* ParametricObjectNode)
{
	checkf(ChildListNode, TEXT("Default Subobject 'ChildListNode' is invalid, this is not expected."));

	return ChildListNode->RemoveParametricObject(ParametricObjectNode);
}
