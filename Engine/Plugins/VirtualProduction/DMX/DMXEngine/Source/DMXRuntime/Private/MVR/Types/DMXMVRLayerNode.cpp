// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVR/Types/DMXMVRLayerNode.h"

#include "DMXRuntimeLog.h"
#include "DMXRuntimeUtils.h"
#include "MVR/Types/DMXMVRChildListNode.h"
#include "MVR/Types/DMXMVRFixtureNode.h"
#include "MVR/Types/DMXMVRParametricObjectNodeBase.h"

#include "XmlNode.h"


UDMXMVRLayerNode::UDMXMVRLayerNode()
{
	UUID = FGuid::NewGuid();
	ChildListNode = CreateDefaultSubobject<UDMXMVRChildListNode>("ChildListNode");
}

void UDMXMVRLayerNode::InitializeFromLayerXmlNode(const FXmlNode& LayerXmlNode)
{
	checkf(ChildListNode, TEXT("Default Subobject 'ChildListNode' is invalid, this is not expected."));
	checkf(LayerXmlNode.GetTag() == TEXT("layer"), TEXT("Trying to initialize a Layer Node from Xml Node, but the Node's Tag is '%s' instead of 'Layer'."), *LayerXmlNode.GetTag());

	// Init self
	constexpr TCHAR AttributeName_UUID[] = TEXT("UUID");
	const FString UUIDString = LayerXmlNode.GetAttribute(AttributeName_UUID);
	if (!FGuid::Parse(UUIDString, UUID))
	{
		UE_LOGF(LogDMXRuntime, Warning, "Found invalid UUID of Layer when parsing MVR File. Skipping Layer.")
		return;
	}
	constexpr TCHAR AttributeName_Name[] = TEXT("Name");
	const FString NameString = LayerXmlNode.GetAttribute(AttributeName_Name);
	if (!NameString.IsEmpty())
	{
		Name.Value = NameString;
	}
	
	// Init Matrix
	static const FString AttributeName_Matrix = TEXT("Matrix");
	const FString MatrixString = LayerXmlNode.GetAttribute(AttributeName_Matrix);
	Matrix = FDMXRuntimeUtils::ParseGDTFMatrix(MatrixString);

	// Init Child List Node
	static const FString NodeName_ChildList = TEXT("ChildList");
	const FXmlNode* ChildListXmlNode = LayerXmlNode.FindChildNode(NodeName_ChildList);

	if (ChildListXmlNode)
	{
		ChildListNode->InitializeFromChildListXmlNode(*ChildListXmlNode);
	}
}

void UDMXMVRLayerNode::CreateXmlNodeInParent(FXmlNode& ParentNode) const
{
	checkf(ChildListNode, TEXT("Default Subobject 'ChildListNode' is invalid, this is not expected."));
	checkf(ParentNode.GetTag() == TEXT("layers"), TEXT("Layer Nodes have to be created in a Layers node, but parent node is %s."), *ParentNode.GetTag());

	// Add self
	constexpr TCHAR Tag[] = TEXT("Layer");
	ParentNode.AppendChildNode(Tag);
	FXmlNode* LayerXmlNode = ParentNode.GetChildrenNodes().Last();
	check(LayerXmlNode);

	TArray<FXmlAttribute> Attributes;

	if (Name.IsSet())
	{
		constexpr TCHAR NameAttributeName[] = TEXT("name");
		FXmlAttribute Attribute = FXmlAttribute(NameAttributeName, Name.GetValue());
		Attributes.Add(Attribute);
	}
	const FString NonOptionalName = Name.IsSet() ? Name.GetValue() : TEXT("Unnamed"); // Hold a name for logging

	if (!UUID.IsValid())
	{
		UE_LOGF(LogDMXRuntime, Warning, "Cannot export MVR Layer '%ls'. Invalid UUID specified.", *NonOptionalName);
		return;
	}

	constexpr TCHAR UUIDAttributeName[] = TEXT("uuid");
	FXmlAttribute Attribute = FXmlAttribute(UUIDAttributeName, UUID.ToString(EGuidFormats::DigitsWithHyphens));
	Attributes.Add(Attribute);

	LayerXmlNode->SetAttributes(Attributes);


	// Add Matrix child Node
	if (Matrix.IsSet())
	{
		constexpr TCHAR MatrixTag[] = TEXT("Matrix");
		const FString MatrixContent = FDMXRuntimeUtils::ConvertTransformToGDTF4x3MatrixString(Matrix.GetValue());
		ParentNode.AppendChildNode(MatrixTag, MatrixContent);
		FXmlNode* MatrixXmlNode = ParentNode.GetChildrenNodes().Last();
		check(MatrixXmlNode);
	}

	// Add Child List child Node
	if (ChildListNode)
	{
		ChildListNode->CreateXmlNodeInParent(*LayerXmlNode);
	}
}

void UDMXMVRLayerNode::GetFixtureNodes(TArray<UDMXMVRFixtureNode*>& OutFixtureNodes) const
{
	checkf(ChildListNode, TEXT("Unexpected: ChildListNode is instanced, but its pointer is invalid."));
	ChildListNode->GetFixtureNodes(OutFixtureNodes);
}

bool UDMXMVRLayerNode::Contains(UDMXMVRParametricObjectNodeBase* ParametricObjectNode)
{
	checkf(ChildListNode, TEXT("Unexpected: ChildListNode is instanced, but its pointer is invalid."));
	if (ChildListNode->Contains(ParametricObjectNode))
	{
		return true;
	}

	return false;
}

TObjectPtr<UDMXMVRParametricObjectNodeBase>* UDMXMVRLayerNode::FindParametricObjectNodeByUUID(const FGuid& InUUID) const
{
	checkf(ChildListNode, TEXT("Unexpected: ChildListNode is instanced, but its pointer is invalid."));
	TObjectPtr<UDMXMVRParametricObjectNodeBase>* ParametricObjectNodePtr = ChildListNode->FindParametricObjectNodeByUUID(InUUID);
	if (ParametricObjectNodePtr)
	{
		return ParametricObjectNodePtr;
	}

	return nullptr;
}
