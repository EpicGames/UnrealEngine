// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVR/Types/DMXMVRLayersNode.h"

#include "MVR/Types/DMXMVRChildListNode.h"
#include "MVR/Types/DMXMVRLayerNode.h"
#include "MVR/Types/DMXMVRParametricObjectNodeBase.h"

#include "XmlNode.h"


void UDMXMVRLayersNode::InitializeFromLayersXmlNode(const FXmlNode& LayersXmlNode)
{
	checkf(LayersXmlNode.GetTag() == TEXT("layers"), TEXT("Trying to initialize a Layers Node from Xml Node, but the Node's Tag is '%s' instead of 'Layers'."), *LayersXmlNode.GetTag());

	// Init Layer Nodes
	LayerNodes.Reset();
	for (const FXmlNode* LayerXmlNode : LayersXmlNode.GetChildrenNodes())
	{
		UDMXMVRLayerNode* NewLayer = CreateLayer();
		NewLayer->InitializeFromLayerXmlNode(*LayerXmlNode);
	}
}

void UDMXMVRLayersNode::CreateXmlNodeInParent(FXmlNode& ParentNode) const
{
	checkf(ParentNode.GetTag() == TEXT("scene"), TEXT("The Layers Node has to be created in a Scene node, but parent node is %s."), *ParentNode.GetTag());

	constexpr TCHAR Tag[] = TEXT("Layers");
	ParentNode.AppendChildNode(Tag);
	FXmlNode* LayersXmlNode = ParentNode.GetChildrenNodes().Last();
	check(LayersXmlNode);

	for (const UDMXMVRLayerNode* LayerNode : LayerNodes)
	{
		LayerNode->CreateXmlNodeInParent(*LayersXmlNode);
	}
}

void UDMXMVRLayersNode::GetFixtureNodes(TArray<UDMXMVRFixtureNode*>& OutFixtureNodes) const
{
	for (UDMXMVRLayerNode* Layer : LayerNodes)
	{
		Layer->GetFixtureNodes(OutFixtureNodes);
	}
}

TObjectPtr<UDMXMVRParametricObjectNodeBase>* UDMXMVRLayersNode::FindParametricObjectNodeByUUID(const FGuid& UUID) const
{
	for (UDMXMVRLayerNode* Layer : LayerNodes)
	{
		TObjectPtr<UDMXMVRParametricObjectNodeBase>* ParametricObjectNodePtr = Layer->FindParametricObjectNodeByUUID(UUID);
		if (ParametricObjectNodePtr)
		{
			return ParametricObjectNodePtr;
		}
	}
	return nullptr;
}

UDMXMVRLayerNode* UDMXMVRLayersNode::CreateLayer()
{
	UDMXMVRLayerNode* NewLayerNode = NewObject<UDMXMVRLayerNode>(this);
	LayerNodes.Add(NewLayerNode);

	return NewLayerNode;
}

void UDMXMVRLayersNode::RemoveLayer(UDMXMVRLayerNode* LayerNodeToRemove)
{
	const int32 NumRemovedElements = LayerNodes.RemoveSingle(LayerNodeToRemove);
	ensureMsgf(NumRemovedElements == 1, TEXT("Tried to remove Layer from MVR Layers Node, but the Layer was not contained in the Layers Node."));
}
