// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVR/Types/DMXMVRParametricObjectNodeBase.h"

#include "MVR/Types/DMXMVRGroupObjectNode.h"
#include "MVR/Types/DMXMVRLayerNode.h"


UDMXMVRParametricObjectNodeBase::UDMXMVRParametricObjectNodeBase()
{
	UUID = FGuid::NewGuid();
}

UDMXMVRLayerNode* UDMXMVRParametricObjectNodeBase::GetLayer()
{
	return const_cast<UDMXMVRLayerNode*>(const_cast<const UDMXMVRParametricObjectNodeBase*>(this)->GetLayer());
}

const UDMXMVRLayerNode* UDMXMVRParametricObjectNodeBase::GetLayer() const
{
	// Traverse up the Outer until we find the Layer
	for (const UObject* Object = this; Object; Object = Object->GetOuter())
	{
		if (Object->GetClass() == UDMXMVRLayerNode::StaticClass())
		{
			return Cast<UDMXMVRLayerNode>(Object);
		}
		Object = Object->GetOuter();
	}

	ensureMsgf(0, TEXT("Unexpected: No outer Layer found for Parametric Object Node."));
	return nullptr;
}

UDMXMVRGroupObjectNode* UDMXMVRParametricObjectNodeBase::GetGroup()
{
	return const_cast<UDMXMVRGroupObjectNode*>(const_cast<const UDMXMVRParametricObjectNodeBase*>(this)->GetGroup());
}

const UDMXMVRGroupObjectNode* UDMXMVRParametricObjectNodeBase::GetGroup() const
{
	// Traverse up the Outer until we find the Layer
	for (const UObject* Object = this; Object; Object = Object->GetOuter())
	{
		if (Object->GetClass() == UDMXMVRLayerNode::StaticClass())
		{
			return Cast<UDMXMVRGroupObjectNode>(Object);
		}
		Object = Object->GetOuter();
	}

	return nullptr;
}

FTransform UDMXMVRParametricObjectNodeBase::GetTransformAbsolute() const
{
	FTransform Result = Matrix.IsSet() ? Matrix.GetValue() : FTransform::Identity;

	for (const UDMXMVRGroupObjectNode* Group = GetGroup(); Group; Group = Group->GetGroup())
	{
		if (Group->Matrix.IsSet())
		{
			Result *= Group->Matrix.GetValue();
		}
	}

	const UDMXMVRLayerNode* Layer = GetLayer();
	if (Layer && Layer->Matrix.IsSet())
	{
		Result *= Layer->Matrix.GetValue();
	}

	return Result;
}

void UDMXMVRParametricObjectNodeBase::SetTransformAbsolute(FTransform NewTransform)
{
	for (UDMXMVRGroupObjectNode* Group = GetGroup(); Group; Group = Group->GetGroup())
	{
		if (Group->Matrix.IsSet())
		{
			NewTransform = NewTransform.Inverse() * Group->Matrix.GetValue();
		}
	}

	UDMXMVRLayerNode* Layer = GetLayer();
	if (Layer && Layer->Matrix.IsSet())
	{
		NewTransform = NewTransform.Inverse() * Layer->Matrix.GetValue();
	}

	Matrix = NewTransform;
}
