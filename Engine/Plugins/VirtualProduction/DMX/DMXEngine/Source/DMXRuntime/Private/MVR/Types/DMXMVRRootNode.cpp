// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVR/Types/DMXMVRRootNode.h"

#include "DMXRuntimeLog.h"
#include "MVR/DMXMVRVersion.h"
#include "MVR/Types/DMXMVRChildListNode.h"
#include "MVR/Types/DMXMVRFixtureNode.h"
#include "MVR/Types/DMXMVRLayerNode.h"
#include "MVR/Types/DMXMVRLayersNode.h"
#include "MVR/Types/DMXMVRSceneNode.h"
#include "MVR/Types/DMXMVRUserDataNode.h"

#include "XmlFile.h"
#include "XmlNode.h"
#include "Misc/MessageDialog.h"


UDMXMVRRootNode::UDMXMVRRootNode()
{
	UserDataNode = CreateDefaultSubobject<UDMXMVRUserDataNode>("UserDataNode");
	SceneNode = CreateDefaultSubobject<UDMXMVRSceneNode>("SceneNode");
}

bool UDMXMVRRootNode::InitializeFromGeneralSceneDescriptionXml(const TSharedRef<FXmlFile>& GeneralSceneDescriptionXml)
{
	checkf(SceneNode, TEXT("Default Subobject 'SceneNode' is invalid, this is not expected."));

	// Init self
	FXmlNode* RootNode = GeneralSceneDescriptionXml->GetRootNode();
	if (!RootNode)
	{
		UE_LOGF(LogDMXRuntime, Warning, "Cannot parse MVR file. Root node is invalid.");
		return false;
	}

	constexpr TCHAR VerMajorAttributeName[] = TEXT("verMajor");
	const FString VerMajorString = RootNode->GetAttribute(VerMajorAttributeName);
	int32 VerMajor;
	if (!LexTryParseString<int32>(VerMajor, *VerMajorString))
	{
		UE_LOGF(LogDMXRuntime, Warning, "Invalid MVR GeneralSceneDescription.xml. Major Version is not set.");
		return false;
	}

	constexpr TCHAR VerMinorAttributeName[] = TEXT("verMinor");
	const FString VerMinorString = RootNode->GetAttribute(VerMinorAttributeName);
	int32 VerMinor;
	if (!LexTryParseString<int32>(VerMinor, *VerMinorString))
	{
		UE_LOGF(LogDMXRuntime, Warning, "Invalid MVR GeneralSceneDescription.xml. Minor Version is not set.");
		return false;
	}

	if (FDMXMVRVersion::MajorVersion < VerMajor ||
		(FDMXMVRVersion::MajorVersion == VerMajor && FDMXMVRVersion::MinorVersion < VerMinor))
	{
		const FText VerMajorText = FText::FromString(VerMajorString);
		const FText VerMinorText = FText::FromString(VerMinorString);
		const FText EngineMajorVerText = FText::FromString(FDMXMVRVersion::GetMajorVersionAsString());
		const FText EngineMinorVerText = FText::FromString(FDMXMVRVersion::GetMinorVersionAsString());

		const EAppReturnType::Type DialogResult = FMessageDialog::Open(EAppMsgType::YesNo, FText::Format(NSLOCTEXT("DMXMVRRootNode", "TryLoadMVRWithVersionNewThanEngineDialog", "Warning: Version '{0}.{1}' of MVR is newer than the MVR Version supported by the Engine, '{2}.{3}'. Do you want to try to load the MVR anyways (not recommended)?"), VerMajorText, VerMinorText, EngineMajorVerText, EngineMinorVerText));
		if (DialogResult == EAppReturnType::No)
		{
			return false;
		}
	}

	// Init Scene Node
	static const FString NodeName_Scene = TEXT("Scene");
	FXmlNode* SceneXmlNode = RootNode->FindChildNode(NodeName_Scene);
	if(SceneXmlNode)
	{
		SceneNode->InitializeFromSceneXmlNode(*SceneXmlNode);
	}

	return true;
}

TSharedPtr<FXmlFile> UDMXMVRRootNode::CreateXmlFile()
{
	checkf(UserDataNode, TEXT("Default Subobject 'UserDataNode' is invalid, this is not expected."));
	checkf(SceneNode, TEXT("Default Subobject 'SceneNode' is invalid, this is not expected."));

	// Don't export if there's nothing to export
	TArray<UDMXMVRFixtureNode*> FixtureNodes;
	GetFixtureNodes(FixtureNodes);
	if (FixtureNodes.IsEmpty())
	{
		return nullptr;
	}

	// Create Xml File
	const FString Buffer = "<?xml version=\"1.0\" encoding=\"UTF - 8\" standalone=\"no\" ?>\n<GeneralSceneDescription>\n</GeneralSceneDescription>";

	TSharedRef<FXmlFile> XmlFile = MakeShared<FXmlFile>();
	const bool bCreatedNewFile = XmlFile->LoadFile(Buffer, EConstructMethod::ConstructFromBuffer);

	FXmlNode* RootNode = XmlFile->GetRootNode();
	if (!ensureAlwaysMsgf(bCreatedNewFile && RootNode, TEXT("Unexpected failed to create a new MVR General Scene Description in memory. MVR Export failed.")))
	{
		return nullptr;
	}
	check(RootNode);

	// Version the Root Node
	constexpr TCHAR VerMajorAttributeName[] = TEXT("verMajor");
	constexpr TCHAR VerMinorAttributeName[] = TEXT("verMinor");
	TArray<FXmlAttribute> Attributes =
	{
		FXmlAttribute(VerMajorAttributeName, FDMXMVRVersion::GetMajorVersionAsString()),
		FXmlAttribute(VerMinorAttributeName, FDMXMVRVersion::GetMinorVersionAsString())
	};
	RootNode->SetAttributes(Attributes);
	
	// Export Children
	if (UserDataNode)
	{
		UserDataNode->CreateXmlNodeInParent(*RootNode);
	}

	if (SceneNode)
	{
		SceneNode->CreateXmlNodeInParent(*RootNode);
	}

	return XmlFile;
}

void UDMXMVRRootNode::GetFixtureNodes(TArray<UDMXMVRFixtureNode*>& OutFixtureNodes) const
{
	checkf(SceneNode, TEXT("Default Subobject 'SceneNode' is invalid, this is not expected."));
	UDMXMVRLayersNode* LayersNode = SceneNode ? SceneNode->GetLayersNode() : nullptr;
	checkf(LayersNode, TEXT("Cannot Find Layer node in Scene Node."));
	LayersNode->GetFixtureNodes(OutFixtureNodes);
}

UDMXMVRChildListNode& UDMXMVRRootNode::GetOrCreateFirstChildListNode()
{
	checkf(SceneNode, TEXT("Default Subobject 'SceneNode' is invalid, this is not expected."));
	UDMXMVRLayersNode* LayersNode = SceneNode ? SceneNode->GetLayersNode() : nullptr;
	checkf(LayersNode, TEXT("Cannot Find Layers node in Scene Node. This is not expected."));

	UDMXMVRLayerNode* LayerNode = nullptr;
	if (LayersNode->GetLayerNodes().IsEmpty())
	{
		LayerNode = LayersNode->CreateLayer();
	}
	else
	{
		LayerNode = LayersNode->GetLayerNodes()[0];
	}
	check(LayerNode);

	UDMXMVRChildListNode* ChildListNode = LayerNode->GetChildListNode();
	checkf(ChildListNode, TEXT("Cannot Find Child List node in Layer Node. This is not expected."));
	return *ChildListNode;
}

TObjectPtr<UDMXMVRParametricObjectNodeBase>* UDMXMVRRootNode::FindParametricObjectNodeByUUID(const FGuid& UUID) const
{
	UDMXMVRLayersNode* LayersNode = SceneNode ? SceneNode->GetLayersNode() : nullptr;
	if (LayersNode)
	{
		return LayersNode->FindParametricObjectNodeByUUID(UUID);
	}
	return nullptr;
}

bool UDMXMVRRootNode::RemoveParametricObjectNode(UDMXMVRParametricObjectNodeBase* ParametricObjectNode)
{
	checkf(SceneNode, TEXT("Default Subobject 'SceneNode' is invalid, this is not expected."));
	UDMXMVRLayersNode* LayersNode = SceneNode ? SceneNode->GetLayersNode() : nullptr;
	checkf(LayersNode, TEXT("Cannot Find Layers node in Scene Node. This is not expected."));

	for (UDMXMVRLayerNode* Layer : LayersNode->GetLayerNodes())
	{
		if (UDMXMVRChildListNode* ChildList = Layer->GetChildListNode())
		{
			if (ChildList->RemoveParametricObject(ParametricObjectNode))
			{
				return true;
			}
		}
	}

	return false;
}
