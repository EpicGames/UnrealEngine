// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVR/Types/DMXMVRSceneNode.h"

#include "MVR/Types/DMXMVRLayersNode.h"

#include "XmlNode.h"


UDMXMVRSceneNode::UDMXMVRSceneNode()
{
	LayersNode = CreateDefaultSubobject<UDMXMVRLayersNode>("LayersNode");
}

void UDMXMVRSceneNode::InitializeFromSceneXmlNode(const FXmlNode& SceneXmlNode)
{
	checkf(LayersNode, TEXT("Default Subobject 'LayersNode' is invalid, this is not expected."));
	checkf(SceneXmlNode.GetTag() == TEXT("scene"), TEXT("Trying to initialize a Scene Node from Xml Node, but the Node's Tag is '%s' instead of 'Scene'."), *SceneXmlNode.GetTag());

	// Init Layers Node
	static const FString LayersTag = TEXT("Layers");
	const FXmlNode* LayersXmlNode = SceneXmlNode.FindChildNode(LayersTag);
	if (LayersXmlNode)
	{
		LayersNode->InitializeFromLayersXmlNode(*LayersXmlNode);
	}
}

void UDMXMVRSceneNode::CreateXmlNodeInParent(FXmlNode& ParentNode) const
{
	checkf(LayersNode, TEXT("Default Subobject 'LayersNode' is invalid, this is not expected."));
	checkf(ParentNode.GetTag() == TEXT("generalscenedescription"), TEXT("The Scene Node has to be created in a General Scene Description node, but parent node is %s."), *ParentNode.GetTag());

	constexpr TCHAR Tag[] = TEXT("Scene");
	constexpr TCHAR Content[] = TEXT("Content");
	ParentNode.AppendChildNode(Tag, Content);
	FXmlNode* SceneXmlNode = ParentNode.GetChildrenNodes().Last();
	check(SceneXmlNode);

	if (LayersNode)
	{
		LayersNode->CreateXmlNodeInParent(*SceneXmlNode);
	}
}
