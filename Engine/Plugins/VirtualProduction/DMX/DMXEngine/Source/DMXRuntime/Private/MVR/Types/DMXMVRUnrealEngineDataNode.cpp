// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVR/Types/DMXMVRUnrealEngineDataNode.h"

#include "XmlNode.h"
#include "Misc/EngineVersion.h"


void UDMXMVRUnrealEngineDataNode::CreateXmlNodeInParent(FXmlNode& ParentNode) const
{
	checkf(ParentNode.GetTag() == TEXT("userdata"), TEXT("The Data Node has to be created in a General Scene Description node, but parent node is %s."), *ParentNode.GetTag());

	constexpr TCHAR Tag[] = TEXT("Data");
	ParentNode.AppendChildNode(Tag);
	FXmlNode* DataXmlNode = ParentNode.GetChildrenNodes().Last();
	check(DataXmlNode);

	constexpr TCHAR ProviderAttributeName[] = TEXT("provider");
	constexpr TCHAR VerAttributeName[] = TEXT("ver");
	const FString VersionString =
		FString::FromInt(FEngineVersion::Current().GetMajor()) +
		TEXT(".") +
		FString::FromInt(FEngineVersion::Current().GetMinor()) +
		TEXT(".") +
		FString::FromInt(FEngineVersion::Current().GetChangelist());

	const TArray<FXmlAttribute> Attributes = 
	{ 
		FXmlAttribute(ProviderAttributeName, ProviderNameUnrealEngine),
		FXmlAttribute(VerAttributeName, VersionString)
	};
	DataXmlNode->SetAttributes(Attributes);

	// 'Date' child node
	constexpr TCHAR ChildTag_Date[] = TEXT("CreationDate");
	DataXmlNode->AppendChildNode(ChildTag_Date, FDateTime::Now().ToString());
}
