// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVR/Types/DMXMVRUserDataNode.h"

#include "MVR/Types/DMXMVRUnrealEngineDataNode.h"

#include "XmlNode.h"


UDMXMVRUserDataNode::UDMXMVRUserDataNode()
{
	UnrealEngineDataNode = CreateDefaultSubobject<UDMXMVRUnrealEngineDataNode>("UnrealEngineDataNode");
}

void UDMXMVRUserDataNode::CreateXmlNodeInParent(FXmlNode& ParentNode) const
{
	checkf(UnrealEngineDataNode, TEXT("Default Subobject 'UnrealEngineDataNode' is invalid, this is not expected."));
	checkf(ParentNode.GetTag() == TEXT("generalscenedescription"), TEXT("The User Data Node has to be created in the General Scene Description node, but parent node is %s."), *ParentNode.GetTag());

	constexpr TCHAR Tag[] = TEXT("UserData");
	ParentNode.AppendChildNode(Tag);
	FXmlNode* UserDataXmlNode = ParentNode.GetChildrenNodes().Last();
	check(UserDataXmlNode);

	UnrealEngineDataNode->CreateXmlNodeInParent(*UserDataXmlNode);
}
