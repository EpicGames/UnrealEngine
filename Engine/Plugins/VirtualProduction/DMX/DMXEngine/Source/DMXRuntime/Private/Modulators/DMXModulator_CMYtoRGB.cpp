// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modulators/DMXModulator_CMYtoRGB.h"


UDMXModulator_CMYtoRGB::UDMXModulator_CMYtoRGB()
{
	AttributeCyanToRed.SetFromName("Cyan");
	AttributeMagentaToGreen.SetFromName("Magenta");
	AttributeYellowToBlue.SetFromName("Yellow");
}

void UDMXModulator_CMYtoRGB::Modulate_Implementation(UDMXEntityFixturePatch* FixturePatch, const TMap<FDMXAttributeName, float>& InNormalizedAttributeValues, TMap<FDMXAttributeName, float>& OutNormalizedAttributeValues)
{
	// Note, in fact implementation wise CMYtoRGB and RGBtoCYM are clones, they exist only for their names
	OutNormalizedAttributeValues = InNormalizedAttributeValues;
	for (TTuple<FDMXAttributeName, float>& NormalizedAttributeValue : OutNormalizedAttributeValues)
	{
		if (NormalizedAttributeValue.Key.Name == AttributeCyanToRed ||
			NormalizedAttributeValue.Key.Name == AttributeMagentaToGreen ||
			NormalizedAttributeValue.Key.Name == AttributeYellowToBlue)
		{
			NormalizedAttributeValue.Value = FMath::Abs(NormalizedAttributeValue.Value - 1.f);
		}
	}
}

void UDMXModulator_CMYtoRGB::ModulateMatrix_Implementation(UDMXEntityFixturePatch* FixturePatch, const TArray<FDMXNormalizedAttributeValueMap>& InNormalizedMatrixAttributeValues, TArray<FDMXNormalizedAttributeValueMap>& OutNormalizedMatrixAttributeValues)
{
	// Note, in fact implementation wise CMYtoRGB and RGBtoCYM are clones, they exist only for their names
	OutNormalizedMatrixAttributeValues = InNormalizedMatrixAttributeValues;;

	for (FDMXNormalizedAttributeValueMap& NormalizedAttributeValueMap : OutNormalizedMatrixAttributeValues)
	{
		for (TTuple<FDMXAttributeName, float>& AttributeNameNormalizedValuePair : NormalizedAttributeValueMap.Map)
		{
			if (AttributeNameNormalizedValuePair.Key.Name == AttributeCyanToRed ||
				AttributeNameNormalizedValuePair.Key.Name == AttributeMagentaToGreen ||
				AttributeNameNormalizedValuePair.Key.Name == AttributeYellowToBlue)
			{
				AttributeNameNormalizedValuePair.Value = FMath::Abs(AttributeNameNormalizedValuePair.Value - 1.f);
			}
		}
	}
}
