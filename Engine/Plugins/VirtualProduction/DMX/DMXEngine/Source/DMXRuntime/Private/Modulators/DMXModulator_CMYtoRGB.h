// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modulators/DMXModulator.h"

#include "DMXModulator_CMYtoRGB.generated.h"


/** Converts Attributes from CMY to RGB. */
UCLASS(NotBlueprintable, DisplayName = "DMX Modulator CMY to RGB", AutoExpandCategories = ("DMX"))
class DMXRUNTIME_API UDMXModulator_CMYtoRGB
	: public UDMXModulator
{
	GENERATED_BODY()

public:
	UDMXModulator_CMYtoRGB();

	/** RGB to CMY implementation */
	virtual void Modulate_Implementation(UDMXEntityFixturePatch* FixturePatch, const TMap<FDMXAttributeName, float>& InNormalizedAttributeValues, TMap<FDMXAttributeName, float>& OutNormalizedAttributeValues) override;

	/** Matrix RGB to CMY implementation */
	virtual void ModulateMatrix_Implementation(UDMXEntityFixturePatch* FixturePatch, const TArray<FDMXNormalizedAttributeValueMap>& InNormalizedMatrixAttributeValues, TArray<FDMXNormalizedAttributeValueMap>& OutNormalizedMatrixAttributeValues) override;

	/** The name of the attribute that is converted from Cyan to Red */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "CMY to RGB", Meta = (DisplayName = "Cyan to Red", AdvancedDisplay))
	FDMXAttributeName AttributeCyanToRed;

	/** The name of the attribute that is converted from Magenta to Green */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "CMY to RGB", Meta = (DisplayName = "Magenta to Green", AdvancedDisplay))
	FDMXAttributeName AttributeMagentaToGreen;

	/** The name of the attribute that is converted from Yellow to Blue */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "CMY to RGB", Meta = (DisplayName = "Yellow to Blue", AdvancedDisplay))
	FDMXAttributeName AttributeYellowToBlue;
};

