// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modulators/DMXModulator_ExtraAttributes.h"


UDMXModulator_ExtraAttributes::UDMXModulator_ExtraAttributes()
{
	FDMXAttributeName Dimmer;
	Dimmer.SetFromName("Dimmer");
	ExtraAttributeNameToNormalizedValueMap.Add(Dimmer, 1.f);
}

void UDMXModulator_ExtraAttributes::Modulate_Implementation(UDMXEntityFixturePatch* FixturePatch, const TMap<FDMXAttributeName, float>& InNormalizedAttributeValues, TMap<FDMXAttributeName, float>& OutNormalizedAttributeValues)
{
	OutNormalizedAttributeValues = InNormalizedAttributeValues;

	for (const TTuple<FDMXAttributeName, float>& ExtraAttributeNormalizedValuePair : ExtraAttributeNameToNormalizedValueMap)
	{
		OutNormalizedAttributeValues.Add(ExtraAttributeNormalizedValuePair.Key, ExtraAttributeNormalizedValuePair.Value);
	}
}
