// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modulators/DMXModulator.h"

#include "DMXModulator_ExtraAttributes.generated.h"


/** Adds attributes that are not received (e.g. because DMX was generated from PixelMapping) to the DMX signal */
UCLASS(NotBlueprintable, DisplayName = "DMX Modulator Extra Attributes", AutoExpandCategories = ("DMX"))
class DMXRUNTIME_API UDMXModulator_ExtraAttributes
	: public UDMXModulator

{
	GENERATED_BODY()

public:
	UDMXModulator_ExtraAttributes();

	/** Attribute value override implementation */
	virtual void Modulate_Implementation(UDMXEntityFixturePatch* FixturePatch, const TMap<FDMXAttributeName, float>& InNormalizedAttributeValues, TMap<FDMXAttributeName, float>& OutNormalizedAttributeValues) override;

	/** Adds the attributes with their values to the Output if they don't exist, or replaces them with the values specified */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Extra Attributes", Meta = (DisplayName = "Attribute to Normalized Value Map"))
	TMap<FDMXAttributeName, float> ExtraAttributeNameToNormalizedValueMap;
};
