// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modulators/DMXModulator_ExtraCellAttributes.h"


UDMXModulator_ExtraCellAttributes::UDMXModulator_ExtraCellAttributes()
{}

void UDMXModulator_ExtraCellAttributes::ModulateMatrix_Implementation(UDMXEntityFixturePatch* FixturePatch, const TArray<FDMXNormalizedAttributeValueMap>& InNormalizedMatrixAttributeValues, TArray<FDMXNormalizedAttributeValueMap>& OutNormalizedMatrixAttributeValues)
{
	OutNormalizedMatrixAttributeValues = InNormalizedMatrixAttributeValues;;

	for (FDMXNormalizedAttributeValueMap& NormalizedAttributeValueMap : OutNormalizedMatrixAttributeValues)
	{
		for (const TTuple<FDMXAttributeName, float>& ExtraAttributeNormalizedValuePair : ExtraAttributeNameToNormalizedValueMap)
		{
			NormalizedAttributeValueMap.Map.Add(ExtraAttributeNormalizedValuePair.Key, ExtraAttributeNormalizedValuePair.Value);
		}
	}
}
