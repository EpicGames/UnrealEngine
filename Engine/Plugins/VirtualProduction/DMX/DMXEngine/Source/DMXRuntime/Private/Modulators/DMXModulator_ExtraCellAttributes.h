// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modulators/DMXModulator.h"

#include "DMXModulator_ExtraCellAttributes.generated.h"


/** Adds matrix attributes that are not received (e.g. because DMX was generated from PixelMapping) to the DMX signal */
UCLASS(NotBlueprintable, DisplayName = "DMX Modulator Extra Cell Attributes", AutoExpandCategories = ("DMX"))
class DMXRUNTIME_API UDMXModulator_ExtraCellAttributes
	: public UDMXModulator

{
	GENERATED_BODY()

public:
	UDMXModulator_ExtraCellAttributes();

	/** Matrix Attribute value override implementation */
	virtual void ModulateMatrix_Implementation(UDMXEntityFixturePatch* FixturePatch, const TArray<FDMXNormalizedAttributeValueMap>& InNormalizedMatrixAttributeValues, TArray<FDMXNormalizedAttributeValueMap>& OutNormalizedMatrixAttributeValues) override;

	/** Adds the attributes with their values to the Output if they don't exist, or replaces them with the values specified */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Extra Cell Attributes", Meta = (DisplayName = "Attribute to Normalized Value Map"))
	TMap<FDMXAttributeName, float> ExtraAttributeNameToNormalizedValueMap;
};
