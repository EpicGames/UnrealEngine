// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modulators/DMXModulator_RGBtoCMY.h"


UDMXModulator_RGBtoCMY::UDMXModulator_RGBtoCMY()
{
	AttributeRedToCyan.SetFromName("Red");
	AttributeGreenToMagenta.SetFromName("Green");
	AttributeBlueToYellow.SetFromName("Blue");
}

void UDMXModulator_RGBtoCMY::Modulate_Implementation(UDMXEntityFixturePatch* FixturePatch, const TMap<FDMXAttributeName, float>& InNormalizedAttributeValues, TMap<FDMXAttributeName, float>& OutNormalizedAttributeValues)
{
	// Note, in fact implementation wise CMYtoRGB and RGBtoCYM are clones, they exist only for their names
	OutNormalizedAttributeValues = InNormalizedAttributeValues;
	for (TTuple<FDMXAttributeName, float>& NormalizedAttributeValue : OutNormalizedAttributeValues)
	{
		if (NormalizedAttributeValue.Key.Name == AttributeRedToCyan ||
			NormalizedAttributeValue.Key.Name == AttributeGreenToMagenta ||
			NormalizedAttributeValue.Key.Name == AttributeBlueToYellow)
		{
			NormalizedAttributeValue.Value = FMath::Abs(NormalizedAttributeValue.Value - 1.f);
		}
	}
}

void UDMXModulator_RGBtoCMY::ModulateMatrix_Implementation(UDMXEntityFixturePatch* FixturePatch, const TArray<FDMXNormalizedAttributeValueMap>& InNormalizedMatrixAttributeValues, TArray<FDMXNormalizedAttributeValueMap>& OutNormalizedMatrixAttributeValues)
{
	// Note, in fact implementation wise CMYtoRGB and RGBtoCYM are clones, they exist only for their names
	OutNormalizedMatrixAttributeValues = InNormalizedMatrixAttributeValues;;

	for (FDMXNormalizedAttributeValueMap& NormalizedAttributeValueMap : OutNormalizedMatrixAttributeValues)
	{
		for (TTuple<FDMXAttributeName, float>& AttributeNameNormalizedValuePair : NormalizedAttributeValueMap.Map)
		{
			if (AttributeNameNormalizedValuePair.Key.Name == AttributeRedToCyan ||
				AttributeNameNormalizedValuePair.Key.Name == AttributeGreenToMagenta ||
				AttributeNameNormalizedValuePair.Key.Name == AttributeBlueToYellow)
			{
				AttributeNameNormalizedValuePair.Value = FMath::Abs(AttributeNameNormalizedValuePair.Value - 1.f);
			}
		}
	}
}

