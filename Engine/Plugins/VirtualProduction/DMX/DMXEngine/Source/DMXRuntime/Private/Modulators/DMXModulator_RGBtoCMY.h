// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modulators/DMXModulator.h"

#include "DMXModulator_RGBtoCMY.generated.h"


/** Converts Attributes from RGB to CMY  */
UCLASS(NotBlueprintable, DisplayName = "DMX Modulator RGB to CMY", AutoExpandCategories = ("DMX"))
class DMXRUNTIME_API UDMXModulator_RGBtoCMY
	: public UDMXModulator

{
	GENERATED_BODY()

public:
	UDMXModulator_RGBtoCMY();

	/** RGB to CMY implementation */
	virtual void Modulate_Implementation(UDMXEntityFixturePatch* FixturePatch, const TMap<FDMXAttributeName, float>& InNormalizedAttributeValues, TMap<FDMXAttributeName, float>& OutNormalizedAttributeValues) override;

	/** Matrix RGB to CMY implementation */
	virtual void ModulateMatrix_Implementation(UDMXEntityFixturePatch* FixturePatch, const TArray<FDMXNormalizedAttributeValueMap>& InNormalizedMatrixAttributeValues, TArray<FDMXNormalizedAttributeValueMap>& OutNormalizedMatrixAttributeValues) override;

	/** The name of the attribute that is converted from Red to Cyan */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "RGB to CMY", Meta = (DisplayName = "Red to Cyan", AdvancedDisplay))
	FDMXAttributeName AttributeRedToCyan;

	/** The name of the attribute that is converted from Green to Magenta */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "RGB to CMY", Meta = (DisplayName = "Green to Magenta", AdvancedDisplay))
	FDMXAttributeName AttributeGreenToMagenta;

	/** The name of the attribute that is converted from Blue to Yellow */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "RGB to CMY", Meta = (DisplayName = "Blue to Yellow", AdvancedDisplay))
	FDMXAttributeName AttributeBlueToYellow;
};
