// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/MovieSceneDMXLibrarySection.h"

#include "Channels/MovieSceneChannelEditorData.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "DMXConversions.h"
#include "DMXProtocolCommon.h"
#include "DMXRuntimeLog.h"
#include "DMXRuntimeMainStreamObjectVersion.h"
#include "DMXSubsystem.h"
#include "Evaluation/Blending/MovieSceneBlendType.h"
#include "Interfaces/IDMXProtocol.h"
#include "IO/DMXInputPort.h"
#include "IO/DMXOutputPort.h"
#include "IO/DMXPortManager.h"
#include "IO/DMXTrace.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/DMXLibrary.h"
#include "UObject/Package.h"
#include "Algo/Find.h"

DECLARE_LOG_CATEGORY_CLASS(MovieSceneDMXLibrarySectionLog, Log, All);

void FDMXFixturePatchChannel::SetFixturePatch(UDMXEntityFixturePatch* InPatch)
{
	if (InPatch != nullptr && InPatch->IsValidLowLevelFast())
	{
		DMXLibrary = InPatch->GetParentLibrary();
		ActiveMode = InPatch->GetActiveModeIndex();
	}

	Reference.SetEntity(InPatch);
	UpdateNumberOfChannels();
}

void FDMXFixturePatchChannel::UpdateNumberOfChannels(bool bResetDefaultValues /*= false*/)
{
	// Test if the patch is still being what was recorded

	UDMXEntityFixturePatch* Patch = Reference.GetFixturePatch();
	UDMXEntityFixtureType* FixtureType = Patch ? Patch->GetFixtureType() : nullptr;
	if (!Patch ||
		!FixtureType ||
		!FixtureType->Modes.IsValidIndex(ActiveMode))
	{
		UE_LOGF(LogDMXRuntime, Warning, "Cannot find patch for Sequence. Likely it was removed from its DMX Library. Corresponding DMX Channel are not created.");

		if (!FixtureType)
		{
			UE_LOGF(LogDMXRuntime, Warning, "Sequence has no valid Parent Fixture Type set. Corresponding DMX Channel removed from Sequencer.");
		}
		else if (!FixtureType->Modes.IsValidIndex(ActiveMode))
		{
			UE_LOGF(LogDMXRuntime, Warning, "Recorded referenced Mode no longer exists in Sequence. Channel Removed from Sequencer.");
		}

		FunctionChannels.Empty();
		ActiveMode = INDEX_NONE;
		return;
	}

	const FDMXFixtureMode& Mode = FixtureType->Modes[ActiveMode];
	if (Patch->GetActiveModeIndex() != ActiveMode)
	{
		UE_LOGF(LogDMXRuntime, Warning, "Active Mode of '%ls' changed. Its channel in Sequencer uses recorded Mode.", *FixtureType->Name);
		UE_LOGF(LogDMXRuntime, Warning, "Only channels with matching attribute will be displayed and played, potentially resulting in empty channels.");
	}

	const int32 PatchChannelOffset = Patch->GetStartingChannel() - 1;

	int32 IdxFunctionChannel = 0;
	for (const FDMXFixtureFunction& Function : Mode.Functions)
	{
		// Add channels for functions that are in mode range and have an Attribute set

		if (Function.GetLastChannel() <= Patch->GetChannelSpan() &&
			!Function.Attribute.Name.IsNone())
		{
			bool bNewChannel = false;
			if (!FunctionChannels.IsValidIndex(IdxFunctionChannel))
			{
				FunctionChannels.Add(FDMXFixtureFunctionChannel());
				bNewChannel = true;
			}					

			FunctionChannels[IdxFunctionChannel].AttributeName = Function.Attribute.Name;

			if (bResetDefaultValues || bNewChannel)
			{
				const int64& FunctionDefaultValue = Function.DefaultValue;
				FunctionChannels[IdxFunctionChannel].DefaultValue = (int32)FunctionDefaultValue;
				FunctionChannels[IdxFunctionChannel].Channel.Reset();
				FunctionChannels[IdxFunctionChannel].Channel.SetDefault((float)FunctionDefaultValue);
			}

			++IdxFunctionChannel;
		}
	}

	if (Mode.bFixtureMatrixEnabled)
	{
		int32 NumXCells = Mode.FixtureMatrixConfig.XCells;
		int32 NumYCells = Mode.FixtureMatrixConfig.YCells;

		for (int32 IdxCellY = 0; IdxCellY < NumYCells; IdxCellY++)
		{
			for (int32 IdxCellX = 0; IdxCellX < NumXCells; IdxCellX++)
			{
				FIntPoint CellCoordinates = FIntPoint(IdxCellX, IdxCellY);

				for (const FDMXFixtureCellAttribute& CellAttribute : Mode.FixtureMatrixConfig.CellAttributes)
				{
					bool bNewChannel = false;
					if (!FunctionChannels.IsValidIndex(IdxFunctionChannel))
					{
						FunctionChannels.Add(FDMXFixtureFunctionChannel());
						bNewChannel = true;
					}

					FunctionChannels[IdxFunctionChannel].AttributeName = CellAttribute.Attribute.Name;
					FunctionChannels[IdxFunctionChannel].CellCoordinate = CellCoordinates;

					if (bResetDefaultValues || bNewChannel)
					{
						const int64& FunctionDefaultValue = CellAttribute.DefaultValue;
						FunctionChannels[IdxFunctionChannel].DefaultValue = (int32)FunctionDefaultValue;
						FunctionChannels[IdxFunctionChannel].Channel.Reset();
						FunctionChannels[IdxFunctionChannel].Channel.SetDefault((float)FunctionDefaultValue);
					}

					++IdxFunctionChannel;
				}
			}
		}
	}

	if(FunctionChannels.Num() > IdxFunctionChannel + 1)
	{
		FunctionChannels.SetNum(IdxFunctionChannel + 1);
	}
}

FDMXCachedFunctionChannelInfo::FDMXCachedFunctionChannelInfo(const TArray<FDMXFixturePatchChannel>& FixturePatchChannels, int32 InPatchChannelIndex, int32 InFunctionChannelIndex)
	: bNeedsInitialization(false)
	, bNeedsEvaluation(false)
	, PatchChannelIndex(InPatchChannelIndex)
	, FunctionChannelIndex(InFunctionChannelIndex)
	, AttributeName(NAME_None)
	, UniverseID(-1)
	, StartingChannel(-1)
	, SignalFormat(EDMXFixtureSignalFormat::E8Bit)
	, bLSBMode(false)
{
	check(FixturePatchChannels.IsValidIndex(PatchChannelIndex));
	const FDMXFixturePatchChannel& FixturePatchChannel = FixturePatchChannels[PatchChannelIndex];

	check(FixturePatchChannel.FunctionChannels.IsValidIndex(FunctionChannelIndex));
	const FDMXFixtureFunctionChannel& FunctionChannel = FixturePatchChannel.FunctionChannels[FunctionChannelIndex];
	
	// Valid patch
	UDMXEntityFixturePatch* FixturePatch = FixturePatchChannel.Reference.GetFixturePatch();
	if (!FixturePatch || !FixturePatch->IsValidLowLevelFast())
	{
		UE_LOGF(MovieSceneDMXLibrarySectionLog, Error, "%s: A Fixture Patch is null.", __FUNCTION__);
		return;
	}

	// Enabled
	if (!FunctionChannel.bEnabled)
	{
		return;
	}

	// Try to access the active mode
	const UDMXEntityFixtureType* FixtureType = FixturePatch->GetFixtureType();
	if (!FixtureType || !FixtureType->IsValidLowLevelFast())
	{
		UE_LOGF(MovieSceneDMXLibrarySectionLog, Error, "%s: Patch %ls has invalid Fixture Type template.", __FUNCTION__, *FixturePatch->GetDisplayName());
		return;
	}

	if (FixturePatchChannel.ActiveMode >= FixtureType->Modes.Num())
	{
		UE_LOGF(MovieSceneDMXLibrarySectionLog, Error, "%s: Patch track %ls ActiveMode is invalid.", __FUNCTION__, *FixturePatch->GetDisplayName());
		return;
	}

	const FDMXFixtureMode& Mode = FixtureType->Modes[FixturePatchChannel.ActiveMode];

	// Cache Fuction properties
	if (FunctionChannel.IsCellFunction())
	{
		const FDMXFixtureMatrix& MatrixConfig = Mode.FixtureMatrixConfig;
		const TArray<FDMXFixtureCellAttribute>& CellAttributes = MatrixConfig.CellAttributes;
		
		TMap<FName, int32> AttributeNameChannelMap;
		GetMatrixCellChannelsAbsoluteNoSorting(FixturePatch, FunctionChannel.CellCoordinate, AttributeNameChannelMap);

		const FDMXFixtureCellAttribute* CellAttributePtr = CellAttributes.FindByPredicate([&FunctionChannel](const FDMXFixtureCellAttribute& CellAttribute) {
			return CellAttribute.Attribute == FunctionChannel.AttributeName;
			});

		const bool bMissingFunction = !AttributeNameChannelMap.Contains(FunctionChannel.AttributeName) || !CellAttributePtr;
		if (!CellAttributePtr || bMissingFunction)
		{
			UE_LOGF(MovieSceneDMXLibrarySectionLog, Warning, "%s: Function with attribute %ls from %ls doesn't have a counterpart Fixture Function.", __FUNCTION__, *FunctionChannel.AttributeName.ToString(), *FixturePatch->GetDisplayName());
			UE_LOGF(MovieSceneDMXLibrarySectionLog, Warning, "%s: Further attributes may be missing. Warnings ommited to avoid overflowing the log.", __FUNCTION__);

			return;
		}

		const FDMXFixtureCellAttribute& CellAttribute = *CellAttributePtr;

		AttributeName = CellAttribute.Attribute.Name;
		StartingChannel = AttributeNameChannelMap[FunctionChannel.AttributeName];
		SignalFormat = CellAttribute.DataType;
		bLSBMode = CellAttribute.bUseLSBMode;
	}
	else
	{
		const TArray<FDMXFixtureFunction>& Functions = Mode.Functions;

		const FDMXFixtureFunction* FunctionPtr = Functions.FindByPredicate([&FunctionChannel](const FDMXFixtureFunction& TestedFunction) {
			return TestedFunction.Attribute == FunctionChannel.AttributeName;
			});

		if (!FunctionPtr)
		{
			UE_LOGF(MovieSceneDMXLibrarySectionLog, Warning, "%s: Function with attribute %ls from %ls doesn't have a counterpart Fixture Function.", __FUNCTION__, *FunctionChannel.AttributeName.ToString(), *FixturePatch->GetDisplayName());

			return;
		}

		const FDMXFixtureFunction& Function = *FunctionPtr;

		AttributeName = Function.Attribute.Name;

		const int32 PatchChannelOffset = FixturePatch->GetStartingChannel() - 1;
		StartingChannel = Function.Channel + PatchChannelOffset;
		SignalFormat = Function.DataType;
		bLSBMode = Function.bUseLSBMode;
	}

	UniverseID = FixturePatch->GetUniverseID();

	// Now that we know it's fully valid, define how it should be processed.
	bNeedsInitialization = FunctionChannel.Channel.GetNumKeys() > 0;
	bNeedsEvaluation = FunctionChannel.Channel.GetNumKeys() > 1;
}
	
const FDMXFixtureFunctionChannel* FDMXCachedFunctionChannelInfo::TryGetFunctionChannel(const TArray<FDMXFixturePatchChannel>& FixturePatchChannels) const
{
	if(FixturePatchChannels.IsValidIndex(PatchChannelIndex))
	{
		const FDMXFixturePatchChannel& FixturePatchChannel = FixturePatchChannels[PatchChannelIndex];

		if(FixturePatchChannel.FunctionChannels.IsValidIndex(FunctionChannelIndex))
		{
			const FDMXFixtureFunctionChannel& FunctionChannel = FixturePatchChannel.FunctionChannels[FunctionChannelIndex];

			if(FunctionChannel.AttributeName == AttributeName)
			{
				return &FunctionChannel;
			}
		}
	}
	
	return nullptr;
}

void FDMXCachedFunctionChannelInfo::GetMatrixCellChannelsAbsoluteNoSorting(UDMXEntityFixturePatch* FixturePatch, const FIntPoint& CellCoordinate, TMap<FName, int32>& OutAttributeToAbsoluteChannelMap) const
{
	UDMXEntityFixtureType* FixtureType = FixturePatch ? FixturePatch->GetFixtureType() : nullptr;
	const FDMXFixtureMode* FixtureModePtr = FixturePatch ? FixturePatch->GetActiveMode() : nullptr;
	if (FixturePatch && FixtureType && FixtureModePtr && FixtureModePtr->bFixtureMatrixEnabled)
	{
		const FDMXFixtureMatrix& FixtureMatrix = FixtureModePtr->FixtureMatrixConfig;

		TMap<const FDMXFixtureCellAttribute*, int32> AttributeToRelativeChannelOffsetMap;
		int32 CellDataSize = 0;
		int32 AttributeChannelOffset = 0;
		for (const FDMXFixtureCellAttribute& CellAttribute : FixtureMatrix.CellAttributes)
		{
			AttributeToRelativeChannelOffsetMap.Add(&CellAttribute, AttributeChannelOffset);
			const int32 AttributeSize = FDMXConversions::GetSizeOfSignalFormat(CellAttribute.DataType);

			CellDataSize += AttributeSize;
			AttributeChannelOffset += FDMXConversions::GetSizeOfSignalFormat(CellAttribute.DataType);
		}

		const int32 FixtureMatrixAbsoluteStartingChannel = FixturePatch->GetStartingChannel() + FixtureMatrix.FirstCellChannel - 1;
		const int32 CellChannelOffset = (CellCoordinate.Y * FixtureMatrix.XCells + CellCoordinate.X) * CellDataSize;
		const int32 AbsoluteCellStartingChannel = FixtureMatrixAbsoluteStartingChannel + CellChannelOffset;

		for (const TTuple<const FDMXFixtureCellAttribute*, int32>& AttributeToRelativeChannelOffsetKvp : AttributeToRelativeChannelOffsetMap)
		{
			const FName FunctionAttributeName = AttributeToRelativeChannelOffsetKvp.Key->Attribute.Name;
			const int32 AbsoluteChannel = AbsoluteCellStartingChannel + AttributeToRelativeChannelOffsetKvp.Value;

			check(AbsoluteChannel > 0 && AbsoluteChannel <= DMX_UNIVERSE_SIZE);
			OutAttributeToAbsoluteChannelMap.Add(FunctionAttributeName, AbsoluteChannel);
		}
	}
}

UMovieSceneDMXLibrarySection::UMovieSceneDMXLibrarySection()
	: bUseNormalizedValues(true)
	, bIsRecording(false)
	, bNeedsSendChannelsToInitialize(true)
{
	BlendType = EMovieSceneBlendType::Absolute;

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		FDMXPortManager::Get().OnPortsChanged.AddUObject(this, &UMovieSceneDMXLibrarySection::RebuildPlaybackCache);
	}
}

void UMovieSceneDMXLibrarySection::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FDMXRuntimeMainStreamObjectVersion::GUID);
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FDMXRuntimeMainStreamObjectVersion::GUID) < FDMXRuntimeMainStreamObjectVersion::DefaultToNormalizedValuesInDMXLibrarySection)
		{
			// For assets created before normalized values were supported, use absolute values
			bUseNormalizedValues = false;
		}

		if (Ar.CustomVer(FDMXRuntimeMainStreamObjectVersion::GUID) < FDMXRuntimeMainStreamObjectVersion::ReplaceWeakWithStrongFixturePatchReferncesInLibrarySection)
		{
			// Add a library reference if possible. In cases where this is serialized before the section's library is loaded, this is not possible programmatically.
			// This is expected to be a rare case (as it wouldn't prevent the sequence from playing before 4.27). For these cases, provide detailed steps to the user how to upgrade in logs.
			for (FDMXFixturePatchChannel& Channel : FixturePatchChannels)
			{
				if (UDMXEntityFixturePatch* FixturePatch = Channel.Reference.GetFixturePatch())
				{
					Channel.DMXLibrary = FixturePatch->GetParentLibrary();
				}
				else
				{
					static const bool bIssueLogged = false;
					if (!bIssueLogged)
					{
						UE_LOGF(LogDMXRuntime, Error, "Found Fixture Patch being used in a dynamically created 4.26 DMX Sequence, e.g. a level sequence player in a blueprint. This caused a issues in 4.26, the sequence would not play.");
						UE_LOGF(LogDMXRuntime, Error, "To restore functionality, please follow these steps:");
						UE_LOGF(LogDMXRuntime, Error, "1. Remove the blueprint that uses the sequence player from the level and save the level.");
						UE_LOGF(LogDMXRuntime, Error, "2. Restart the engine: This message should no longer appear and dmx tracks should show in the sequence. Resave your sequence.");
						UE_LOGF(LogDMXRuntime, Error, "3. Restart the engine again. Now add the blueprint back to the level, save the level.");
						UE_LOGF(LogDMXRuntime, Error, "From hereon the sequence should play fine. These steps will not be required with newly recorded sequences from 4.27 on. Appologies for the inconveniences.");
					}
				}
			}
		}
	}
}

void UMovieSceneDMXLibrarySection::PostLoad()
{
	Super::PostLoad();

	UpdateChannelProxy();
}

void UMovieSceneDMXLibrarySection::PostEditImport()
{
	Super::PostEditImport();

	UpdateChannelProxy();
}

#if WITH_EDITOR
void UMovieSceneDMXLibrarySection::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	UpdateChannelProxy();
}
#endif // WITH_EDITOR

void UMovieSceneDMXLibrarySection::RefreshChannels()
{
	UpdateChannelProxy();
}

void UMovieSceneDMXLibrarySection::AddFixturePatch(UDMXEntityFixturePatch* InPatch)
{
	if (InPatch == nullptr || !InPatch->IsValidLowLevelFast())
	{
		return;
	}

	FDMXFixturePatchChannel NewPatchChannel;
	NewPatchChannel.SetFixturePatch(InPatch);
	FixturePatchChannels.Add(NewPatchChannel);

	UpdateChannelProxy();
}

void UMovieSceneDMXLibrarySection::AddFixturePatches(const TArray<FDMXEntityFixturePatchRef>& InFixturePatchRefs)
{
	for (const FDMXEntityFixturePatchRef& PatchRef : InFixturePatchRefs)
	{
		UDMXEntityFixturePatch* FixturePatch = PatchRef.GetFixturePatch();
		if (IsValid(FixturePatch))
		{
			FDMXFixturePatchChannel NewPatchChannel;
			NewPatchChannel.SetFixturePatch(FixturePatch);
			FixturePatchChannels.Add(NewPatchChannel);
		}
	}

	UpdateChannelProxy();
}

void UMovieSceneDMXLibrarySection::RemoveFixturePatch(UDMXEntityFixturePatch* InPatch)
{
	int32 PatchIndex = FixturePatchChannels.IndexOfByPredicate([InPatch](const FDMXFixturePatchChannel& PatchChannel)
		{
			return PatchChannel.Reference.GetFixturePatch() == InPatch;
		});

	if (PatchIndex != INDEX_NONE)
	{
		FixturePatchChannels.RemoveAt(PatchIndex);

		UpdateChannelProxy();
	}
}

void UMovieSceneDMXLibrarySection::RemoveFixturePatch(const FName& InPatchName)
{
	// Search for the Fixture Patch
	const FString&& TargetPatchName = InPatchName.ToString();

	for (const FDMXFixturePatchChannel& PatchChannel : FixturePatchChannels)
	{
		if (UDMXEntityFixturePatch* Patch = PatchChannel.Reference.GetFixturePatch())
		{
			if (Patch->GetDisplayName().Equals(TargetPatchName))
			{
				RemoveFixturePatch(Patch);
				UpdateChannelProxy();
				break;
			}
		}
	}
}

bool UMovieSceneDMXLibrarySection::ContainsFixturePatch(UDMXEntityFixturePatch* InPatch) const
{
	int32 PatchIndex = FixturePatchChannels.IndexOfByPredicate([InPatch](const FDMXFixturePatchChannel& PatchChannel) {
			return PatchChannel.Reference.GetFixturePatch() == InPatch;
		});

	return PatchIndex != INDEX_NONE;
}

void UMovieSceneDMXLibrarySection::SetFixturePatchActiveMode(UDMXEntityFixturePatch* InPatch, int32 InActiveMode)
{
	if (InPatch == nullptr || !InPatch->IsValidLowLevelFast())
	{
		return;
	}

	// Make sure the Mode Index is valid
	const UDMXEntityFixtureType* FixtureType = InPatch->GetFixtureType();
	if (!IsValid(FixtureType))
	{
		return;
	}
	if (InActiveMode < 0 || InActiveMode >= FixtureType->Modes.Num())
	{
		return;
	}

	// Find the PatchChannel object that represents the passed in Patch
	for (FDMXFixturePatchChannel& PatchChannel : FixturePatchChannels)
	{
		UDMXEntityFixturePatch* Patch = PatchChannel.Reference.GetFixturePatch();
		if (Patch == InPatch)
		{
			PatchChannel.ActiveMode = InActiveMode;
			const bool bResetFunctionChannelsValues = true;
			UpdateChannelProxy(bResetFunctionChannelsValues);

			break;
		}
	}
}

void UMovieSceneDMXLibrarySection::ToggleFixturePatchChannel(UDMXEntityFixturePatch* InPatch, int32 InChannelIndex)
{
	if (IsValid(InPatch))
	{
		// Find the PatchChannel object that represents the passed in Patch
		for (FDMXFixturePatchChannel& PatchChannel : FixturePatchChannels)
		{
			UDMXEntityFixturePatch* Patch = PatchChannel.Reference.GetFixturePatch();
			if (Patch == InPatch)
			{
				PatchChannel.UpdateNumberOfChannels();

				FDMXFixtureFunctionChannel& FunctionChannel = PatchChannel.FunctionChannels[InChannelIndex];
				FunctionChannel.bEnabled = !FunctionChannel.bEnabled;

				UpdateChannelProxy();

				break;
			}
		}
	}
}

void UMovieSceneDMXLibrarySection::ToggleFixturePatchChannel(const FName& InPatchName, const FName& InChannelName)
{
	// Search for the Fixture Patch
	const FString&& TargetPatchName = InPatchName.ToString();

	for (FDMXFixturePatchChannel& PatchChannel : FixturePatchChannels)
	{
		if (UDMXEntityFixturePatch* Patch = PatchChannel.Reference.GetFixturePatch())
		{
			if (Patch->GetDisplayName().Equals(TargetPatchName))
			{
				const UDMXEntityFixtureType* FixtureType = Patch->GetFixtureType();
				if (!IsValid(FixtureType))
				{
					return;
				}

				if (FixtureType->Modes.Num() <= PatchChannel.ActiveMode)
				{
					return;
				}

				// Search for the Function index
				const FString&& TargetFunctionName = InChannelName.ToString();
				int32 FunctionIndex = 0;
				for (const FDMXFixtureFunction& Function : FixtureType->Modes[PatchChannel.ActiveMode].Functions)
				{
					if (Function.FunctionName.Equals(TargetFunctionName))
					{
						// PatchChannel could be out of sync in channels count
						if (PatchChannel.FunctionChannels.Num() <= FunctionIndex)
						{
							PatchChannel.UpdateNumberOfChannels();
						}

						FDMXFixtureFunctionChannel& FunctionChannel = PatchChannel.FunctionChannels[FunctionIndex];
						FunctionChannel.bEnabled = !FunctionChannel.bEnabled;

						UpdateChannelProxy();
						return;
					}
					++FunctionIndex;
				}

				return;
			}
		}
	}
}

bool UMovieSceneDMXLibrarySection::GetFixturePatchChannelEnabled(UDMXEntityFixturePatch* InPatch, int32 InChannelIndex) const
{
	if (IsValid(InPatch))
	{
		// Find the PatchChannel object that represents the passed in Patch
		for (const FDMXFixturePatchChannel& PatchChannel : FixturePatchChannels)
		{
			UDMXEntityFixturePatch* Patch = PatchChannel.Reference.GetFixturePatch();
			if (Patch == InPatch &&
				PatchChannel.FunctionChannels.IsValidIndex(InChannelIndex))
			{
				return PatchChannel.FunctionChannels[InChannelIndex].bEnabled;
			}
		}
	}

	return false;
}

TArray<UDMXEntityFixturePatch*> UMovieSceneDMXLibrarySection::GetFixturePatches() const
{
	TArray<UDMXEntityFixturePatch*> Result;
	Result.Reserve(FixturePatchChannels.Num());

	for (const FDMXFixturePatchChannel& PatchRef : FixturePatchChannels)
	{
		// Add only valid patches
		UDMXEntityFixturePatch* Patch = PatchRef.Reference.GetFixturePatch();
		if (IsValid(Patch))
		{
			Result.Add(Patch);
		}
	}

	return Result;
}

FDMXFixturePatchChannel* UMovieSceneDMXLibrarySection::GetPatchChannel(UDMXEntityFixturePatch* Patch)
{
	return 
		FixturePatchChannels.FindByPredicate([Patch](const FDMXFixturePatchChannel& Channel) {
			return Channel.Reference.GetFixturePatch() == Patch;
		});
}

void UMovieSceneDMXLibrarySection::RebuildPlaybackCache() const
{
	

	CachedOutputPorts.Reset();
	CachedChannelsToEvaluate.Reset();
	CachedChannelsToInitialize.Reset();

	// Cache channel data to streamline performance
	for (int32 IndexPatchChannel = 0; IndexPatchChannel < FixturePatchChannels.Num(); IndexPatchChannel++)
	{
		if (UDMXEntityFixturePatch* FixturePatch = FixturePatchChannels[IndexPatchChannel].Reference.GetFixturePatch())
		{
			if (UDMXLibrary* Library = FixturePatch->GetParentLibrary())
			{
				for (const FDMXOutputPortSharedRef& OutputPort : Library->GetOutputPorts())
				{
					if (!CachedOutputPorts.Contains(OutputPort))
					{
						CachedOutputPorts.Add(OutputPort);
					}
				}

				for (int32 IndexFunctionChannel = 0; IndexFunctionChannel < FixturePatchChannels[IndexPatchChannel].FunctionChannels.Num(); IndexFunctionChannel++)
				{
					FDMXCachedFunctionChannelInfo CachedFunctionChannelInfo = FDMXCachedFunctionChannelInfo(FixturePatchChannels, IndexPatchChannel, IndexFunctionChannel);
					if (CachedFunctionChannelInfo.NeedsEvaluation())
					{
						CachedChannelsToEvaluate.Add(CachedFunctionChannelInfo);
					}
					else if (CachedFunctionChannelInfo.NeedsInitialization())
					{
						CachedChannelsToInitialize.Add(CachedFunctionChannelInfo);
					}
				}
			}
		}
	}

	bNeedsSendChannelsToInitialize = true;
}

void UMovieSceneDMXLibrarySection::EvaluateAndSendDMX(const FFrameTime& FrameTime) const
{
	// Send channels to initialize if required
	if (bNeedsSendChannelsToInitialize)
	{
		SendDMXForChannelsToInitialize();
		bNeedsSendChannelsToInitialize = false;
	}

	UDMXSubsystem* DMXSubsystem = UDMXSubsystem::GetDMXSubsystem_Pure();
	check(DMXSubsystem);

	TMap<int32 /** Universe */, TMap<int32, uint8> /** ChannelToValueMap */> UniverseToChannelToValueMap;

	for (const FDMXCachedFunctionChannelInfo& InfoForChannelToEvaluate : CachedChannelsToEvaluate)
	{
		bool bIsCacheValid = true;
		if (const FDMXFixtureFunctionChannel* FixtureFunctionChannelPtr = InfoForChannelToEvaluate.TryGetFunctionChannel(GetFixturePatchChannels()))
		{
			float ChannelValue = 0.0f;
			if (FixtureFunctionChannelPtr->Channel.Evaluate(FrameTime, ChannelValue))
			{
				TArray<uint8> ByteArr;
				if (bUseNormalizedValues)
				{
					DMXSubsystem->NormalizedValueToBytes(ChannelValue, InfoForChannelToEvaluate.GetSignalFormat(), ByteArr, InfoForChannelToEvaluate.ShouldUseLSBMode());
				}
				else
				{
					// Round to int so if the user draws into the tracks, values are assigned to int accurately
					const int32 RoundedAbsoluteValue = FMath::RoundToInt(ChannelValue);
					DMXSubsystem->IntValueToBytes(RoundedAbsoluteValue, InfoForChannelToEvaluate.GetSignalFormat(), ByteArr, InfoForChannelToEvaluate.ShouldUseLSBMode());
				}

				TMap<int32, uint8>& ChannelToValueMap = UniverseToChannelToValueMap.FindOrAdd(InfoForChannelToEvaluate.GetUniverseID());
				for (int32 ByteIdx = 0; ByteIdx < ByteArr.Num(); ByteIdx++)
				{
					uint8& Value = ChannelToValueMap.FindOrAdd(InfoForChannelToEvaluate.GetStartingChannel() + ByteIdx);
					Value = ByteArr[ByteIdx];
				}
			}
		}
	}

	UE_DMX_SCOPED_TRACE_SENDDMX(GetOutermost()->GetFName());
	for (const TPair<int32, TMap<int32, uint8>>& UniverseToChannelToValueMapKvp : UniverseToChannelToValueMap)
	{
		for (const FDMXOutputPortSharedRef& OutputPort : CachedOutputPorts)
		{
			OutputPort->SendDMX(UniverseToChannelToValueMapKvp.Key, UniverseToChannelToValueMapKvp.Value);
		}
	}
}

void UMovieSceneDMXLibrarySection::SendDMXForChannelsToInitialize() const
{
	UDMXSubsystem* DMXSubsystem = UDMXSubsystem::GetDMXSubsystem_Pure();
	check(DMXSubsystem);

	TMap<int32 /** Universe */, TMap<int32, uint8> /** ChannelToValueMap */> UniverseToChannelToValueMap;

	for (const FDMXCachedFunctionChannelInfo& InfoForChannelToInitialize : CachedChannelsToInitialize)
	{
		if (const FDMXFixtureFunctionChannel* FixtureFunctionChannelPtr = InfoForChannelToInitialize.TryGetFunctionChannel(GetFixturePatchChannels()))
		{
			TArrayView<const FMovieSceneFloatValue> ValueArr = FixtureFunctionChannelPtr->Channel.GetValues();

			if (ensureMsgf(ValueArr.Num() == 1, TEXT("Error in CachedChannelsToInitialize. Contains channel with more or less than one value. Cannot be 'initialized only' alike.")))
			{
				const float ChannelValue = ValueArr[0].Value;

				TArray<uint8> ByteArr;
				if (bUseNormalizedValues)
				{
					DMXSubsystem->NormalizedValueToBytes(ChannelValue, InfoForChannelToInitialize.GetSignalFormat(), ByteArr, InfoForChannelToInitialize.ShouldUseLSBMode());
				}
				else
				{
					// Round to int so if the user draws into the tracks, values are assigned to int accurately
					const uint32 RoundedAbsoluteValue = FMath::RoundToInt(ChannelValue);

					DMXSubsystem->IntValueToBytes(RoundedAbsoluteValue, InfoForChannelToInitialize.GetSignalFormat(), ByteArr, InfoForChannelToInitialize.ShouldUseLSBMode());
				}

				TMap<int32, uint8>& ChannelToValueMap = UniverseToChannelToValueMap.FindOrAdd(InfoForChannelToInitialize.GetUniverseID());

				for (int32 ByteIdx = 0; ByteIdx < ByteArr.Num(); ByteIdx++)
				{
					uint8& Value = ChannelToValueMap.FindOrAdd(InfoForChannelToInitialize.GetStartingChannel() + ByteIdx);
					Value = ByteArr[ByteIdx];
				}
			}
		}
	}

	UE_DMX_SCOPED_TRACE_SENDDMX(GetOutermost()->GetFName());
	for (const TPair<int32, TMap<int32, uint8>>& UniverseToChannelToValueMapKvp : UniverseToChannelToValueMap)
	{
		for (const FDMXOutputPortSharedRef& OutputPort : CachedOutputPorts)
		{
			OutputPort->SendDMX(UniverseToChannelToValueMapKvp.Key, UniverseToChannelToValueMapKvp.Value);
		}
	}
}

void UMovieSceneDMXLibrarySection::UpdateChannelProxy(bool bResetDefaultChannelValues /*= false*/)
{
	// Rebuild caches for the relevant DMX Libraries in this section. 
	// By this the DMXFixture digital twins receives the initial DMX values regardless of loading order during engine startup.
	UDMXSubsystem* Subsystem = UDMXSubsystem::GetDMXSubsystem_Callable();
	TArray<TSoftObjectPtr<UDMXLibrary>> DMXLibraries = Subsystem->GetDMXLibraries();
	TArray<TSoftObjectPtr<UDMXLibrary>> RelevantDMXLibraries;

	Algo::TransformIf(DMXLibraries, RelevantDMXLibraries,
		[this](const TSoftObjectPtr<UDMXLibrary>& DMXLibrary)
		{
			return DMXLibrary.IsValid() && Algo::FindBy(FixturePatchChannels, DMXLibrary, [](const FDMXFixturePatchChannel& Channel) { return Channel.DMXLibrary.Get(); }) != nullptr;
		},
		[](const TSoftObjectPtr<UDMXLibrary>& DMXLibrary)
		{
			return DMXLibrary;
		});

	for (const TSoftObjectPtr<UDMXLibrary>& DMXLibrary : RelevantDMXLibraries)
	{
		DMXLibrary.Get()->ForEachEntityOfType<UDMXEntityFixturePatch>([](UDMXEntityFixturePatch* Patch)
			{
				Patch->RebuildCache();
			});
	}

	FMovieSceneChannelProxyData ChannelProxyData;
	TArray<int32> InvalidPatchChannelIndices;
	
	int32 PatchChannelIndex = 0; // Safer because the ranged for ensures the array length isn't changed
	for (FDMXFixturePatchChannel& PatchChannel : FixturePatchChannels)
	{
		if (!IsValid(PatchChannel.DMXLibrary))
		{
			UE_LOGF(MovieSceneDMXLibrarySectionLog, Warning, "%s: Missing library for sequence section.", __FUNCTION__);
		}

		PatchChannel.UpdateNumberOfChannels(bResetDefaultChannelValues);

		UDMXEntityFixturePatch* Patch = PatchChannel.Reference.GetFixturePatch();
		if (!IsValid(Patch))
		{
			UE_LOGF(MovieSceneDMXLibrarySectionLog, Warning, "%s: Ignoring null Patch. Presumably the library changed and patches were removed or changed. This is not supported.", __FUNCTION__);
			InvalidPatchChannelIndices.Add(PatchChannelIndex);
			continue;
		}

		// If the Patch is null, invalid, doesn't have Modes or the selected mode doesn't have any functions,
		// FunctionChannels will be empty
		if (PatchChannel.FunctionChannels.Num() == 0)
		{
			// With no Function Channels to be displayed, the Patch group won't be displayed.
			// This will give users the impression that the Patch isn't added, but it is,
			// which prevents the user from adding it again. So, to mitigate that, we remove
			// the Patch from the track section.
			UE_LOGF(MovieSceneDMXLibrarySectionLog, Warning, "%s: Ignoring patch without functions %ls", __FUNCTION__, *Patch->GetDisplayName());
			InvalidPatchChannelIndices.Add(PatchChannelIndex);
			continue;
		}

		UDMXEntityFixtureType* FixtureType = Patch->GetFixtureType();
		if (!FixtureType)
		{
			UE_LOGF(MovieSceneDMXLibrarySectionLog, Warning, "%s: Ignoring patch  %ls without valid parent fixture type", __FUNCTION__, *Patch->GetDisplayName());
			continue;
		}

		if (!FixtureType->Modes.IsValidIndex(PatchChannel.ActiveMode))
		{
			UE_LOGF(MovieSceneDMXLibrarySectionLog, Warning, "%s: Recorded active mode no longer valid. Ignoring Patch %ls.", __FUNCTION__, *Patch->GetDisplayName());
			continue;
		}

		const FDMXFixtureMode& Mode = FixtureType->Modes[PatchChannel.ActiveMode];
		const TArray<FDMXFixtureFunction>& Functions = Mode.Functions;
		TArray<FDMXFixtureFunctionChannel>& FunctionChannels = PatchChannel.FunctionChannels;

		// Add a channel proxy entry for each Function channel
		// We use the length of FunctionChannels because it's possible some Fixture Functions
		// are outside the valid range for the Mode's channels or the Universe's channels.
		// And that's already been accounted for when generating the Function channels
		int32 SortOrder = 0;
		for (FDMXFixtureFunctionChannel& FunctionChannel : PatchChannel.FunctionChannels)
		{
			if (!FunctionChannel.bEnabled)
			{
				continue;
			}

#if WITH_EDITOR
			FString AttributeName = FunctionChannel.AttributeName.ToString();

			FString ChannelDisplayNameString;
			if (FunctionChannel.IsCellFunction())
			{
				ChannelDisplayNameString =
					TEXT(" Cell ") +
					FString::FromInt(FunctionChannel.CellCoordinate.X) +
					TEXT("x") +
					FString::FromInt(FunctionChannel.CellCoordinate.Y);

				// Tabulator cosmetics
				if (ChannelDisplayNameString.Len() < 10)
				{
					ChannelDisplayNameString =
						ChannelDisplayNameString +
						TCString<TCHAR>::Tab(4);
				}
				else
				{
					ChannelDisplayNameString =
						ChannelDisplayNameString +
						TCString<TCHAR>::Tab(3);
				}

				ChannelDisplayNameString = ChannelDisplayNameString + AttributeName;
			}
			else
			{
				ChannelDisplayNameString = AttributeName;
			}

			const FText ChannelDisplayName(FText::FromString(ChannelDisplayNameString));
			const FName ChannelPropertyName(*FString::Printf(TEXT("%s.%s"), *Patch->GetDisplayName(), *ChannelDisplayNameString));

			FMovieSceneChannelMetaData MetaData;
			MetaData.SetIdentifiers(ChannelPropertyName, ChannelDisplayName, FText::FromString(Patch->GetDisplayName()));
			MetaData.SortOrder = SortOrder++;
			MetaData.bCanCollapseToTrack = false;

			ChannelProxyData.Add(FunctionChannel.Channel, MetaData, TMovieSceneExternalValue<float>());
#else
			ChannelProxyData.Add(FunctionChannel.Channel);
#endif // WITH_EDITOR
		}

		++PatchChannelIndex;
	}

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(ChannelProxyData));

	RebuildPlaybackCache();
}
