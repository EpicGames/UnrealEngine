// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/MovieSceneDMXLibraryTemplate.h"

#include "DMXRuntimeLog.h"
#include "DMXProtocolTypes.h"
#include "DMXProtocolCommon.h"
#include "DMXStats.h"
#include "Interfaces/IDMXProtocol.h"
#include "IO/DMXOutputPort.h"
#include "Library/DMXLibrary.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXEntityFixtureType.h"

#include "MovieSceneExecutionToken.h"


DECLARE_LOG_CATEGORY_CLASS(MovieSceneDMXLibraryTemplateLog, Log, All);

DECLARE_CYCLE_STAT(TEXT("Sequencer create execution token"), STAT_DMXSequencerCreateExecutionTokens, STATGROUP_DMX);
DECLARE_CYCLE_STAT(TEXT("Sequencer execute execution token"), STAT_DMXSequencerExecuteExecutionToken, STATGROUP_DMX);

struct FDMXLibraryExecutionToken 
	: IMovieSceneExecutionToken
{
	FDMXLibraryExecutionToken(const UMovieSceneDMXLibrarySection* InSection)
		: Section(InSection) 
	{}

	FDMXLibraryExecutionToken(FDMXLibraryExecutionToken&&) = default;
	FDMXLibraryExecutionToken& operator=(FDMXLibraryExecutionToken&&) = default;

	// Non-copyable
	FDMXLibraryExecutionToken(const FDMXLibraryExecutionToken&) = delete;
	FDMXLibraryExecutionToken& operator=(const FDMXLibraryExecutionToken&) = delete;

private:
	const UMovieSceneDMXLibrarySection* Section;

public:
	virtual void Execute(const FMovieSceneContext& Context, const FMovieSceneEvaluationOperand& Operand, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) override
	{
		SCOPE_CYCLE_COUNTER(STAT_DMXSequencerExecuteExecutionToken);

		check(Section);
		Section->EvaluateAndSendDMX(Context.GetTime());
	}
};

FMovieSceneDMXLibraryTemplate::FMovieSceneDMXLibraryTemplate(const UMovieSceneDMXLibrarySection& InSection)
	: Section(&InSection)
{
	check(IsValid(Section));
}

void FMovieSceneDMXLibraryTemplate::Setup(FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) const
{
	Section->RebuildPlaybackCache();
}

void FMovieSceneDMXLibraryTemplate::Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const
{
	SCOPE_CYCLE_COUNTER(STAT_DMXSequencerCreateExecutionTokens);

	// Don't evaluate while recording to prevent conflicts between sent DMX data and incoming recorded data
	if (Section->GetIsRecording())
	{
		return;
	}
	
	FDMXLibraryExecutionToken ExecutionToken(Section);
	ExecutionTokens.Add(MoveTemp(ExecutionToken));
}
