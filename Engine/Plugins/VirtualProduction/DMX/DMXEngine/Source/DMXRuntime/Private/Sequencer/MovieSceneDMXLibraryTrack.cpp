// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/MovieSceneDMXLibraryTrack.h"
#include "Library/DMXLibrary.h"
#include "Sequencer/MovieSceneDMXLibrarySection.h"
#include "Sequencer/MovieSceneDMXLibraryTemplate.h"

#define LOCTEXT_NAMESPACE "MovieSceneDMXLibraryTrack"

DECLARE_LOG_CATEGORY_CLASS(MovieSceneDMXLibraryTrackLog, Log, All);


void UMovieSceneDMXLibraryTrack::SetDMXLibrary(UDMXLibrary* InLibrary)
{
	if (Library == InLibrary)
	{
		return;
	}

	Library = InLibrary;

#if WITH_EDITORONLY_DATA
	SetDisplayName(FText::FromString(Library->GetName()));
#endif
}

UMovieSceneDMXLibraryTrack::UMovieSceneDMXLibraryTrack()
{
	SupportedBlendTypes.Add(EMovieSceneBlendType::Absolute);

#if WITH_EDITORONLY_DATA
	TrackTint = FColor(0, 125, 255, 65);
#endif
}

UMovieSceneSection* UMovieSceneDMXLibraryTrack::CreateNewSection()
{
	UMovieSceneDMXLibrarySection* NewSection = NewObject<UMovieSceneDMXLibrarySection>(this, NAME_None, RF_Transactional);
	return NewSection;
}

void UMovieSceneDMXLibraryTrack::AddSection(UMovieSceneSection& Section)
{
	if (Section.IsA<UMovieSceneDMXLibrarySection>())
	{
		Sections.Add(&Section);
	}
}

void UMovieSceneDMXLibraryTrack::RemoveSection(UMovieSceneSection& Section)
{
	return; // Can't remove the single section
	// Sections.Remove(&Section);
}

void UMovieSceneDMXLibraryTrack::RemoveSectionAt(int32 SectionIndex)
{
	return; // Can't remove the single section
	// Sections.RemoveAt(SectionIndex);
}

const TArray<UMovieSceneSection*>& UMovieSceneDMXLibraryTrack::GetAllSections() const
{
	return Sections;
}

bool UMovieSceneDMXLibraryTrack::IsEmpty() const
{
	return Sections.Num() == 0;
}

bool UMovieSceneDMXLibraryTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.Contains(&Section);
}

void UMovieSceneDMXLibraryTrack::RemoveAllAnimationData()
{
	Sections.Empty();
}

bool UMovieSceneDMXLibraryTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneDMXLibrarySection::StaticClass();
}

bool UMovieSceneDMXLibraryTrack::SupportsMultipleRows() const
{
	return false;
}

FMovieSceneEvalTemplatePtr UMovieSceneDMXLibraryTrack::CreateTemplateForSection(const UMovieSceneSection& InSection) const
{
	return FMovieSceneDMXLibraryTemplate(*CastChecked<UMovieSceneDMXLibrarySection>(&InSection));
}

#undef LOCTEXT_NAMESPACE
