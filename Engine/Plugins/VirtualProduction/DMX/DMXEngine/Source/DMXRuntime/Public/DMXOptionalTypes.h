// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXTypes.h"

#include "DMXOptionalTypes.generated.h"


//////////////////////////////
// Wrappers for TOptional<Types> common to DMX related formats such as GDTF and MVR.

USTRUCT()
struct DMXRUNTIME_API FDMXOptionalBool
{
	GENERATED_BODY()

	/** The Optional Value */
	TOptional<bool> Value;

	/** Resets the optional */
	void Reset() { Value.Reset(); }
	
	/** Returns true if the optional is set */
	bool IsSet() const { return Value.IsSet(); }
	
	/** Returns the Value of the optional, only call when the value is set. */
	const bool& GetValue() const { return Value.GetValue(); }

	const FDMXOptionalBool& operator=(const bool& Transform)
	{
		Value = Transform;
		return *this;
	}

	friend bool operator==(const FDMXOptionalBool& A, const FDMXOptionalBool& B)
	{
		const bool bBothSet = A.Value.IsSet() && B.Value.IsSet();
		if (bBothSet)
		{
			return A.Value.GetValue() == B.Value.GetValue();
		}
		else
		{
			return A.Value.IsSet() == B.Value.IsSet();
		}
	}

	friend bool operator!=(const FDMXOptionalBool& A, const FDMXOptionalBool& B)
	{
		return !(A == B);
	}

	bool Serialize(FArchive& Ar)
	{
		Ar << Value;
		return true;
	}

	friend FArchive& operator<<(FArchive& Ar, FDMXOptionalBool& Struct)
	{
		Ar << Struct.Value;
		return Ar;
	}
};

template<>
struct TStructOpsTypeTraits<FDMXOptionalBool>
	: public TStructOpsTypeTraitsBase2<FDMXOptionalBool>
{
	enum
	{
		WithSerializer = true,
		WithCopy = true,
		WithIdenticalViaEquality = true
	};
};

USTRUCT()
struct DMXRUNTIME_API FDMXOptionalInt32
{
	GENERATED_BODY()

	/** The Optional Value */
	TOptional<int32> Value;

	/** Resets the optional */
	void Reset() { Value.Reset(); }
	
	/** Returns true if the optional is set */
	bool IsSet() const { return Value.IsSet(); }
	
	/** Returns the Value of the optional, only call when the value is set. */
	const int32& GetValue() const { return Value.GetValue(); }

	const FDMXOptionalInt32& operator=(const int32& Transform)
	{
		Value = Transform;
		return *this;
	}

	friend bool operator==(const FDMXOptionalInt32& A, const FDMXOptionalInt32& B)
	{
		const bool bBothSet = A.Value.IsSet() && B.Value.IsSet();
		if (bBothSet)
		{
			return A.Value.GetValue() == B.Value.GetValue();
		}
		else
		{
			return A.Value.IsSet() == B.Value.IsSet();
		}
	}

	friend bool operator!=(const FDMXOptionalInt32& A, const FDMXOptionalInt32& B)
	{
		return !(A == B);
	}

	bool Serialize(FArchive& Ar)
	{
		Ar << Value;
		return true;
	}
	
	FArchive& operator<<(FArchive& Ar)
	{
		Ar << Value;
		return Ar;
	}

	friend FArchive& operator<<(FArchive& Ar, FDMXOptionalInt32& Struct)
	{
		Ar << Struct.Value;
		return Ar;
	}
};

template<>
struct TStructOpsTypeTraits<FDMXOptionalInt32>
	: public TStructOpsTypeTraitsBase2<FDMXOptionalInt32>
{
	enum
	{
		WithSerializer = true,
		WithCopy = true,
		WithIdenticalViaEquality = true
	};
};

USTRUCT()
struct DMXRUNTIME_API FDMXOptionalFloat
{
	GENERATED_BODY()

	/** The Optional Value */
	TOptional<float> Value;

	/** Resets the optional */
	void Reset() { Value.Reset(); }
	
	/** Returns true if the optional is set */
	bool IsSet() const { return Value.IsSet(); }
	
	/** Returns the Value of the optional, only call when the value is set. */
	const float& GetValue() const { return Value.GetValue(); }

	const FDMXOptionalFloat& operator=(const float& Transform)
	{
		Value = Transform;
		return *this;
	}

	friend bool operator==(const FDMXOptionalFloat& A, const FDMXOptionalFloat& B)
	{
		const bool bBothSet = A.Value.IsSet() && B.Value.IsSet();
		if (bBothSet)
		{
			return A.Value.GetValue() == B.Value.GetValue();
		}
		else
		{
			return A.Value.IsSet() == B.Value.IsSet();
		}
	}

	friend bool operator!=(const FDMXOptionalFloat& A, const FDMXOptionalFloat& B)
	{
		return !(A == B);
	}

	bool Serialize(FArchive& Ar)
	{
		Ar << Value;
		return true;
	}

	friend FArchive& operator<<(FArchive& Ar, FDMXOptionalFloat& Struct)
	{
		Ar << Struct.Value;
		return Ar;
	}
};

template<>
struct TStructOpsTypeTraits<FDMXOptionalFloat>
	: public TStructOpsTypeTraitsBase2<FDMXOptionalFloat>
{
	enum
	{
		WithSerializer = true,
		WithCopy = true,
		WithIdenticalViaEquality = true
	};
};

USTRUCT()
struct DMXRUNTIME_API FDMXOptionalGuid
{
	GENERATED_BODY()

	/** The Optional Value */
	TOptional<FGuid> Value;

	/** Resets the optional */
	void Reset() { Value.Reset(); }
	
	/** Returns true if the optional is set */
	bool IsSet() const { return Value.IsSet(); }
	
	/** Returns the Value of the optional, only call when the value is set. */
	const FGuid& GetValue() const { return Value.GetValue(); }

	const FDMXOptionalGuid& operator=(const FGuid& Transform)
	{
		Value = Transform;
		return *this;
	}

	friend bool operator==(const FDMXOptionalGuid& A, const FDMXOptionalGuid& B)
	{
		const bool bBothSet = A.Value.IsSet() && B.Value.IsSet();
		if (bBothSet)
		{
			return A.Value.GetValue() == B.Value.GetValue();
		}
		else
		{
			return A.Value.IsSet() == B.Value.IsSet();
		}
	}

	friend bool operator!=(const FDMXOptionalGuid& A, const FDMXOptionalGuid& B)
	{
		return !(A == B);
	}

	bool Serialize(FArchive& Ar)
	{
		Ar << Value;
		return true;
	}

	FArchive& operator<<(FArchive& Ar)
	{
		Ar << Value;
		return Ar;
	}

	friend FArchive& operator<<(FArchive& Ar, FDMXOptionalGuid& Struct)
	{
		Ar << Struct.Value;
		return Ar;
	}
};

template<>
struct TStructOpsTypeTraits<FDMXOptionalGuid>
	: public TStructOpsTypeTraitsBase2<FDMXOptionalGuid>
{
	enum
	{
		WithSerializer = true,
		WithCopy = true,
		WithIdenticalViaEquality = true
	};
};


USTRUCT()
struct DMXRUNTIME_API FDMXOptionalVector2D
{
	GENERATED_BODY()

	/** The Optional Value */
	TOptional<FVector2D> Value;

	/** Resets the optional */
	void Reset() { Value.Reset(); }
	
	/** Returns true if the optional is set */
	bool IsSet() const { return Value.IsSet(); }
	
	/** Returns the Value of the optional, only call when the value is set. */
	const FVector2D& GetValue() const { return Value.GetValue(); }

	const FDMXOptionalVector2D& operator=(const FVector2D& Vector2D)
	{
		Value = Vector2D;
		return *this;
	}

	friend bool operator==(const FDMXOptionalVector2D& A, const FDMXOptionalVector2D& B)
	{
		const bool bBothSet = A.Value.IsSet() && B.Value.IsSet();
		if (bBothSet)
		{
			return A.Value.GetValue() == B.Value.GetValue();
		}
		else
		{
			return A.Value.IsSet() == B.Value.IsSet();
		}
	}

	friend bool operator!=(const FDMXOptionalVector2D& A, const FDMXOptionalVector2D& B)
	{
		return !(A == B);
	}

	bool Serialize(FArchive& Ar)
	{
		Ar << Value;
		return true;
	}

	FArchive& operator<<(FArchive& Ar)
	{
		Ar << Value;
		return Ar;
	}

	friend FArchive& operator<<(FArchive& Ar, FDMXOptionalVector2D& Struct)
	{
		Ar << Struct.Value;
		return Ar;
	}
};
	
template<>
struct TStructOpsTypeTraits<FDMXOptionalVector2D>
	: public TStructOpsTypeTraitsBase2<FDMXOptionalVector2D>
{
	enum
	{
		WithSerializer = true,
		WithCopy = true,
		WithIdenticalViaEquality = true
	};
};

USTRUCT()
struct DMXRUNTIME_API FDMXOptionalTransform
{
	GENERATED_BODY()

	/** The Optional Value */
	TOptional<FTransform> Value;

	/** Resets the optional */
	void Reset() { Value.Reset(); }
	
	/** Returns true if the optional is set */
	bool IsSet() const { return Value.IsSet(); }
	
	/** Returns the Value of the optional, only call when the value is set. */
	const FTransform& GetValue() const { return Value.GetValue(); }

	const FDMXOptionalTransform& operator=(const FTransform& Transform)
	{
		Value = Transform;
		return *this;
	}

	friend bool operator==(const FDMXOptionalTransform& A, const FDMXOptionalTransform& B)
	{
		const bool bBothSet = A.Value.IsSet() && B.Value.IsSet();
		if (bBothSet)
		{
			return A.Value.GetValue().Equals(B.Value.GetValue(), 0.0);
		}
		else
		{
			return A.Value.IsSet() == B.Value.IsSet();
		}
	}

	friend bool operator!=(const FDMXOptionalTransform& A, const FDMXOptionalTransform& B)
	{
		return !(A == B);
	}

	bool Serialize(FArchive& Ar)
	{
		Ar << Value;
		return true;
	}

	FArchive& operator<<(FArchive& Ar)
	{
		Ar << Value;
		return Ar;
	}

	friend FArchive& operator<<(FArchive& Ar, FDMXOptionalTransform& Struct)
	{
		Ar << Struct.Value;
		return Ar;
	}
};
	
template<>
struct TStructOpsTypeTraits<FDMXOptionalTransform>
	: public TStructOpsTypeTraitsBase2<FDMXOptionalTransform>
{
	enum
	{
		WithSerializer = true,
		WithCopy = true,
		WithIdenticalViaEquality = true
	};
};

USTRUCT()
struct DMXRUNTIME_API FDMXOptionalColorCIE1931
{
	GENERATED_BODY()

	/** The Optional Value */
	TOptional<FDMXGDTFColorCIE1931xyY> Value;

	/** Resets the optional */
	void Reset() { Value.Reset(); }
	
	/** Returns true if the optional is set */
	bool IsSet() const { return Value.IsSet(); }
	
	/** Returns the Value of the optional, only call when the value is set. */
	const FDMXGDTFColorCIE1931xyY& GetValue() const { return Value.GetValue(); }

	const FDMXOptionalColorCIE1931& operator=(const FDMXGDTFColorCIE1931xyY& Transform)
	{
		Value = Transform;
		return *this;
	}

	friend bool operator==(const FDMXOptionalColorCIE1931& A, const FDMXOptionalColorCIE1931& B)
	{
		const bool bBothSet = A.Value.IsSet() && B.Value.IsSet();
		if (bBothSet)
		{
			return A.Value.GetValue() == B.Value.GetValue();
		}
		else
		{
			return A.Value.IsSet() == B.Value.IsSet();
		}
	}

	friend bool operator!=(const FDMXOptionalColorCIE1931& A, const FDMXOptionalColorCIE1931& B)
	{
		return !(A == B);
	}

	bool Serialize(FArchive& Ar)
	{
		Ar << Value;
		return true;
	}

	friend FArchive& operator<<(FArchive& Ar, FDMXOptionalColorCIE1931& Struct)
	{
		Ar << Struct.Value;
		return Ar;
	}
};

template<>
struct TStructOpsTypeTraits<FDMXOptionalColorCIE1931>
	: public TStructOpsTypeTraitsBase2<FDMXOptionalColorCIE1931>
{
	enum
	{
		WithSerializer = true,
		WithCopy = true,
		WithIdenticalViaEquality = true
	};
};

USTRUCT()
struct DMXRUNTIME_API FDMXOptionalString
{
	GENERATED_BODY()

	/** The Optional Value */
	TOptional<FString> Value;

	/** Resets the optional */
	void Reset() { Value.Reset(); }
	
	/** Returns true if the optional is set */
	bool IsSet() const { return Value.IsSet(); }
	
	/** Returns the Value of the optional, only call when the value is set. */
	const FString& GetValue() const { return Value.GetValue(); }

	const FDMXOptionalString& operator=(const FString& Transform)
	{
		Value = Transform;
		return *this;
	}

	friend bool operator==(const FDMXOptionalString& A, const FDMXOptionalString& B)
	{
		const bool bBothSet = A.Value.IsSet() && B.Value.IsSet();
		if (bBothSet)
		{
			return A.Value.GetValue() == B.Value.GetValue();
		}
		else
		{
			return A.Value.IsSet() == B.Value.IsSet();
		}
	}

	friend bool operator!=(const FDMXOptionalString& A, const FDMXOptionalString& B)
	{
		return !(A == B);
	}

	bool Serialize(FArchive& Ar)
	{
		Ar << Value;
		return true;
	}

	friend FArchive& operator<<(FArchive& Ar, FDMXOptionalString& Struct)
	{
		Ar << Struct.Value;
		return Ar;
	}
};

template<>
struct TStructOpsTypeTraits<FDMXOptionalString>
	: public TStructOpsTypeTraitsBase2<FDMXOptionalString>
{
	enum
	{
		WithSerializer = true,
		WithCopy = true,
		WithIdenticalViaEquality = true
	};
};
