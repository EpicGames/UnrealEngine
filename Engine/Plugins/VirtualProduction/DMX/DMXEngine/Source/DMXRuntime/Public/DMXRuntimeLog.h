// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

DECLARE_LOG_CATEGORY_CLASS(LogDMXRuntime, Log, All);

#define UE_LOG_DMXRUNTIME(Verbosity, Format, ...) \
{ \
	UE_LOG(LogDMXRuntime, Verbosity, Format, ##__VA_ARGS__); \
}

#define UE_CLOG_DMXRUNTIME(Conditional, Verbosity, Format, ...) \
{ \
	UE_CLOG(Conditional, LogDMXRuntime, Verbosity, Format, ##__VA_ARGS__); \
}
