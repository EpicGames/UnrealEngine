// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXOptionalTypes.h"
#include "Library/DMXEntityFixtureType.h"

#include "CoreMinimal.h"
#include "JsonObjectConverter.h"

class UDMXEntity;
class UDMXEntityFixturePatch;
class UDMXLibrary;


class DMXRUNTIME_API FDMXRuntimeUtils
{
public:
	/** Parses a GDTF (or MVR) Transformation Matrix from a String. Result is optional and set if parsing succeeded. */
	static FDMXOptionalTransform ParseGDTFMatrix(const FString& String);

	/** Converts a UE transform to a GDTF style string */
	static FString ConvertTransformToGDTF4x3MatrixString(FTransform Transform);

	/**
	 * Generates a unique name given a base one and a list of existing ones, by appending an index.
	 * 
	 * @param InExistingNames					Set of exiting names the new name should not be using.
	 * @param InDesiredName						The desired Name. Note, desired name should not be empty (ensured). Generates 'New' if desired name is empty.
	 * @param NumDigitsInPostfix (optional)		The min num digits to generate when appending a post fix.
	 * @param bAlwaysGeneratePostfix (optional)	If true, always generates digits as postfix.
	 * @return									Returns the unique name.
	 */
	static FString GenerateUniqueNameFromExisting(const TSet<FString>& InExistingNames, FString InDesiredName, uint8 NumDigitsInPostfix = 2, bool bAlwaysGeneratePostfix = false);

	/**
	 * Creates an unique name for an Entity from a specific type, using the type name as base.
	 * @param InLibrary		The DMXLibrary object the entity will belong to.
	 * @param InEntityClass	The class of the Entity, to check the name against others from same type.
	 * @param InBaseName	Optional base name to use instead of the type name.
	 * @return Unique name for an Entity amongst others from the same type.
	 */
	static FString FindUniqueEntityName(const UDMXLibrary* InLibrary, TSubclassOf<UDMXEntity> InEntityClass, const FString& InBaseName = TEXT(""));

	/**
	 * Utility to separate a name from an index at the end.
	 * @param InString	The string to be separated.
	 * @param OutName	The string without an index at the end. White spaces and '_' are also removed.
	 * @param OutIndex	Index that was separated from the name. If there was none, it's zero.
	 *					Check the return value to know if there was an index on InString.
	 * @return True if there was an index on InString.
	 */
	static bool GetNameAndIndexFromString(const FString& InString, FString& OutName, int32& OutIndex);

	/**
	 * Maps each patch to its universe.
	 *
	 * @param AllPatches Patches to map
	 *
	 * @return Key: universe Value: patches in universe with associated key
	 */
	static TMap<int32, TArray<UDMXEntityFixturePatch*>> MapToUniverses(const TArray<UDMXEntityFixturePatch*>& AllPatches);

	/**
	 * Generates a unique name given a base one and a list of potential ones
	 */
	static FString GenerateUniqueNameForImportFunction(TMap<FString, uint32>& OutPotentialFunctionNamesAndCount, const FString& InBaseName);
	
	/**
	 * Sort a distribution array for widget
	 * InDistribution			Distribution schema
	 * InNumXPanels				Num columns
	 * InNumYPanels				Num rows
	 * InUnorderedList			Templated unsorted array input
	 * OutSortedList			Templated unsorted array output
	 */
	template<typename T>
	static void PixelMappingDistributionSort(EDMXPixelMappingDistribution InDistribution, int32 InNumXPanels, int32 InNumYPanels, const TArray<T>& InUnorderedList, TArray<T>& OutSortedList)
	{
		if (InDistribution == EDMXPixelMappingDistribution::TopLeftToRight)
		{	
			// Do nothing it is default, just copy array
			OutSortedList = InUnorderedList;
		}
		else if (InDistribution == EDMXPixelMappingDistribution::TopLeftToBottom)
		{
			for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
			{
				for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
				{
					OutSortedList.Add(InUnorderedList[YIndex + XIndex * InNumYPanels]);
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::TopLeftToClockwise)
		{
			for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
			{
				for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
				{
					if ((YIndex % 2) == 0)
					{
						OutSortedList.Add(InUnorderedList[XIndex + YIndex * InNumXPanels]);
					}
					else
					{
						OutSortedList.Add(InUnorderedList[((YIndex + 1) * InNumXPanels) - (XIndex + 1)]);
					}
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::TopLeftToAntiClockwise)
		{
			for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
			{
				for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
				{
					if ((XIndex % 2) == 0)
					{
						OutSortedList.Add(InUnorderedList[YIndex + (XIndex * InNumYPanels)]);
					}
					else
					{
						OutSortedList.Add(InUnorderedList[((XIndex + 1) * InNumYPanels) - (YIndex + 1)]);
					}
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::TopRightToLeft)
		{
			for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
			{
				for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
				{
					OutSortedList.Add(InUnorderedList[InNumXPanels - 1 - XIndex + YIndex * InNumXPanels]);
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::BottomLeftToTop)
		{
			for (int32 YIndex = InNumYPanels - 1; YIndex >= 0; --YIndex)
			{
				for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
				{
					OutSortedList.Add(InUnorderedList[YIndex + XIndex * InNumYPanels]);
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::TopRightToAntiClockwise)
		{
			for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
			{
				for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
				{
					if ((YIndex % 2) == 0)
					{
						OutSortedList.Add(InUnorderedList[(InNumXPanels - XIndex - 1) + (YIndex * InNumXPanels)]);
					}
					else
					{
						OutSortedList.Add(InUnorderedList[XIndex + YIndex * InNumXPanels]);
					}
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::BottomLeftToClockwise)
		{
			for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
			{
				for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
				{
					if ((XIndex % 2) == 0)
					{
						OutSortedList.Add(InUnorderedList[(InNumYPanels * (XIndex + 1)) - (YIndex + 1)]);
					}
					else
					{
						OutSortedList.Add(InUnorderedList[YIndex + (XIndex * InNumYPanels)]);
					}
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::BottomLeftToRight)
		{
			for (int32 YIndex = InNumYPanels - 1; YIndex >= 0; --YIndex)
			{
				for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
				{
					OutSortedList.Add(InUnorderedList[YIndex * InNumXPanels + XIndex]);
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::TopRightToBottom)
		{
			for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
			{
				for (int32 XIndex = InNumXPanels - 1; XIndex >= 0; --XIndex)
				{
					OutSortedList.Add(InUnorderedList[YIndex + XIndex * InNumYPanels]);
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::BottomLeftAntiClockwise)
		{
			for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
			{
				for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
				{
					if ((InNumYPanels) % 2 == 0)
					{
						if ((YIndex % 2) == 0)
						{
							OutSortedList.Add(InUnorderedList[(InNumXPanels * InNumYPanels) - (YIndex * InNumXPanels) - (1 + XIndex)]);
						}
						else
						{
							OutSortedList.Add(InUnorderedList[(InNumXPanels * InNumYPanels) - ((1 + YIndex) * InNumXPanels) + XIndex]);
						}
					}
					else
					{
						if ((YIndex % 2) == 0)
						{
							OutSortedList.Add(InUnorderedList[(InNumXPanels * InNumYPanels) - ((1 + YIndex) * InNumXPanels) + XIndex]);
						}
						else
						{
							OutSortedList.Add(InUnorderedList[(InNumXPanels * InNumYPanels) - (YIndex * InNumXPanels) - (1 + XIndex)]);
						}
					}
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::TopRightToClockwise)
		{
			for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
			{
				for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
				{
					if ((InNumXPanels) % 2 == 0)
					{
						if ((XIndex % 2) == 0)
						{
							OutSortedList.Add(InUnorderedList[((InNumXPanels - XIndex) * InNumYPanels) - (YIndex + 1)]);
						}
						else
						{
							OutSortedList.Add(InUnorderedList[((InNumXPanels - XIndex) * InNumYPanels) - (InNumYPanels - YIndex)]);
						}
					}
					else
					{
						if ((XIndex % 2) == 0)
						{
							OutSortedList.Add(InUnorderedList[((InNumXPanels - XIndex) * InNumYPanels) - (InNumYPanels - YIndex)]);
						}
						else
						{
							OutSortedList.Add(InUnorderedList[((InNumXPanels - XIndex) * InNumYPanels) - (YIndex + 1)]);
						}
					}
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::BottomRightToLeft)
		{
			for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
			{
				for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
				{
					OutSortedList.Add(InUnorderedList[InNumXPanels * InNumYPanels - 1 - (XIndex + YIndex * InNumXPanels)]);
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::BottomRightToTop)
		{
			for (int32 YIndex = InNumYPanels - 1; YIndex >= 0; --YIndex)
			{
				for (int32 XIndex = InNumXPanels - 1; XIndex >= 0; --XIndex)
				{
					OutSortedList.Add(InUnorderedList[YIndex + XIndex * InNumYPanels]);
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::BottomRightToClockwise)
		{
			for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
			{
				for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
				{
					if ((InNumYPanels) % 2 == 0)
					{
						if ((YIndex % 2) == 0)
						{
							OutSortedList.Add(InUnorderedList[(InNumXPanels * InNumYPanels) - ((1 + YIndex) * InNumXPanels) + XIndex]);
						}
						else
						{
							OutSortedList.Add(InUnorderedList[(InNumXPanels * InNumYPanels) - (YIndex * InNumXPanels) - (1 + XIndex)]);
						}
					}
					else
					{
						if ((YIndex % 2) == 0)
						{
							OutSortedList.Add(InUnorderedList[(InNumXPanels * InNumYPanels) - (YIndex * InNumXPanels) - (1 + XIndex)]);
						}
						else
						{
							OutSortedList.Add(InUnorderedList[(InNumXPanels * InNumYPanels) - ((1 + YIndex) * InNumXPanels) + XIndex]);
						}
					}
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::BottomRightToAntiClockwise)
		{
			for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
			{
				for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
				{
					if ((InNumXPanels) % 2 == 0)
					{
						if ((XIndex % 2) == 0)
						{
							OutSortedList.Add(InUnorderedList[(InNumXPanels * InNumYPanels) - ((XIndex + 1) * InNumYPanels) + YIndex]);
						}
						else
						{
							OutSortedList.Add(InUnorderedList[(InNumYPanels * (InNumXPanels - XIndex)) - (YIndex + 1)]);
						}
					}
					else
					{
						if ((XIndex % 2) == 0)
						{
							OutSortedList.Add(InUnorderedList[(InNumYPanels * (InNumXPanels - XIndex)) - (YIndex + 1)]);
						}
						else
						{
							OutSortedList.Add(InUnorderedList[(InNumXPanels * InNumYPanels) - ((XIndex + 1) * InNumYPanels) + YIndex]);
						}
					}
				}
			}
		}
	}

	/** Serializes a struct to a Sting */
	template <typename StructType>
	static TOptional<FString> SerializeStructToString(const StructType& Struct)
	{
		TSharedRef<FJsonObject> RootJsonObject = MakeShareable(new FJsonObject());

		FJsonObjectConverter::UStructToJsonObject(StructType::StaticStruct(), &Struct, RootJsonObject, 0, 0);

		typedef TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriter;
		typedef TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriterFactory;

		FString CopyStr;
		TSharedRef<FStringWriter> Writer = FStringWriterFactory::Create(&CopyStr);
		FJsonSerializer::Serialize(RootJsonObject, Writer);

		if (!CopyStr.IsEmpty())
		{
			return CopyStr;
		}

		return TOptional<FString>();
	}

	// can't instantiate this class
	FDMXRuntimeUtils() = delete;
};
