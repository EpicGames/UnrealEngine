// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolTypes.h"
#include "DMXTypes.h"
#include "IO/DMXInputPortReference.h"
#include "IO/DMXOutputPortReference.h"
#include "Library/DMXEntityReference.h"
#include "Subsystems/EngineSubsystem.h"

#include "DMXSubsystem.generated.h"

class UDMXEntityFixtureType;
class UDMXEntityFixturePatch;
class UDMXLibrary;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FProtocolReceivedDelegate, FDMXProtocolName, Protocol, int32, RemoteUniverse, const TArray<uint8>&, DMXBuffer);

DECLARE_MULTICAST_DELEGATE_OneParam(FDMXOnDMXLibraryAssetDelegate, UDMXLibrary*);

/**
 * UDMXSubsystem
 * Collections of DMX context blueprint subsystem functions and internal functions for DMX K2Nodes
 */
UCLASS()
class DMXRUNTIME_API UDMXSubsystem 
	: public UEngineSubsystem
{
	GENERATED_BODY()
	
public:
	/** 
	 * Clears all buffered DMX data of Ports and Fixture Patches. 
	 * Note, this function clears the buffers, it does not zero them out.
	 * To reset to default or zero, see Fixture Patch members Send Default Values and Send Zero Values.
	 */
	UFUNCTION(BlueprintCallable, Category = "DMX", meta = (DisplayName = "Clear DMX Buffers"))
	static void ClearDMXBuffers();

	/** Send DMX using function names and integer values. */
	UE_DEPRECATED(4.27, "Use UDMXEntityFixurePatch::SendDMX instead.")
	UFUNCTION(BlueprintCallable, Category = "DMX", meta = (DeprecatedFunction, DeprecationMessage = "Deprecated 4.27. Use DMXEntityFixurePatch::SendDMX instead"))
	void SendDMX(UDMXEntityFixturePatch* FixturePatch, TMap<FDMXAttributeName, int32> AttributeMap, EDMXSendResult& OutResult);

	/** DEPRECATED 4.27 */
	UE_DEPRECATED(4.27, "Use UDMXSubsystem::SendDMXToOutputPort instead.")
	UFUNCTION(BlueprintCallable, Category = "DMX", meta = (DeprecatedFunction, DeprecationMessage = "Deprecated 4.27. Use SendDMXToOutputPort instead."))
	void SendDMXRaw(FDMXProtocolName SelectedProtocol, int32 RemoteUniverse, TMap<int32, uint8> AddressValueMap, EDMXSendResult& OutResult);

	/** Sends DMX via an Output Port. */
	UFUNCTION(BlueprintCallable, Category = "DMX", Meta = (DisplayName = "Send DMX To Output Port"))
	static void SendDMXToOutputPort(FDMXOutputPortReference OutputPortReference, TMap<int32, uint8> ChannelToValueMap, int32 LocalUniverse = 1);

	/** DEPRECATED 4.27 */
	UE_DEPRECATED(4.27, "Use UDMXSubsystem::GetDMXDataFromInputPort or GetDMXDataFromOutputPort instead.")
	UFUNCTION(BlueprintCallable, Category = "DMX", meta = (DeprecatedFunction, DeprecationMessage = "Deprecated 4.27. Use GetDMXDataFromInputPort or GetDMXDataFromOutputPort instead."))
	void GetRawBuffer(FDMXProtocolName SelectedProtocol, int32 RemoteUniverse, TArray<uint8>& DMXBuffer);
	
	/** Gets latest DMX Values from a DMX Universe of a DMX Input Port. If no DMX was received the resulting array will be empty. */
	UFUNCTION(BlueprintCallable, Category = "DMX", Meta = (DisplayName = "Get DMX Data From Input Port"))
	static void GetDMXDataFromInputPort(FDMXInputPortReference InputPortReference, TArray<uint8>& DMXData, int32 LocalUniverse = 1);
	
	/** Gets latest DMX Values from a DMX Universe of a DMX Output Port. If no DMX was received the resulting array will be empty. */
	UFUNCTION(BlueprintCallable, Category = "DMX", Meta = (DisplayName = "Get DMX Data From Output Port"))
	static void GetDMXDataFromOutputPort(FDMXOutputPortReference OutputPortReference, TArray<uint8>& DMXData, int32 LocalUniverse = 1);

	/** Return an array of Fixture Patches that use the provided Fixture Type. */
	UFUNCTION(BlueprintCallable, Category = "DMX", meta = (AutoCreateRefTerm = "FixtureType"))
	void GetAllFixturesOfType(const FDMXEntityFixtureTypeRef& FixtureType, TArray<UDMXEntityFixturePatch*>& OutResult);

	/** Return an array of Fixture Patches that use the provided category. */
	UFUNCTION(BlueprintCallable, Category = "DMX")
	void GetAllFixturesOfCategory(const UDMXLibrary* DMXLibrary, FDMXFixtureCategory Category, TArray<UDMXEntityFixturePatch*>& OutResult);

	/** Return an array of Fixture Patches that reside in the provided universe. */
	UFUNCTION(BlueprintCallable, Category = "DMX")
	void GetAllFixturesInUniverse(const UDMXLibrary* DMXLibrary, int32 UniverseId, TArray<UDMXEntityFixturePatch*>& OutResult);

	/** Return a map with all DMX functions and their associated values provided DMX buffer and desired universe. */
	UE_DEPRECATED(5.5, "Instead please call UDMXEntityFixturePatch::GetAttributeValues to retrieve attribute values safely.")
	UFUNCTION(BlueprintCallable, Category = "DMX", meta = (DeprecatedFunction, DeprecationMessage = "Deprecated 5.5. Instead please call the Fixture Patch function 'Get Attribute Values' to retrieve attribute values safely."))
	void GetFixtureAttributes(const UDMXEntityFixturePatch* InFixturePatch, const TArray<uint8>& DMXBuffer, TMap<FDMXAttributeName, int32>& OutResult);

	/** Return an array of Fixture Patches that have the custom tag set. */
	UFUNCTION(BlueprintCallable, Category = "DMX")
	TArray<UDMXEntityFixturePatch*> GetAllFixturesWithTag(const UDMXLibrary* DMXLibrary, FName CustomTag);

	/** Return an array of Fixture Patches in the provided DMX Library. */
	UFUNCTION(BlueprintCallable, Category = "DMX")
	TArray<UDMXEntityFixturePatch*> GetAllFixturesInLibrary(const UDMXLibrary* DMXLibrary);

	/** Return the Fixture Patch with given name or an invalid object if no Fixture Patch matches the name. */
	UFUNCTION(BlueprintCallable, Category = "DMX", meta = (AutoCreateRefTerm="Name"))
	UDMXEntityFixturePatch* GetFixtureByName(const UDMXLibrary* DMXLibrary, const FString& Name);

	/** Returns all Fixture Types in a DMX Library. */
	UFUNCTION(BlueprintCallable, Category = "DMX")
	TArray<UDMXEntityFixtureType*> GetAllFixtureTypesInLibrary(const UDMXLibrary* DMXLibrary);

	/** Return the Fixture Type with provided name or an invalid object if no Fixture Type matches the name. */
	UFUNCTION(BlueprintCallable, Category = "DMX", meta = (AutoCreateRefTerm = "Name"))
	UDMXEntityFixtureType* GetFixtureTypeByName(const UDMXLibrary* DMXLibrary, const FString& Name);

	/** DEPRECATED 4.27 */
	UE_DEPRECATED(4.27, "Controllers are removed in favor of Ports.")
	UFUNCTION(BlueprintCallable, Category = "DMX", meta = (DeprecatedFunction, DeprecationMessage = "Deprecated 4.27. Controllers are removed in favor of Ports."))
	void GetAllUniversesInController(const UDMXLibrary* DMXLibrary, FString ControllerName, TArray<int32>& OutResult);

	/** DEPRECATED 4.27 */
	UE_DEPRECATED(4.27, "Controllers are removed in favor of Ports.")
	UFUNCTION(BlueprintCallable, Category = "DMX", meta = (DeprecatedFunction, DeprecationMessage = "Deprecated 4.27. Controllers are removed in favor of Ports."))
	TArray<UDMXEntityController*> GetAllControllersInLibrary(const UDMXLibrary* DMXLibrary);

	/** DEPRECATED 4.27 */
	UE_DEPRECATED(4.27, "Controllers are removed in favor of Ports.")
	UFUNCTION(BlueprintCallable, Category = "DMX", meta = (AutoCreateRefTerm = "Name"), meta = (DeprecatedFunction, DeprecationMessage = "Deprecated 4.27. Controllers are removed in favor of Ports."))
	UDMXEntityController* GetControllerByName(const UDMXLibrary* DMXLibrary, const FString& Name);

	/** DEPRECATED 5.5, instead use 'Load All DMX Libraries Synchronous' */
	UE_DEPRECATED(5.5, "Renamed to UDMXSubsystem::LoadAllDMXLibrariesSynchronous. See also UDMXSubsystem::GetDMXLibraries to get soft object ptrs for all DMX Libraries.")
	TArray<UDMXLibrary*> GetAllDMXLibraries();

	/** Loads all DMX Libraries in this project synchronous, returns an array of DMX Libraries */
	UFUNCTION(BlueprintCallable, Category = "DMX", Meta = (DisplayName = "Load DMX Libraries Synchronous"))
	TArray<UDMXLibrary*> LoadDMXLibrariesSynchronous() const;

	/** Gets all DMX Libraries in this project, returns an array of Soft Object References to the DMX Libraries without loading them. */
	UFUNCTION(BlueprintCallable, Category = "DMX", Meta = (DisplayName = "Get DMX Libraries"))
	TArray<TSoftObjectPtr<UDMXLibrary>> GetDMXLibraries() const;

	/**
	 * Converts consecutive DMX channel values to a signed 32bit integer value.
	 * 
	 * @param Bytes				The byte array that is converted to a normalized value. Up to 3 bytes (24 bits) are supported.
	 * @param bUseLSB			When true, the byte array is interpreted in little endian format (least significant byte first) otherwise big endian.
	 * 
	 * @return					The signed 32bit integer value.
	 */
	UFUNCTION(BlueprintPure, Category = "DMX")
	int32 BytesToInt(const TArray<uint8>& Bytes, bool bUseLSB = false) const;

	/**
	 * Converts consecutive DMX channel values to a normalized value.
	 *
	 * @param Bytes				The byte array that is converted to a normalized value. Up to 4 bytes (32 bits) are supported.
	 * @param bUseLSB			When true, the byte array is returned in little endian format (least significant byte first) otherwise big endian.
	 * 
	 * @return					The normalized value.
	 */
	UFUNCTION(BlueprintPure, Category = "DMX", meta = (Keywords = "float"))
	float BytesToNormalizedValue(const TArray<uint8>& Bytes, bool bUseLSB = false) const;

	/**
	 * Converts a normalized value to an array of DMX channel values.
	 *
	 * @param InValue			The normalized floating point value in the range of 0.0 - 1.0. Other values get clamped.
	 * @param InSignalFormat	Specifies the resolution of the resulting byte array, hence the precision of resulting data (e.g. 0-65535 for 16bit).
	 * @param Bytes				The resulting byte array.
	 * @param bUseLSB			When true, the byte array is returned in little endian format (least significant byte first) otherwise big endian.
	 */
	UFUNCTION(BlueprintPure, Category = "DMX", meta = (Keywords = "float"))
	void NormalizedValueToBytes(float InValue, EDMXFixtureSignalFormat InSignalFormat, TArray<uint8>& Bytes, bool bUseLSB = false) const;

	/**
	 * Converts a signed 32bit integer value to an array of DMX channel values.
	 * 
	 * @param InValue			The signed 32bit integer value. The value range depends on the signal format. Excess values get clamped.
	 * @param InSignalFormat	Specifies the resolution of the resulting byte array, hence the precision of resulting data (e.g. 0-65535 for 16bit).
	 * @param Bytes				The resulting byte array.
	 * @param bUseLSB			When true, the byte array is returned in little endian format (least significant byte first) otherwise big endian.
	 */
	UFUNCTION(BlueprintPure, Category = "DMX", Meta = (DisplayName = "Int To Bytes"))
	static void IntValueToBytes(int32 InValue, EDMXFixtureSignalFormat InSignalFormat, TArray<uint8>& Bytes, bool bUseLSB = false);

	/**
	 * Converts a signed 32bit integer value to a normalized value.
	 * 
	 * @param InValue			The signed 32bit integer value. The value range depends on the signal format (e.g. 0-65535 for 16bit). Excess values get clamped.
	 * @param bUseLSB			When true, the byte array is returned in little endian format (least significant byte first) otherwise big endian.
	 *  
	 * @return					The normalized value.
	 */
	UFUNCTION(BlueprintPure, Category = "DMX", meta = (Keywords = "float"))
	float IntToNormalizedValue(int32 InValue, EDMXFixtureSignalFormat InSignalFormat) const;

	/**
	 * Return the normalized value of an Int value from a Fixture Patch function.
	 * @return	The normalized value of the passed in Int using the Function's signal format.
	 *			-1.0 if the Function is not found in the Fixture Patch.
	 */
	UE_DEPRECATED(5.5, "Instead please call the optimized UDMXEntityFixturePatch::GetAttributeValue to retrieve the attribute value.")
	UFUNCTION(BlueprintPure, Category = "DMX", meta = (DeprecatedFunction, DeprecationMessage = "Deprecated 5.5. Instead please call the optimized Fixture Patch function 'Get Attribute Value' to retrieve the attribute value."))
	float GetNormalizedAttributeValue(UDMXEntityFixturePatch* InFixturePatch, FDMXAttributeName InFunctionAttribute, int32 InValue) const;

	/**
	 * Gets the Fixture Type from a Fixture Type Reference.
	 * 
	 * @param	InFixtureType		Can be used to set the Fixture Type of the reference.
	 * @return						The referenced Fixture Type or nullptr if no valid Fixture Type is referenced.
	 */
	UFUNCTION(BlueprintPure, meta = (BlueprintInternalUseOnly = "TRUE"), Category = "DMX")
	UDMXEntityFixtureType* GetFixtureType(FDMXEntityFixtureTypeRef InFixtureType);

	/**
	 * Gets the Fixture Patch from a Fixture Patch Reference.
	 *
	 * @param	InFixturePatch		Can be used to set the referenced Fixture Patch of the reference.
	 * @return						The referenced Fixture Patch or nullptr if no valid Fixture Patch is referenced.
	 */
	UFUNCTION(BlueprintPure, meta = (BlueprintInternalUseOnly = "TRUE"), Category = "DMX")
	UDMXEntityFixturePatch* GetFixturePatch(FDMXEntityFixturePatchRef InFixturePatch);

	/**
	 * Gets a function map based on you active mode from FixturePatch
	 * @param	InFixturePatch Selected Patch
	 * @param	OutAttributesMap Function and Channel value output map
	 * @return	True if outputting was successfully
	 */
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "TRUE", AutoCreateRefTerm = "SelectedProtocol"), Category = "DMX")
	bool GetFunctionsMap(UDMXEntityFixturePatch* InFixturePatch, TMap<FDMXAttributeName, int32>& OutAttributesMap);

	/**
	 * Gets a function map based on you active mode from FixturePatch, but instead of passing a Protocol as parameters, it looks for
	 * the first Protocol found in the Patch's universe and use that one
	 * @param	InFixturePatch Selected Patch
	 * @param	OutAttributesMap Function and Channel value output map
	 * @return	True if outputting was successfully
	 */
	UE_DEPRECATED(5.5, "Duplicate of GetFunctionsMap. Instead please call UDMXSubsystem::GetFunctionsMap or UDMXEntityFixturePatch::GetAttributeValue.")
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "TRUE", AutoCreateRefTerm = "SelectedProtocol", DeprecatedFunction, DeprecationMessage = "Duplicate of GetFunctionsMap. Instead please call DMXSubsystem::GetFunctionsMap or DMXEntityFixturePatch::GetAttributeValues."), Category = "DMX")
	bool GetFunctionsMapForPatch(UDMXEntityFixturePatch* InFixturePatch, TMap<FDMXAttributeName, int32>& OutAttributesMap);

	/**
	 * Gets function channel value by input function name
	 * @param	InName Looking fixture function name
	 * @param	InAttributesMap Function and Channel value input map
	 * @return	Function channel value
	 */
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "TRUE", AutoCreateRefTerm = "InName, InAttributesMap"), Category = "DMX")
	int32 GetFunctionsValue(const FName FunctionAttributeName, const TMap<FDMXAttributeName, int32>& InAttributesMap);

	/**
	 * Returns true if a Fixture Patch is of a given FixtureType
	 * 
	 * @param	InFixturePatch Fixture Patch to check
	 * @param	RefTypeValue a FixtureTypeRef to check against the Fixture Patch.
	 * @return	Returns true if the Fixture Patch matches the Fixture Type.
	 */
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "TRUE"), Category = "DMX")
	bool PatchIsOfSelectedType(UDMXEntityFixturePatch* InFixturePatch, FString RefTypeValue);

	/** Get the DMX Subsystem, pure. */
	UFUNCTION(BlueprintPure, Category = "DMX Subsystem", meta = (BlueprintInternalUseOnly = "true"))
	static UDMXSubsystem* GetDMXSubsystem_Pure();

	/** Get the DMX Subsystem, callable. */
	UFUNCTION(BlueprintCallable, Category = "DMX Subsystem", meta = (BlueprintInternalUseOnly = "true"))
	static UDMXSubsystem* GetDMXSubsystem_Callable();

	/**
	 * Gets the FName of an Attribute Name.
	 * 
	 * @param AttributeName		The Attribute Name struct.
	 * @return					The name of the Attribute
	 */
	UFUNCTION(BlueprintCallable, Category = "DMX")
	FName GetAttributeLabel(FDMXAttributeName AttributeName);

	UE_DEPRECATED(4.26, "Use DMXComponent's OnFixturePatchReceived event or GetRawBuffer instead.")
	UPROPERTY(BlueprintAssignable, Category = "DMX", meta = (DeprecatedProperty, DeprecationMessage = "WARNING: This can execute faster than tick leading to possible blueprint performance issues. Use DMXComponent's OnFixturePatchReceived event or GetRawBuffer instead."))
	FProtocolReceivedDelegate OnProtocolReceived_DEPRECATED;

	/** Set DMX Cell value using matrix coordinates. */
	UE_DEPRECATED(4.27, "Use UDMXEntityFixturePatch::SetMatrixCellValue instead.")
	UFUNCTION(BlueprintCallable, Category = "DMX", meta = (DeprecatedFunction, DeprecationMessage = "Deprecated 4.26. DMXEntityFixurePatch::SendMatrixCellValue instead"))
	bool SetMatrixCellValue(UDMXEntityFixturePatch* FixturePatch, FIntPoint Coordinate /* Cell coordinate X/Y */, FDMXAttributeName Attribute, int32 Value);

	/** Get DMX Cell value using matrix coordinates. */
	UE_DEPRECATED(4.27, "Use UDMXEntityFixturePatch::GetMatrixCellValue instead.")
	UFUNCTION(BlueprintCallable, Category = "DMX", meta = (DeprecatedFunction, DeprecationMessage = "Deprecated 4.26. DMXEntityFixurePatch::GetMatrixCellValues instead"))
	bool GetMatrixCellValue(UDMXEntityFixturePatch* FixturePatch, FIntPoint Coordinate /* Cell coordinate X/Y */, TMap<FDMXAttributeName, int32>& AttributeValueMap);

	/** Gets the starting channel of each cell attribute at given coordinate, relative to the Starting Channel of the patch. */
	UE_DEPRECATED(4.27, "Use UDMXEntityFixturePatch::GetMatrixCellChannelsRelative instead.")
	UFUNCTION(BlueprintCallable, Category = "DMX", meta = (DeprecatedFunction, DeprecationMessage = "Deprecated 4.26. DMXEntityFixurePatch::GetMatrixCellChannelsRelative instead"))
	bool GetMatrixCellChannelsRelative(UDMXEntityFixturePatch* FixturePatch, FIntPoint Coordinate /* Cell coordinate X/Y */, TMap<FDMXAttributeName, int32>& AttributeChannelMap);
	
	/** Gets the absolute starting channel of each cell attribute at given coordinate */
	UE_DEPRECATED(4.27, "Use UDMXEntityFixturePatch::GetMatrixCellChannelsAbsolute instead.")
	UFUNCTION(BlueprintCallable, Category = "DMX", meta = (DeprecatedFunction, DeprecationMessage = "Deprecated 4.26. DMXEntityFixurePatch::GetMatrixCellChannelsAbsolute instead"))
	bool GetMatrixCellChannelsAbsolute(UDMXEntityFixturePatch* FixturePatch, FIntPoint Coordinate /* Cell coordinate X/Y */, TMap<FDMXAttributeName, int32>& AttributeChannelMap);

	/** Get Matrix Fixture properties */
	UE_DEPRECATED(4.27, "Use UDMXEntityFixturePatch::GetMatrixProperties instead.")
	UFUNCTION(BlueprintPure, Category = "DMX", meta = (DeprecatedFunction, DeprecationMessage = "Deprecated 4.26. DMXEntityFixurePatch::GetMatrixProperties instead"))
	bool GetMatrixProperties(UDMXEntityFixturePatch* FixturePatch, FDMXFixtureMatrix& MatrixProperties);

	/** Get all attributes for the Fixture Patch. */
	UE_DEPRECATED(4.27, "Use UDMXEntityFixturePatch::GetCellAttributes instead.")
	UFUNCTION(BlueprintCallable, Category = "DMX", meta = (DeprecatedFunction, DeprecationMessage = "Deprecated 4.26. DMXEntityFixurePatch::GetCellAttributes instead"))
	bool GetCellAttributes(UDMXEntityFixturePatch* FixturePatch, TArray<FDMXAttributeName>& CellAttributes);

	/** Get data for single cell. */
	UE_DEPRECATED(4.27, "Use UDMXEntityFixturePatch::GetMatrixCell instead.")
	UFUNCTION(BlueprintCallable, Category = "DMX", meta = (DeprecatedFunction, DeprecationMessage = "Deprecated 4.26. DMXEntityFixurePatch::GetMatrixCell instead"))
	bool GetMatrixCell(UDMXEntityFixturePatch* FixturePatch, FIntPoint Coordinate /* Cell coordinate X/Y */, FDMXCell& Cell);

	/** Get array of all cells and associated data. */
	UE_DEPRECATED(4.27, "Use UDMXEntityFixturePatch::GetAllMatrixCells instead.")
	UFUNCTION(BlueprintCallable, Category = "DMX", meta = (DeprecatedFunction, DeprecationMessage = "Deprecated 4.26. DMXEntityFixurePatch::GetAllMatrixCells instead"))
	bool GetAllMatrixCells(UDMXEntityFixturePatch* FixturePatch, TArray<FDMXCell>& Cells);

	/** Sort an array according to the selected distribution pattern. */
	UFUNCTION(BlueprintCallable, Category = "DMX")
	void PixelMappingDistributionSort(EDMXPixelMappingDistribution InDistribution, int32 InNumXPanels, int32 InNumYPanels, const TArray<int32>& InUnorderedList, TArray<int32>& OutSortedList);

public:
	/** Delegate broadcast when all DMX Library assets were loaded */
	UE_DEPRECATED(5.5, "DMX Libraries are no longer loaded by default and this delegate is no longer raised. Instead please use UDMXSubsystem::GetDMXLibraries or UDMXSubsystem::LoadDMXLibrariesSynchronous.")
	FSimpleMulticastDelegate OnAllDMXLibraryAssetsLoaded;

#if WITH_EDITOR
	/** Delegate broadcast when a DMX Library asset was added */
	UE_DEPRECATED(5.5, "DMX Libraries are no longer loaded by default and this delegate is no longer raised. Instead please refer to the asset subsystem directly. See IAssetRegistry::OnAssetAdded.")
	FDMXOnDMXLibraryAssetDelegate OnDMXLibraryAssetAdded;

	/** Delegate broadcast when a DMX Library asset was removed */
	UE_DEPRECATED(5.5, "DMX Libraries are no longer loaded by default and this delegate is no longer raised. Instead please refer to the asset subsystem directly. See IAssetRegistry::OnAssetRemoved.")
	FDMXOnDMXLibraryAssetDelegate OnDMXLibraryAssetRemoved;
#endif // WITH_EDITOR
};
