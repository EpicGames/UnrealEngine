// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXAttribute.h"
#include "DMXGDTFColorCIE1931xyY.h"
#include "Templates/SubclassOf.h"

#include "DMXTypes.generated.h"

class UDMXLibrary;

UE_DEPRECATED(5.5, "FDMXColorCIE1931xyY is deprecated. Please use FDMXGDTFColorCIE1931xyY instead.")
typedef FDMXGDTFColorCIE1931xyY FDMXColorCIE1931xyY;

/** Holds an array Attribute Names with their normalized Values (expand the property to see the map) */
USTRUCT(BlueprintType, Category = "DMX", meta = (DisplayName = "DMX Normalized Attribute Value Map"))
struct DMXRUNTIME_API FDMXNormalizedAttributeValueMap
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "DMX")
	TMap<FDMXAttributeName, float> Map;
};

USTRUCT()
struct FDMXByteArray64
{
	GENERATED_BODY()

	TArray64<uint8> ByteArray;

	const FDMXByteArray64& operator=(const FDMXByteArray64& Rhs)
	{
		ByteArray = Rhs.ByteArray;
		return *this;
	}
	const FDMXByteArray64& operator=(const TArray64<uint8>& Rhs)
	{
		ByteArray = Rhs;
		return *this;
	}

	friend bool operator==(const FDMXByteArray64& A, const FDMXByteArray64& B)
	{
		return A.ByteArray == B.ByteArray;
	}

	friend bool operator!=(const FDMXByteArray64& A, const FDMXByteArray64& B)
	{
		return !(A == B);
	}

	bool Serialize(FArchive& Ar)
	{
		Ar << ByteArray;
		return true;
	}

	friend FArchive& operator<<(FArchive& Ar, FDMXByteArray64& Struct)
	{
		Ar << Struct.ByteArray;
		return Ar;
	}
};

template<>
struct TStructOpsTypeTraits<FDMXByteArray64>
	: public TStructOpsTypeTraitsBase2<FDMXByteArray64>
{
	enum
	{
		WithSerializer = true,
		WithCopy = true,
		WithIdenticalViaEquality = true
	};
};

struct UE_DEPRECATED(5.5, "This stuct had no specific use and is now deprecated.") FDMXRequestBase;
USTRUCT(BlueprintType, meta = (Deprecated = "Deprecated 5.5. This stuct had no specific use and is now deprecated."))
struct FDMXRequestBase
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DMX")
	uint8 Value = 0;

};

struct UE_DEPRECATED(5.5, "This stuct had no specific use and is now deprecated.") FDMXRequest;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
USTRUCT(BlueprintType, meta = (Deprecated = "Deprecated 5.5. This stuct had no specific use and is now deprecated."))
struct FDMXRequest : public FDMXRequestBase
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DMX|Request")
	TSubclassOf<UDMXLibrary> DMXLibrary;

};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

struct UE_DEPRECATED(5.5, "This stuct had no specific use and is now deprecated.") FDMXRawArtNetRequest;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
USTRUCT(BlueprintType, meta = (Deprecated = "Deprecated 5.5. This stuct had no specific use and is now deprecated."))
struct FDMXRawArtNetRequest : public FDMXRequestBase
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DMX|RawRequest", meta=(ClampMin=0, ClampMax=137, UIMin = 0, UIMax = 137))
	int32 Net = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DMX|RawRequest", meta = (ClampMin = 0, ClampMax = 15, UIMin = 0, UIMax = 15))
	int32 SubNet = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DMX|RawRequest", meta = (ClampMin = 0, ClampMax = 15, UIMin = 0, UIMax = 15))
	int32 Universe = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DMX|RawRequest", meta = (ClampMin = 1, ClampMax = 512, UIMin = 1, UIMax = 512))
	int32 Address = 1;

};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

struct UE_DEPRECATED(5.5, "This stuct had no specific use and is now deprecated.") FDMXRawSACN;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
USTRUCT(BlueprintType, meta = (Deprecated = "Deprecated 5.5. This stuct had no specific use and is now deprecated."))
struct FDMXRawSACN : public FDMXRequestBase
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DMX|RawRequest", meta = (ClampMin = 0, ClampMax = 63999, UIMin = 0, UIMax = 63999))
	int32 Universe = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DMX|RawRequest", meta = (ClampMin = 0, ClampMax = 512, UIMin = 0, UIMax = 512))
	int32 Address = 0;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
