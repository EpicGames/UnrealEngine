// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"

#include "DMXTypes.h"
#include "Library/DMXEntityReference.h"

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"

#include "DMXComponent.generated.h"

struct FDMXAttributeName;
class UDMXLibrary;
class UDMXEntityFixturePatch;

/** 
 * Component that receives DMX input each Tick from a fixture patch.  
 */
UCLASS( ClassGroup=(DMX), meta=(BlueprintSpawnableComponent), HideCategories = ("Variable", "Sockets", "Activation", "Cooking", "ComponentReplication", "Collision", "ComponentTick"))
class DMXRUNTIME_API UDMXComponent
	: public UActorComponent
{
	GENERATED_BODY()

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FDMXComponentFixturePatchReceivedSignature, UDMXEntityFixturePatch*, FixturePatch, const FDMXNormalizedAttributeValueMap&, ValuePerAttribute);
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FDMXOnDMXComponentTickSignature, float, DeltaTime);

public:
	UDMXComponent();

	/** Broadcast when the component's fixture patch received DMX */
	UPROPERTY(BlueprintAssignable, Category = "DMX");
	FDMXComponentFixturePatchReceivedSignature OnFixturePatchReceived;

	/**
	 * Event raised each tick, when the component has a Fixture Patch set and either:
	 * - The assigned fixture patch is set to 'receives DMX in editor'.
	 * - Project Settings -> Plugins -> DMX -> 'Fixture Patches receive DMX in Editor' is set to true.
	 */
	UPROPERTY(BlueprintAssignable, Category = "DMX", Meta = (DisplayName = "On DMX Component Tick"))
	FDMXOnDMXComponentTickSignature OnDMXComponentTick;

	/** Gets the fixture patch used in the component */
	UFUNCTION(BlueprintPure, Category = "DMX")
	UDMXEntityFixturePatch* GetFixturePatch() const;

	/** Sets the fixture patch used in the component */
	UFUNCTION(BlueprintCallable, Category = "DMX")
	void SetFixturePatch(UDMXEntityFixturePatch* InFixturePatch);

	/** Sets whether the component receives dmx from the patch. Note, this is saved with the component when called in editor. */
	UFUNCTION(BlueprintCallable, Category = "DMX", meta = (DisplayName = "Set Receive DMX From Patch"))
	void SetReceiveDMXFromPatch(bool bReceive);

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "DMX", Meta = (ShowOnlyInnerProperties))
	FDMXEntityFixturePatchRef FixturePatchRef;

protected:
	/** Called when the fixture patch received DMX */
	UFUNCTION()
	void OnFixturePatchReceivedDMX(UDMXEntityFixturePatch* FixturePatch, const FDMXNormalizedAttributeValueMap& NormalizedValuePerAttribute);

	/** Sets up binding for receiving depending on the patch's and the component's properties */
	void SetupReceiveDMXBinding();

	/** Enables or disables the component tick depending of it being required for the OnDXMComponentTick function */
	void UpdateTickEnabled();

	/** If true, the component will receive DMX from the patch */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, AdvancedDisplay, Category = "DMX", Meta = (DisplayPriority = 0))
	bool bReceiveDMXFromPatch;

	// ~Begin UActorComponent interface
	virtual void OnRegister() override;
	virtual void BeginPlay() override;
	virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	virtual void DestroyComponent(bool bPromoteChildren) override;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	// ~End UActorComponent interface

private:
	/** Called when receive DMX in Editor was enabled or disabled for DMX Components (via DMXProtocolSettings) */
	virtual void OnAllFixturePatchesReceiveDMXInEditorEnabled(bool bEnabled);

	/** Called when the fixture patch changed. Can be editor only as we only want to update the component tick */
	void OnFixturePatchPropertiesChanged(const UDMXEntityFixturePatch* FixturePatch);
#endif // WITH_EDITOR
};
