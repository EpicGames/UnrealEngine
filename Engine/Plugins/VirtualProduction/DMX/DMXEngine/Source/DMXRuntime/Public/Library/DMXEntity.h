// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolTypes.h"
#include "Library/DMXObjectBase.h"

#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"

#include "DMXEntity.generated.h"

class UDMXLibrary;

/**  Base class for all entity types */
UCLASS(abstract, meta = (DisplayName = "DMX Entity"))
class DMXRUNTIME_API UDMXEntity
	: public UDMXObjectBase
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Entity Properties", meta = (DisplayName = "Name", DisplayPriority = "1"))
	FString Name;

public:
	/** Constructor, for default objects only */
	UDMXEntity();

	/** Constructor that specifies the outer Library */
	UDMXEntity(UDMXLibrary* InParentLibrary);

	//~ Begin UObject interface
	virtual void PostInitProperties() override;
	//~ End UObject interface

	/** Destroys the Entity */
	void Destroy();

	/**  Returns the entity name to be used in UI elements */
	FString GetDisplayName() const;

	/**  Updates this Entity's name and the UI friendly display name */
	void SetName(const FString& InNewName);

	/**
	 * Checks for Entity correctness for usability with protocols.
	 * @param OutReason	Reason for being invalid, if that's the case.
	 * @return True if the Entity can be used with protocols.
	 */
	virtual bool IsValidEntity(FText& OutReason) const { return true; }

	/**
	 * Checks for Entity correctness for usability with protocols.
	 * @return True if the Entity can be used with protocols.
	 */
	virtual bool IsValidEntity() const
	{
		FText OutReason = FText::GetEmpty();
		return IsValidEntity(OutReason);
	}

	/** Sets the DMX Library of the entity */
	void SetParentLibrary(UDMXLibrary* InParent);

	/** Gets the library the Entity resides in */
	UDMXLibrary* GetParentLibrary() const { return ParentLibrary.Get(); }

	/** This Entity's unique ID */
	const FGuid& GetID() const { return Id; }

	/** Used by DMX Library to resolve ID conflicts among entities */
	void RefreshID();

	/** Copy another Entity's ID. Used when copying, to not lose the original Entity's reference  */
	void ReplicateID(UDMXEntity* Other);

protected:
	UPROPERTY()
	TWeakObjectPtr<UDMXLibrary> ParentLibrary;
	
	/** Uniquely identifies the parameter, used for fixing up Blueprints that reference this Entity when renaming. */
	UPROPERTY(DuplicateTransient)
	FGuid Id;
};
