// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolTypes.h"
#include "Library/DMXEntity.h"

#include "DMXEntityController.generated.h"

struct FDMXBuffer;


class UE_DEPRECATED(4.27, "DMXEntityUniverseManaged and DMXEntityController are deprecated in favor of the DMX Port System. Please refer to FDMXPortManager instead.") UDMXEntityUniverseManaged;
UCLASS()
class DMXRUNTIME_API UDMXEntityUniverseManaged
	: public UDMXEntity
{
	GENERATED_BODY()
public:

	UE_DEPRECATED(4.27, "UDMXEntityUniverseManaged is deprecated. Use Ports instead.")
	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Controllers are no longer in use. Use Ports instead."))
	FDMXProtocolName DeviceProtocol;
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS // Leave it to the base class (UDMXEntityUniverseManaged) to raise deprecation warnings
UCLASS()
class DMXRUNTIME_API UDMXEntityController
	: public UDMXEntityUniverseManaged
{
	GENERATED_BODY()

PRAGMA_ENABLE_DEPRECATION_WARNINGS

public:
	UE_DEPRECATED(4.27, "UDMXEntityController is deprecated. Use Ports instead.")
	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Controllers are no longer in use. Use Ports instead."))
	EDMXCommunicationType CommunicationMode;
	
	UE_DEPRECATED(4.27, "UDMXEntityController is deprecated. Use Ports instead.")
	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Controllers are no longer in use. Use Ports instead."))
	int32 UniverseLocalStart;

	UE_DEPRECATED(4.27, "UDMXEntityController is deprecated. Use Ports instead.")
	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Controllers are no longer in use. Use Ports instead."))
	int32 UniverseLocalNum;

	UE_DEPRECATED(4.27, "UDMXEntityController is deprecated. Use Ports instead.")
	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Controllers are no longer in use. Use Ports instead."))
	int32 UniverseLocalEnd;

	/**
	 * Offsets the Universe IDs range on this Controller before communication with other devices.
	 * Useful to solve conflicts with Universe IDs from other devices on the same network.
	 *
	 * All other DMX Library settings use the normal Universe IDs range.
	 * This allows the user to change all Universe IDs used by the Fixture Patches and
	 * avoid conflicts with other devices by updating only the Controller's Remote Offset.
	 */
	UE_DEPRECATED(4.27, "UDMXEntityController is deprecated. Use Ports instead.")
	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Controllers are no longer in use. Use Ports instead."))
	int32 RemoteOffset;

	/**
	 * First Universe ID on this Controller's range that is sent over the network.
	 * Universe Start + Remote Offset
	 */
	UE_DEPRECATED(4.27, "UDMXEntityController is deprecated. Use Ports instead.")
	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Controllers are no longer in use. Use Ports instead."))
	int32 UniverseRemoteStart;

	/**
	 * Last Universe ID in this Controller's range that is sent over the network.
	 * Universe End + Remote Offset
	 */
	UE_DEPRECATED(4.27, "UDMXEntityController is deprecated. Use Ports instead.")
	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Controllers are no longer in use. Use Ports instead."))
	int32 UniverseRemoteEnd;

	UE_DEPRECATED(4.27, "UDMXEntityController is deprecated. Use Ports instead.")
	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Controllers are no longer in use. Use Ports instead."))
	TArray<FString> AdditionalUnicastIPs;
};
