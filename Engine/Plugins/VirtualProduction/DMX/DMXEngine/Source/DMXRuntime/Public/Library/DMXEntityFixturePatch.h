// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXAttribute.h"
#include "DMXProtocolCommon.h"
#include "DMXTypes.h"
#include "Library/DMXEntity.h"
#include "Library/DMXEntityFixturePatchCache.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/DMXEntityReference.h"
#include "MVR/DMXMVRGeneralSceneDescription.h"
#include "Tickable.h"

#include "DMXEntityFixturePatch.generated.h"

class FDMXEntityFixturePatchCache;
class UDMXEntityController;
class UDMXEntityFixtureType;
class UDMXModulator;

struct FPropertyChangedEvent;


DECLARE_MULTICAST_DELEGATE_OneParam(FDMXOnFixturePatchChangedDelegate, const UDMXEntityFixturePatch* /** ChangedFixturePatch */);

/** Parameters to construct a Fixture Patch. */
USTRUCT(BlueprintType, meta = (DisplayName = "DMX Entity Fixture Patch Construction Params"))
struct DMXRUNTIME_API FDMXEntityFixturePatchConstructionParams
{
	GENERATED_BODY()
	
	/** The fixture type of the newly constructed fixture patch */
	UPROPERTY(VisibleDefaultsOnly, BlueprintReadWrite, Category = "Fixture Patch", meta = (DisplayName = "Fixture Type"))
	FDMXEntityFixtureTypeRef FixtureTypeRef;

	/** The index of the mode in the fixture type the fixture patch uses */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Fixture Patch")
	int32 ActiveMode = 0;

	/** The local universe of the fixture patch */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Fixture Patch", meta = (ClampMin = 0, DisplayName = "Universe"))
	int32 UniverseID = 1;

	/** Starting channel for when auto-assign address is false */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Fixture Patch", meta = (DisplayName = "Starting Channel", UIMin = "1", UIMax = "512", ClampMin = "1", ClampMax = "512"))
	int32 StartingAddress = 1;

#if WITH_EDITOR
	/**
	 * The transform used when the DMX Library is spawned in a level.
	 *
	 * When the DMX Library is exported as an MVR file, this transform is used unless the 'Use Transforms from Level' export option is checked.
	 */
	FTransform DefaultTransform = FTransform::Identity;
#endif 

	/** 
	 * When spawning the DMX Library as MVR Scene in Editor, each Fixture Patch has to correspond to a Fixture in the World (if it is desired to export the Scene as MVR later).
	 * Mostly useful when importing an MVR into the DMX Library (see DMXLibraryFromMVRFactory). If left '00000000-00000000-00000000-00000000', a Unique ID will be generated for the patch. 
	 * Ensures the Unique ID is not used by another patch in the DMX Library already.
	 */
	FGuid MVRFixtureUUID;
};


/** 
 * A DMX fixture patch that can be patch to channels in a DMX Universe via the DMX Library Editor. 
 * 
 * Use in DMXComponent or call SetReceiveDMXEnabled with 'true' to enable receiving DMX. 
 */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "DMX Fixture Patch"))
class DMXRUNTIME_API UDMXEntityFixturePatch
	: public UDMXEntity
	, public FTickableGameObject
{
	GENERATED_BODY()

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FDMXOnFixturePatchReceivedDMXDelegate, UDMXEntityFixturePatch*, FixturePatch, const FDMXNormalizedAttributeValueMap&, ValuePerAttribute);

	DECLARE_MULTICAST_DELEGATE_TwoParams(FDMXOnFixturePatchReceivedDMXDelegateNative, UDMXEntityFixturePatch* /** FixturePatch */, const FDMXNormalizedAttributeValueMap& /** ValuePerAttribute */)

public:
	UDMXEntityFixturePatch();

	/** Creates a new Fixture Patch in the DMX Library using the specified Fixture Type */
	UFUNCTION(BlueprintCallable, Category = "DMX")
	static UDMXEntityFixturePatch* CreateFixturePatchInLibrary(FDMXEntityFixturePatchConstructionParams ConstructionParams, const FString& DesiredName = TEXT(""), bool bMarkDMXLibraryDirty = true);

	/** Removes a fixture patch from the DMX Library */
	UFUNCTION(BlueprintCallable, Category = "DMX")
	static void RemoveFixturePatchFromLibrary(FDMXEntityFixturePatchRef FixturePatchRef);

	// ~Begin UObject Interface
#if WITH_EDITOR
	virtual bool Modify(bool bAlwaysMarkDirty = true) override;
#endif
protected:
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostInitProperties() override;
	virtual void PostLoad() override;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent) override;
#endif // WITH_EDITOR
	// ~End UObject Interface

	// ~Begin FTickableGameObject interface
	virtual void Tick(float DeltaTime) override;
	virtual bool IsTickable() const override;
	virtual bool IsTickableInEditor() const override;
	virtual ETickableTickType GetTickableTickType() const override;
	virtual TStatId GetStatId() const override;
	// ~End FTickableGameObject interface

public:
	/** Returns a delegate that is and should be broadcast whenever a Fixture Type changed */
	static FDMXOnFixturePatchChangedDelegate& GetOnFixturePatchChanged();

	/** Send DMX using attribute names and integer values. */
	UFUNCTION(BlueprintCallable, Category = "DMX")
	void SendDMX(TMap<FDMXAttributeName, int32> AttributeMap);

	/** 
	 * Sends the default value for all attributes, including matrix attributes. 
	 * Note, calls will not be considered by the DMX Conflict Monitor.
	 */
	UFUNCTION(BlueprintCallable, Category = "DMX")
	void SendDefaultValues();

	/** 
	 * Sends zeroes for all attributes, including matrix attributes.
	 * Note, calls will not be considered by the DMX Conflict Monitor.
	 */
	UFUNCTION(BlueprintCallable, Category = "DMX")
	void SendZeroValues();

	/** 
	 * Rebuilds the cache. Should be called when relevant properties, for example the starting channel changed. 
	 * This will not clear cached DMX data.
	 */
	void RebuildCache();

	/** Returns the last received DMX signal. */
	const FDMXSignalSharedPtr& GetLastReceivedDMXSignal() const { return LastDMXSignal; }

	/** 
	 * Delegate broadcast when the fixture patch received DMX. 
	 * This event only fires when any attribute value changed. 
	 * Use GetAttributeValues or GetNormalizedAttributeValues to get unchanged values.
	 * 
	 * Native version, should be preferred when binding UObjects in C++ to avoid serializing the reference. 
	 */
	FDMXOnFixturePatchReceivedDMXDelegateNative OnFixturePatchReceivedDMXNative;

	/**
	 * Returns a delegate broadcast when the fixture patch received DMX.
	 * This event only fires when any attribute value changed.
	 * Use GetAttributeValues or GetNormalizedAttributeValues to get unchanged values.
	 */
	UPROPERTY(BlueprintAssignable, Category = "DMX");
	FDMXOnFixturePatchReceivedDMXDelegate OnFixturePatchReceivedDMX;

private:
	/** Updates the cache. Returns true if the values got updated (if the values changed) */
	bool UpdateCache();

	/** The last received DMX signal */
	FDMXSignalSharedPtr LastDMXSignal;

	/** Cache of dmx values */
	FDMXEntityFixturePatchCache Cache;

public:
	//~ Begin UDMXEntity Interface
	virtual bool IsValidEntity(FText& OutReason) const override;
	//~ End UDMXEntity Interface

	/** Called from Fixture Type to keep ActiveMode in valid range when Modes are removed from the Type */
	void ValidateActiveMode();

	/** Returns the active mode, or nullptr if there is no valid active mode */
	const FDMXFixtureMode* GetActiveMode() const;
	
	/** Gets the parent fixture type this was constructed from */
	UDMXEntityFixtureType* GetFixtureType() const { return ParentFixtureTypeTemplate; }

	/** Sets the fixture type this is using */
	UFUNCTION(BlueprintCallable, Category = "DMX|Fixture Patch")
	void SetFixtureType(UDMXEntityFixtureType* NewFixtureType);

	/** Returns the universe ID of the patch */
	int32 GetUniverseID() const { return UniverseID; }

	/** Sets the Universe ID of the patch */
	UFUNCTION(BlueprintCallable, Category = "DMX|Fixture Patch")
	void SetUniverseID(int32 NewUniverseID);

	/** Sets the starting channel of the Fixture Patch. */
	UFUNCTION(BlueprintCallable, Category = "DMX|Fixture Patch")
	void SetStartingChannel(int32 NewStartingChannel);

	/** Return the starting channel */
	UFUNCTION(BlueprintCallable, Category = "DMX|Fixture Patch")
	int32 GetStartingChannel() const { return StartingChannel; }

	/** Returns the number of channels this Patch occupies with the Fixture functions from its Active Mode or 0 if the patch has no valid Active Mode */
	UFUNCTION(BlueprintPure, Category = "DMX|Fixture Patch")
	int32 GetChannelSpan() const;

	/** Returns the last channel of the patch */
	UFUNCTION(BlueprintPure, Category = "DMX|Fixture Patch")
	int32 GetEndingChannel() const;

	/** Sets the index of the Mode the Patch uses from its Fixture Type */
	bool SetActiveModeIndex(int32 NewActiveModeIndex);

	/** Returns the index of the Mode the Patch uses from its Fixture Type */
	int32 GetActiveModeIndex() const { return ActiveMode; }

	/** Returns custom tags defined for the patch */
	const TArray<FName>& GetCustomTags() const { return CustomTags; }

	/** Returns the MVR Fixture UUIDs of this patch */
	const FGuid& GetMVRFixtureUUID() const { return MVRFixtureUUID; }

	/** Returns the MVR Fixture ÎD of this patch */
	int32 GetFixtureID() const { return FixtureID; }

	/** 
	 * Generates a unique Fixture ID for this patch. 
	 * If DesiredFixtureID is > 0, tries to use this fixture ID, generates a unique one if the desired Fixture ID was already in use.
	 */
	void GenerateFixtureID(int32 DesiredFixtureID = -1);

	/** 
	 * Tries to find the fixture ID of the patch. Looks up the general scene description resulting in a relatively slow operation. 
	 * Returns false if the patch has no fixture ID could be found, typically the case when the patch is no valid MVR Fixture.
	 */
	UE_DEPRECATED(5.5, "The patches now hold their Fixture ID. Use UDMXEntityFixturePatch::GetFixtureID.")
	bool FindFixtureID(int32& OutFixtureID) const;

#if WITH_EDITOR
	/**
	 * Sets The transform used when the DMX Library is spawned in a level.
	 *
	 * When the DMX Library is exported as an MVR file, this transform is used unless the 'Use Transforms from Level' export option is checked.
	 */
	void SetDefaultTransform(const FTransform& NewDefaultTransform) { DefaultTransform = NewDefaultTransform; }

	/**
	 * Returns The transform used when the DMX Library is spawned in a level.
	 *
	 * When the DMX Library is exported as an MVR file, this transform is used unless the 'Use Transforms from Level' export option is checked.
	 */
	const FTransform& GetDefaultTransform() const { return DefaultTransform; }

	/** Property name getters. When accessing the this way, use with care. The patch will need to update its cache after using property setters, use Pre/PostEditChanged events to achieve that. */
	static FName GetUniverseIDPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(UDMXEntityFixturePatch, UniverseID); }
	static FName GetParentFixtureTypeTemplatePropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(UDMXEntityFixturePatch, ParentFixtureTypeTemplate); }
	static FName GetActiveModePropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(UDMXEntityFixturePatch, ActiveMode); }
	static FName GetDefaultTransformPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(UDMXEntityFixturePatch, DefaultTransform); }
	static FName GetMVRFixtureUUIDPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(UDMXEntityFixturePatch, MVRFixtureUUID); }
	static FName GetFixtureIDPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(UDMXEntityFixturePatch, FixtureID); }
	static FName GetStartingChannelPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(UDMXEntityFixturePatch, StartingChannel); }
#endif // WITH_EDITOR

protected:
	/** The local universe of the patch */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Fixture Patch", meta = (ClampMin = 0))
	int32 UniverseID;

#if WITH_EDITORONLY_DATA
	/** DEPRECATED 5.1 */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "bAutoAssignAddress and related members are deprecated. Auto assign is now only a method in FDMXEditorUtils and should be applied on demand."))
	bool bAutoAssignAddress_DEPRECATED = true;

	/** DEPRECATED 5.1 */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "bAutoAssignAddress and related members are deprecated. Auto assign is now only a method in FDMXEditorUtils and should be applied on demand."))
	int32 ManualStartingAddress_DEPRECATED = 1;

	/** DEPRECATED 5.1 */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "bAutoAssignAddress and related members are deprecated. Auto assign is now only a method in FDMXEditorUtils and should be applied on demand."))
	int32 AutoStartingAddress_DEPRECATED = 1;
#endif // WITH_EDITORONLY_DATA

	/** Starting Channel of the Patch */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Fixture Patch", meta = (UIMin = "1", UIMax = "512", ClampMin = "1", ClampMax = "512"))
	int32 StartingChannel = 0;

	/** The Fixture Type that defines the DMX channel layout of this Fixture Patch */
	UPROPERTY(VisibleDefaultsOnly, BlueprintReadWrite, BlueprintSetter = SetFixtureType, Category = "Fixture Patch", meta = (DisplayName = "Fixture Type"))
	TObjectPtr<UDMXEntityFixtureType> ParentFixtureTypeTemplate;

	/** The Index of the Mode in the Fixture Type the Patch uses */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, meta = (DisplayName = "Active Mode Index"), Category = "Fixture Patch")
	int32 ActiveMode = INDEX_NONE;

#if WITH_EDITORONLY_DATA
	/** 
	 * The transform used when the DMX Library is spawned in a level.
	 * 
	 * When the DMX Library is exported as an MVR file, this transform is used unless the 'Use Transforms from Level' export option is checked.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MVR")
	FTransform DefaultTransform = FTransform::Identity;
#endif // WITH_EDITORONLY_DATA

	/** The Fixture ID of this patch */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "MVR")
	int32 FixtureID = 0;
	
	/** The MVR Fixture UUID */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, meta = (DisplayName = "MVR Fixture UUID"), Category = "MVR")
	FGuid MVRFixtureUUID;

	/** Delegate broadcast when a Fixture Patch changed */
	static FDMXOnFixturePatchChangedDelegate OnFixturePatchChangedDelegate;

public:
	/** Custom tags for filtering patches  */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Fixture Patch")
	TArray<FName> CustomTags;

#if WITH_EDITORONLY_DATA
	/** Color when displayed in the fixture patch editor */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Fixture Patch")
	FLinearColor EditorColor;

	/** 
	 * If true, the patch receives dmx and raises the OnFixturePatchReceivedDMX events in editor. 
	 * NOTE: If 'All Fixture Patches receive DMX in editor' is set to true in Project Settings -> Plugins -> DMX, this setting here is ignored.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Fixture Patch")
	bool bReceiveDMXInEditor;
#endif

public:
#if WITH_EDITOR
	/** DEPRECATED 4.27 */
	UE_DEPRECATED(4.27, "Controllers are replaced with DMX Ports.")
	UFUNCTION(BlueprintPure, Category = "DMX|Fixture Patch", meta = (DeprecatedFunction, DeprecationMessage = "Deprecated 4.27. No clear remote universe can be deduced from controllers (before 4.27) or ports (from 4.27 on)."))
	int32 GetRemoteUniverse() const;
#endif // WITH_EDITOR

	/**
	 * Returns an array of attributes for the currently active mode.
	 * Attributes outside the Active Mode's channel span range are ignored.
	 */
	UFUNCTION(BlueprintPure, Category = "DMX|Fixture Patch")
	TArray<FDMXAttributeName> GetAllAttributesInActiveMode() const;

	/**
	 * Returns a map of attributes and function names.
	 * Attributes outside the Active Mode's channel span range are ignored.
	 */
	UFUNCTION(BlueprintPure, Category = "DMX|Fixture Patch")
	TMap<FDMXAttributeName, FDMXFixtureFunction> GetAttributeFunctionsMap() const;

	/**
	 * Returns a map of function names and default values.
	 * Functions outside the Active Mode's channel span range are ignored.
	 */
	UFUNCTION(BlueprintPure, Category = "DMX|Fixture Patch")
	TMap<FDMXAttributeName, int32> GetAttributeDefaultMap() const;

	/** Returns a map of Attributes and their assigned channels */
	UFUNCTION(BlueprintPure, Category = "DMX|Fixture Patch")
	TMap<FDMXAttributeName, int32> GetAttributeChannelAssignments() const;

	/** Returns a map of function names and their Data Types. */
	UFUNCTION(BlueprintPure, Category = "DMX|Fixture Patch")
	TMap<FDMXAttributeName, EDMXFixtureSignalFormat> GetAttributeSignalFormats() const;

	/** DEPRECATED 4.27 */
	UE_DEPRECATED(4.27, "Deprecated since it's unclear how to use this function correctly. Use UDMXSubsystem::BytesToInt instead.")
	UFUNCTION(BlueprintPure, Category = "DMX|Fixture Patch", meta = (DeprecatedFunction, DeprecationMessage = "Deprecated 4.27. Please use DMXSubsystem's Bytes to Int instead, then create a map from that with Attribute Names ."))
	TMap<FDMXAttributeName, int32> ConvertRawMapToAttributeMap(const TMap<int32, uint8>& RawMap) const;

	/** Converts a map of Attribute Names with their DMX values to a map of DMX channels and with their DMX Values. */
	UFUNCTION(BlueprintPure, Category = "DMX|Fixture Patch")
	TMap<int32, uint8> ConvertAttributeMapToRawMap(const TMap<FDMXAttributeName, int32>& FunctionMap) const;

	/** Returns true if the Fixture Patch contains all Attributes in an Attribute Name to DMX Value Map. */
	UFUNCTION(BlueprintPure, Category = "DMX|Fixture Patch")
	bool IsMapValid(const TMap<FDMXAttributeName, int32>& FunctionMap) const;

	/** Returns true if the Fixture Patch contains the specified attribute and can use it to send and receive DMX */
	UFUNCTION(BlueprintPure, Category = "DMX|Fixture Patch")
	bool ContainsAttribute(FDMXAttributeName FunctionAttribute) const;

	/** Removes any Attribute Name that can not be sent or received by a Fixture Patch from an Attribute Name to DMX Value Map */
	UFUNCTION(BlueprintPure, Category = "DMX|Fixture Patch")
	TMap<FDMXAttributeName, int32> ConvertToValidMap(const TMap<FDMXAttributeName, int32>& FunctionMap) const;
	
	/** DEPRECATED 4.27 */
	UE_DEPRECATED(4.27, "Controllers are replaced with DMX Ports.")
	UFUNCTION(BlueprintPure, Category = "DMX|Fixture Patch", meta = (DeprecatedFunction, DeprecationMessage = "Deprecated 4.27. Controllers are replaced with DMX Ports."))
	TArray<UDMXEntityController*> GetRelevantControllers() const;

	/** DEPRECATED 4.27 */
	UE_DEPRECATED(4.27, "Controllers are replaced with DMX Ports.")
	UFUNCTION(BlueprintPure, Category = "DMX|Fixture Patch", meta = (DeprecatedFunction, DeprecationMessage = "Deprecated 4.27. Controllers are replaced with DMX Ports."))
	bool IsInControllerRange(const UDMXEntityController* InController) const { return false; }

	/** DEPRECATED 4.27 */
	UE_DEPRECATED(4.27, "Controllers are replaced with DMX Ports.")
	UFUNCTION(BlueprintPure, Category = "DMX|Fixture Patch", meta = (DeprecatedFunction, DeprecationMessage = "Deprecated 4.27. Controllers are replaced with DMX Ports."))
	bool IsInControllersRange(const TArray<UDMXEntityController*>& InControllers) const { return false; }

	/**
	 * Returns the function currently mapped to the passed in Attribute, if any.
	 * If no function is mapped to it, returns nullptr.
	 *
	 * @param Attribute The attribute name to search for.
	 * @return			The function mapped to the passed in Attribute or nullptr
	 *					if no function is mapped to it.
	 */
	const FDMXFixtureFunction* GetAttributeFunction(const FDMXAttributeName& Attribute) const;

	/**
	 * Retrieves the value of an Attribute. Will fail and return 0 if the Attribute doesn't exist.
	 *
	 * @param Attribute	The Attribute to try to get the value from.
	 * @param bSuccess	Whether the Attribute was found in this Fixture Patch
	 * @return			The value of the Function mapped to the selected Attribute, if found.
	 */
	UFUNCTION(BlueprintCallable, Category = "DMX|Fixture Patch")
	int32 GetAttributeValue(FDMXAttributeName Attribute, bool& bSuccess);

	/**
	 * Retrieves the normalized value of an Attribute. Will fail and return 0 if the Attribute doesn't exist.
	 *
	 * @param Attribute	The Attribute to try to get the value from.
	 * @param bSuccess	Whether the Attribute was found in this Fixture Patch
	 * @return			The value of the Function mapped to the selected Attribute, if found.
	 */
	UFUNCTION(BlueprintCallable, Category = "DMX|Fixture Patch", meta = (DeprecatedFunction, DeprecationMessage = "Deprecated 4.26. Use GetNormalizedAttributeValue instead. Note, new method returns normalized values!"))
	float GetNormalizedAttributeValue(FDMXAttributeName Attribute, bool& bSuccess);


	/** DEPRECATED 5.5, renamed to GetAttributeValues. */
	UE_DEPRECATED(5.5, "Renamed to UDMXEntityFixturePatch::GetAttributeValues for consistency with similar methods.")
	void GetAttributesValues(TMap<FDMXAttributeName, int32>& AttributesValues);

	/**
	 * Returns the value of each attribute, or zero if no value was ever received.
	 *
	 * @param AttributesValues	Out: Resulting map of Attributes with their values
	 */
	UFUNCTION(BlueprintCallable, Category = "DMX|Fixture Patch")
	void GetAttributeValues(TMap<FDMXAttributeName, int32>& AttributeValues);

	/** DEPRECATED 5.5, renamed to GetNormalizedAttributeValues. */
	UE_DEPRECATED(5.5, "Renamed to UDMXEntityFixturePatch::GetNormalizedAttributeValues for consistency with similar methods.")
	void GetNormalizedAttributesValues(FDMXNormalizedAttributeValueMap& NormalizedAttributeValues);

	/**
	 * Returns the normalized value of each attribute, or zero if no value was ever received.
	 *
	 * @param AttributesValues	Out: Resulting map of Attributes with their normalized values
	 */
	UFUNCTION(BlueprintCallable, Category = "DMX|Fixture Patch")
	void GetNormalizedAttributeValues(FDMXNormalizedAttributeValueMap& NormalizedAttributesValues);

	/** Sends the DMX value of the Attribute to specified matrix coordinates */
	UFUNCTION(BlueprintCallable, Category = "DMX|Fixture Patch")
	bool SendMatrixCellValue(const FIntPoint& CellCoordinate /* Cell coordinate X/Y */, const FDMXAttributeName& Attribute, int32 Value);

	/** Sends the DMX value of the Attribute to specified matrix coordinates with given Attribute Name Channel Map */
	UE_DEPRECATED(4.27, "Deprecated due to ambigous arguments CellCoordinate and InAttributeNameChannelMap. Use SendMatrixCellValue or SendNormalizedMatrixCellValue instead.")
	UFUNCTION(BlueprintCallable, Category = "DMX|Fixture Patch", meta = (DeprecatedFunction, DeprecationMessage = "Deprecated due to ambigous arguments CellCoordinate and InAttributeNameChannelMap. Use SendMatrixCellValue or SendNormalizedMatrixCellValue instead."))
	bool SendMatrixCellValueWithAttributeMap(const FIntPoint& CellCoordinate /* Cell coordinate X/Y */, const FDMXAttributeName& Attribute, int32 Value, const TMap<FDMXAttributeName, int32>& InAttributeNameChannelMap);
	
	/** Maps the normalized value to the Attribute's full value range and sends it to specified matrix coordinates  */
	UFUNCTION(BlueprintCallable, Category = "DMX|Fixture Patch")
	bool SendNormalizedMatrixCellValue(const FIntPoint& CellCoordinate /* Cell coordinate X/Y */, const FDMXAttributeName& Attribute, float RelativeValue);

	/** Gets the DMX Cell value using matrix coordinates */
	UFUNCTION(BlueprintCallable, Category = "DMX|Fixture Patch")
	bool GetMatrixCellValues(const FIntPoint& CellCoordinate /* Cell coordinate X/Y */, TMap<FDMXAttributeName, int32>& ValuePerAttribute);
	
	/** Gets the DMX Cell value using matrix coordinates. */
	UFUNCTION(BlueprintCallable, Category = "DMX|Fixture Patch")
	bool GetNormalizedMatrixCellValues(const FIntPoint& CellCoordinate /* Cell coordinate X/Y */, TMap<FDMXAttributeName, float>& NormalizedValuePerAttribute);

	/** Gets the starting channel of each cell attribute at given coordinate, relative to the Starting Channel of the patch. */
	UFUNCTION(BlueprintCallable, Category = "DMX|Fixture Patch")
	bool GetMatrixCellChannelsRelative(const FIntPoint& CellCoordinate /* Cell coordinate X/Y */, TMap<FDMXAttributeName, int32>& AttributeChannelMap);
	
	/** Gets the absolute starting channel of each cell attribute at given coordinate */
	UFUNCTION(BlueprintCallable, Category = "DMX|Fixture Patch")
	bool GetMatrixCellChannelsAbsolute(const FIntPoint& CellCoordinate /* Cell coordinate X/Y */, TMap<FDMXAttributeName, int32>& AttributeChannelMap);

	/** Validates and gets the absolute starting channel of each cell attribute at given coordinate */
	UFUNCTION(BlueprintCallable, Category = "DMX|Fixture Patch")
	bool GetMatrixCellChannelsAbsoluteWithValidation(const FIntPoint& InCellCoordinate /* Cell coordinate X/Y */, TMap<FDMXAttributeName, int32>& OutAttributeChannelMap);

	/** Gets the Matrix Fixture properties, returns false if the patch is not using a matrix fixture */
	UFUNCTION(BlueprintPure, Category = "DMX|Fixture Patch")
	bool GetMatrixProperties(FDMXFixtureMatrix& MatrixProperties) const;

	/** Gets all attributes names of a cell */
	UFUNCTION(BlueprintCallable, Category = "DMX|Fixture Patch")
	bool GetCellAttributes(TArray<FDMXAttributeName>& CellAttributes);

	/** Gets the cell corresponding to the passed in coordinate */
	UFUNCTION(BlueprintCallable, Category = "DMX|Fixture Patch")
	bool GetMatrixCell(const FIntPoint& CellCoordinate /* Cell coordinate X/Y */, FDMXCell& Cell);

	/** Gets all matrix cells */
	UFUNCTION(BlueprintCallable, Category = "DMX|Fixture Patch")
	bool GetAllMatrixCells(TArray<FDMXCell>& Cells);

private:
	/** 
	 * Sends reset data to all channels. If bUseDefaultValues is true, sends default values. If it's false it sends zeroes.
	 * Note, this call will not raise send DMX traces.
	 */
	void SendResetDataToAllAttributes(bool bUseDefaultValues);

	/** Called when a Fixture Type changed */
	void OnFixtureTypeChanged(const UDMXEntityFixtureType* FixtureType);

	/** Tries to access the FixtureMatrix config of this patch and logs issues. Returns the matrix of nullptr if it isn't valid. */
	const FDMXFixtureMatrix* GetFixtureMatrix() const;

	/** Returns true if the specified coordinates are valid for the specified matrix */
	static bool AreCoordinatesValid(const FDMXFixtureMatrix& FixtureMatrix, const FIntPoint& Coordinate, bool bLogged = true);
};
