// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolCommon.h"
#include "Library/DMXEntityFixtureType.h"

#include "CoreMinimal.h"

struct FDMXAttributeName;
struct FDMXNormalizedAttributeValueMap;
struct FDMXRawAttributeValueMap;
class UDMXModulator;


/** 
 * Cache for fixture patches. This object takes care of handling the fixture patch data in a single point. The fixture patch self is used to define it.
 */
class FDMXEntityFixturePatchCache
{
public:
	/** Default constructor */
	FDMXEntityFixturePatchCache();

	/** 
	 * Constructs the cache. Arguments are used to determine what's cached.
	 * 
	 * @Param InStartingChannel		The Starting Channel of the patch.
	 * @Param InActiveModePtr		Pointer to the Active Mode of the patch, or nullptr if no Active Mode is available.
	 * @Param InFixtureMatrixPtr	Pointer to the fixture matrix of the patch, or nullptr if no Fixture Matrix is available
	 */
	FDMXEntityFixturePatchCache(int32 InStartingChannel, const FDMXFixtureMode* InActiveModePtr, const FDMXFixtureMatrix* InFixtureMatrixPtr);

	/** Returns true if the cache holds valid properties retrived from mode and matrix. Cached values may be empty if no signal was input yet. */
	bool HasValidProperties() const { return bValid; }

	/** Returns true if there is the matrix properties are valid and can ever hold data */
	bool IsFixtureMatrix() const { return bFixtureMatrix; }


	////////////////////////////////////
	// Core functionality

	/** Inputs a signal into the cache. Returns true if cached data changed. */
	bool InputDMXSignal(const FDMXSignalSharedPtr& DMXSignal);

	/** Resets cached data */
	void Reset();

	/** Applies the modulator to the cache */
	void Modulate(UDMXEntityFixturePatch* FixturePatch, UDMXModulator* Modulator);
	
	////////////////////////////////////
	// Value Getters

	/** Returns the raw attribute value for the specified Attribute Name. Returns nullptr if the attribute cannot be found. */
	const int32* GetRawAttributeValue(const FDMXAttributeName& AttributeName) const;

	/** Returns all raw attribute values. Returns nullptr if there is no data */
	const TMap<FDMXAttributeName, int32>* GetAllRawAttributeValues() const;

	/** Returns the raw attribute value for the specified Attribute Name. Returns nullptr if the attribute cannot be found */
	const float* GetNormalizedAttributeValue(const FDMXAttributeName& AttributeName) const;

	/** Returns all raw attribute values. Returns nullptr if there is no data */
	const FDMXNormalizedAttributeValueMap* GetAllNormalizedAttributeValues() const;

	/** Returns a pixel mapping distribution ordered Cell Index from a Cell Coordinate, or INDEX_NONE if not a valid index */
	int32 GetDistributedCellIndex(const FIntPoint& CellCoordinate) const;

	/** Returns a pixel mapping distribution ordered Cell Index from a Cell Coordinate, or INDEX_NONE if not a valid index */
	int32 GetDistributedCellIndex(int32 CellIndex) const;

	/** Returns the raw attribute value for the specified Attribute Name. Returns nullptr if the attribute cannot be found. */
	const int32* GetRawMatrixAttributeValueFromCell(int32 CellIndex, const FDMXAttributeName& AttributeName) const;

	/** Returns the all raw attribute values for the specified Cell. Returns nullptr if the attribute cannot be found. */
	const TMap<FDMXAttributeName, int32>* GetAllRawMatrixAttributeValuesFromCell(int32 CellIndex) const;

	/** Returns all raw matrix attribute values. Returns nullptr if the attribute cannot be accessed or if there is no data. */
	const TArray<TMap<FDMXAttributeName, int32>>* GetAllRawMatrixAttributeValues() const;

	/** Returns the normalized attribute value for the specified Attribute Name. Returns nullptr if the attribute cannot be found. */
	const float* GetNormalizedMatrixAttributeValueFromCell(int32 CellIndex, const FDMXAttributeName& AttributeName) const;

	/** Returns the all raw attribute values for the specified Cell. Returns nullptr if the attribute cannot be found. */
	const FDMXNormalizedAttributeValueMap* GetAllNormalizedMatrixAttributeValuesFromCell(int32 CellIndex) const;

	/** Returns all raw matrix attribute values. Returns nullptr if the attribute cannot be accessed or if there is no data. */
	const TArray<FDMXNormalizedAttributeValueMap>* GetAllNormalizedMatrixAttributeValues() const;

	////////////////////////////////////
	// Property getters

	/** Returns the functions of the mode */
	const TArray<FDMXFixtureFunction>& GetFunctions() const { return Mode.Functions; }

	/** Returns the cell attributes of the fixture matrix */
	const TArray<FDMXFixtureCellAttribute>& GetCellAttributes() const { return FixtureMatrix.CellAttributes; }

	/** Returns the absolute starting channel of the matrix attributes */
	const int32 GetMatrixStartingChannelAbsolute() const { return DataIndex + MatrixOffset + 1; }

	/** Returns the number of collumns in the matrix. Does not test if the cache is using the matrix. */
	int32 GetMatrixNumXCells() const { return FixtureMatrix.XCells; }

	/** Returns the number of rows in the matrix. Does not test if the cache is using the matrix. */
	int32 GetMatrixNumYCells() const { return FixtureMatrix.YCells; }

	/** Returns the data size of a single cell */
	int32 GetCellSize() const { return CellDataSize; }

	/** Returns the channel span */
	int32 GetChannelSpan() const { return DataSize; }

	/** Returns the attributes of the mode's function, without matrix attributes */
	TArray<FDMXAttributeName> GetAttributeNames() const;

	/** Returns the attribute names of the fixture matrix' cell attributes, without common function attributes */
	TArray<FDMXAttributeName> GetCellAttributeNames() const;

private:	
	////////////////////
	// Properties
		
	/** True if it holds valid properties */
	bool bValid;

	/** True if this is constructed for a matrix fixture */
	bool bFixtureMatrix;

	/** Index where the data of the Patch starts in a Universe */
	int32 DataIndex;

	/** Number of DMX channels the patch spans, including the matrix */
	int32 DataSize;

	/** Offset of the fixture matrix from the starting channel. Can exceed the data size. */
	int32 MatrixOffset;

	/** Number of cells in the matrix */
	int32 NumCells;

	/** Data Size of a single Cell */
	int32 CellDataSize;

	/** The mode to use */
	FDMXFixtureMode Mode;

	/** The matrix to use */
	FDMXFixtureMatrix FixtureMatrix;

	/** Cell indicies ordered by fixture, for fast conversion of a raw cell index */
	TArray<int32> OrderedCellIndicies;

	////////////////////
	// Cache

	/**
	 * Raw cached DMX values, last received. This only contains DMX data relevant to the patch, from starting channel to starting channel + channel span.
	 * Matrix values contained here are NOT pixelmapping distribution ordered.
	 */
	TArray<uint8> CachedDMXValues;

	/** Map of latest normalized values per (non-matrix) attribute */
	TMap<FDMXAttributeName, int32> CachedRawValuesPerAttribute;

	/** Map of latest normalized values per (non-matrix) attribute */
	FDMXNormalizedAttributeValueMap CachedNormalizedValuesPerAttribute;

	/** Map of raw values per matrix attribute */
	TArray<TMap<FDMXAttributeName, int32>> CachedRawValuesPerMatrixCell;

	/** Map of latest normalized values per matrix attributen */
	TArray<FDMXNormalizedAttributeValueMap> CachedNormalizedValuesPerMatrixCell;
};
