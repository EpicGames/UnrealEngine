// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXAttribute.h"
#include "DMXProtocolTypes.h"
#include "GDTF/AttributeDefinitions/DMXGDTFPhysicalUnit.h"
#include "Library/DMXEntity.h"
#include "Library/DMXEntityReference.h"
#include "Modulators/DMXModulator.h"

#include "DMXEntityFixtureType.generated.h"

class UDMXGDTF;
class UDMXImport;
class UDMXImportGDTF;


UENUM(BlueprintType, DisplayName = "DMX Pixel Mapping Distribution")
enum class EDMXPixelMappingDistribution : uint8
{
	TopLeftToRight,
	TopLeftToBottom,
	TopLeftToClockwise,
	TopLeftToAntiClockwise,

	TopRightToLeft,
	BottomLeftToTop,
	TopRightToAntiClockwise,
	BottomLeftToClockwise,

	BottomLeftToRight,
	TopRightToBottom,
	BottomLeftAntiClockwise,
	TopRightToClockwise,

	BottomRightToLeft,
	BottomRightToTop,
	BottomRightToClockwise,
	BottomRightToAntiClockwise
};

USTRUCT(BlueprintType, meta = (DisplayName = "DMX Fixture Function"))
struct DMXRUNTIME_API FDMXFixtureFunction
{
	GENERATED_BODY()

	/** Constructor */
	FDMXFixtureFunction()
		: Attribute()
	{}

	/** Implementing Serialize to convert UClass to FFieldClass */
	void PostSerialize(const FArchive& Ar);

	/** Returns the number of channels the function spans, according to its data type */
	uint8 GetNumChannels() const { return static_cast<uint8>(DataType) + 1; }

	/** Returns the last channel of the Function */
	int32 GetLastChannel() const;

#if WITH_EDITOR
	/** Gets the Physical Value of the Function */
	double GetPhysicalDefaultValue() const { return PhysicalDefaultValue; }

	/** The Physical Unit this Physical Value is based on */
	EDMXGDTFPhysicalUnit GetPhysicalUnit() const { return PhysicalUnit; }

	/** The starting value of the Physical Value range, based on the Physical Unit */
	double GetPhysicalFrom() const { return PhysicalFrom; }

	/** The ending value of the Physical Value range, based on the Physical Unit */
	double GetPhysicalTo() const { return PhysicalTo; }

	/** Sets the Physical Default Value of the Function. */
	void SetPhysicalUnit(EDMXGDTFPhysicalUnit NewPhysicalUnit) { PhysicalUnit = NewPhysicalUnit; }

	/** Sets the Physical Default Value of the Function. */
	void SetPhysicalDefaultValue(double InPhysicalDefaultValue);

	/** Sets the Physical Default Value range of the Function. */
	void SetPhysicalValueRange(double InPhysicalFrom, double InPhysicalTo);

	/** Updated the Physica Default Value of the Function by the Default Value. */
	void UpdatePhysicalDefaultValue();

	// Property Name getters
	static FName GetPhysicalDefaultValuePropertyName() { return GET_MEMBER_NAME_CHECKED(FDMXFixtureFunction, PhysicalDefaultValue); }
	static FName GetPhysicalUnitPropertyName() { return GET_MEMBER_NAME_CHECKED(FDMXFixtureFunction, PhysicalUnit); }
	static FName GetPhysicalFromPropertyName() { return GET_MEMBER_NAME_CHECKED(FDMXFixtureFunction, PhysicalFrom); }
	static FName GetPhysicalToPropertyName() { return GET_MEMBER_NAME_CHECKED(FDMXFixtureFunction, PhysicalTo); }
#endif // WITH_EDITOR

	/**
	 * The Attribute name to map this Function to.
	 * This is used to easily find the Function in Blueprints, using an Attribute
	 * list instead of typing the Function name directly.
	 * The list of Attributes can be edited on
	 * Project Settings->Plugins->DMX Protocol->Fixture Settings->Fixture Function Attributes
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayName = "Attribute Mapping", DisplayPriority = "11"), Category = "Function Settings")
	FDMXAttributeName Attribute;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayPriority = "10"), Category = "Function Settings")
	FString FunctionName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayPriority = "20"), Category = "Function Settings")
	FString Description;

	/** The Default DMX Value of the function */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayPriority = "30"), Category = "Function Settings")
	int64 DefaultValue = 0;

	/** This function's starting channel (use editor above to make changes) */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, meta = (DisplayName = "Channel Assignment", DisplayPriority = "2"), Category = "Function Settings")
	int32 Channel = 1;

	/** This function's data type. Defines the used number of channels (bytes) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayPriority = "5"), Category = "Function Settings")
	EDMXFixtureSignalFormat DataType = EDMXFixtureSignalFormat::E8Bit;

	/**
	 * Least Significant Byte mode makes the individual bytes (channels) of the function be
	 * interpreted with the first bytes being the lowest part of the number (endianness).
	 * 
	 * E.g., given a 16 bit function with two channel values set to [0, 1],
	 * they would be interpreted as the binary number 0x01 0x00, which means 256.
	 * The first byte (0) became the lowest part in binary form and the following byte (1), the highest.
	 * 
	 * Most Fixtures use MSB (Most Significant Byte) mode, which interprets bytes as highest first.
	 * In MSB mode, the example above would be interpreted in binary as 0x00 0x01, which means 1.
	 * The first byte (0) became the highest part in binary form and the following byte (1), the lowest.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayName = "Use LSB Mode", DisplayPriority = "29"), Category = "Function Settings")
	bool bUseLSBMode = false;

private:
#if WITH_EDITORONLY_DATA
	/** The Physical Value used by default, based on the Physical Unit */
	UPROPERTY(EditAnywhere, Transient, Category = "Physical Properties")
	double PhysicalDefaultValue = 0.0;

	/** The Physical Unit this Physical Value is based on */
	UPROPERTY(EditAnywhere, Category = "Physical Properties")
	EDMXGDTFPhysicalUnit PhysicalUnit = EDMXGDTFPhysicalUnit::None;

	/** The starting value of the Physical Value range, based on the Physical Unit */
	UPROPERTY(EditAnywhere, Category = "Physical Properties")
	double PhysicalFrom = 0.0;

	/** The ending value of the Physical Value range, based on the Physical Unit */
	UPROPERTY(EditAnywhere, Category = "Physical Properties")
	double PhysicalTo = 1.0;
#endif // WITH_EDITORONLY_DATA

public:
	// Workaround for clang deprecation warnings for any deprecated members in implicitly-defined special member functions
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FDMXFixtureFunction(const FDMXFixtureFunction&) = default;
	FDMXFixtureFunction(FDMXFixtureFunction&&) = default;
	FDMXFixtureFunction& operator=(const FDMXFixtureFunction&) = default;
	FDMXFixtureFunction& operator=(FDMXFixtureFunction&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS


	//////////////////////////////////////////////////
	// Deprecated Members

#if WITH_EDITORONLY_DATA
	/** DEPRECATED 5.0 */
	UE_DEPRECATED(5.0, "Instead please refer to the Channel property")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Deprecated, instead please refer to the Channel property."))
	int32 ChannelOffset_DEPRECATED = 0;
#endif // WITH_EDITORONLY_DATA 

};

template<>
struct TStructOpsTypeTraits<FDMXFixtureFunction> : public TStructOpsTypeTraitsBase2<FDMXFixtureFunction>
{
	enum
	{
		WithPostSerialize = true,
	};
};


USTRUCT(BlueprintType, meta = (DisplayName = "DMX Fixture Cell Attribute"))
struct DMXRUNTIME_API FDMXFixtureCellAttribute
{
	GENERATED_BODY()

	/** Constructor */
	FDMXFixtureCellAttribute()
		: Attribute()
		, Description()
		, DefaultValue(0)
		, DataType(EDMXFixtureSignalFormat::E8Bit)
		, bUseLSBMode(false)
	{}

	/** Returns the number of channels of the attribute */
	uint8 GetNumChannels() const { return static_cast<uint8>(DataType) + 1; }

	/**
	 * The Attribute name to map this Function to.
	 * This is used to easily find the Function in Blueprints, using an Attribute
	 * list instead of typing the Function name directly.
	 * The list of Attributes can be edited on
	 * Project Settings->Plugins->DMX Protocol->Fixture Settings->Fixture Function Attributes
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayName = "Attribute Mapping", DisplayPriority = "11"), Category = "DMX")
	FDMXAttributeName Attribute;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayPriority = "20", DisplayName = "Description"), Category = "DMX")
	FString Description;

	/** Initial value for this function when no value is set */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayPriority = "30", DisplayName = "Default Value"), Category = "DMX")
	int64 DefaultValue;

	/** This function's data type. Defines the used number of channels (bytes) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayPriority = "5", DisplayName = "Data Type"), Category = "DMX")
	EDMXFixtureSignalFormat DataType;

	/**
	 * The Endianess of the Attribute:
	 * Least Significant Byte mode makes the individual bytes (channels) of the function be
	 * interpreted with the first bytes being the lowest part of the number.
	 *
	 * E.g., given a 16 bit function with two channel values set to [0, 1],
	 * they would be interpreted as the binary number 00000001 00000000, which means 256.
	 * The first byte (0) became the lowest part in binary form and the following byte (1), the highest.
	 *
	 * Most Fixtures use MSB (Most Significant Byte) mode, which interprets bytes as highest first.
	 * In MSB mode, the example above would be interpreted in binary as 00000000 00000001, which means 1.
	 * The first byte (0) became the highest part in binary form and the following byte (1), the lowest.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayName = "Use LSB Mode", DisplayPriority = "29"), Category = "DMX")
	bool bUseLSBMode;
};

USTRUCT(BlueprintType, meta = (DisplayName = "DMX Fixture Matrix"))
struct DMXRUNTIME_API FDMXFixtureMatrix
{
	GENERATED_BODY()

	/** Constructor */
	FDMXFixtureMatrix();

	/** Returns the number of channels of the Matrix */
	int32 GetNumChannels() const;

	/** Returns the last channel of the Matrix */
	int32 GetLastChannel() const;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayPriority = "60", DisplayName = "Cell Attributes"), Category = "Mode Settings")
	TArray<FDMXFixtureCellAttribute> CellAttributes;

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, meta = (DisplayPriority = "20", DisplayName = "First Cell Channel", ClampMin = "1", ClampMax = "512"), Category = "Mode Settings")
	int32 FirstCellChannel = 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayPriority = "30", DisplayName = "X Cells", ClampMin = "1", ClampMax = "512", UIMin = "1", UIMax = "512"), Category = "Mode Settings")
	int32 XCells = 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayPriority = "40", DisplayName = "Y Cells", ClampMin = "1", ClampMax = "512", UIMin = "1", UIMax = "512"), Category = "Mode Settings")
	int32 YCells = 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayPriority = "50", DisplayName = "PixelMapping Distribution"), Category = "Mode Settings")
	EDMXPixelMappingDistribution PixelMappingDistribution = EDMXPixelMappingDistribution::TopLeftToRight;
};

USTRUCT(BlueprintType, meta = (DisplayName = "DMX Cell"))
struct DMXRUNTIME_API FDMXCell
{
	GENERATED_BODY()

	/** The cell index in a 1D Array (row order), starting from 0 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayPriority = "20", DisplayName = "Cell ID", ClampMin = "0"), Category = "DMX")
	int32 CellID;

	/** The cell coordinate in a 2D Array, starting from (0, 0) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayPriority = "30", DisplayName = "Coordinate"), Category = "DMX")
	FIntPoint Coordinate;

	FDMXCell()
		: CellID(0)
		, Coordinate(FIntPoint (-1,-1))
	{}
};

USTRUCT(BlueprintType, meta = (DisplayName = "DMX Fixture Mode"))
struct DMXRUNTIME_API FDMXFixtureMode
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayPriority = "10"), Category = "Mode Settings")
	FString ModeName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayPriority = "20"), Category = "Mode Settings")
	TArray<FDMXFixtureFunction> Functions;

	/**
	 * When enabled, ChannelSpan is automatically set based on the created functions and their data types.
	 * If disabled, ChannelSpan can be manually set and functions and functions' channels beyond the
	 * specified span will be ignored.
	 */
	UPROPERTY(EditAnywhere, Category = "Mode Settings", meta = (DisplayPriority = "30"))
	bool bAutoChannelSpan = true;

	/** Number of channels (bytes) used by this mode's functions */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mode Settings", meta = (ClampMin = "4", ClampMax = "512", DisplayPriority = "40", EditCondition = "!bAutoChannelSpan"))
	int32 ChannelSpan = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mode Settings", meta = (DisplayPriority = "60"))
	bool bFixtureMatrixEnabled = false;

	UPROPERTY(EditAnywhere, Category = "Mode Settings", meta = (DisplayPriority = "70"))
	FDMXFixtureMatrix FixtureMatrixConfig;

#if WITH_EDITOR
	/** DEPRECATED 5.0 */
	UE_DEPRECATED(5.0, "Removed in favor of UDMXEntityFixtureType::AddFunction and UDMXEntityFixtureType::InsertFunction")
	int32 AddOrInsertFunction(int32 IndexOfFunction, FDMXFixtureFunction InFunction);
#endif
};


/** Parameters to construct a Fixture Type. */
USTRUCT(BlueprintType, meta = (DisplayName = "DMX Entity Fixture Type Construction Params"))
struct DMXRUNTIME_API FDMXEntityFixtureTypeConstructionParams
{
	GENERATED_BODY()

	/** The DMX Library in which the Fixture Type will be constructed */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Fixture Type", meta = (DisplayName = "Parent DMX Library"))
	TObjectPtr<UDMXLibrary> ParentDMXLibrary = nullptr;

	/** The Category of the Fixture, useful for Filtering */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Fixture Type", meta = (DisplayName = "DMX Category"))
	FDMXFixtureCategory DMXCategory;

	/** The Modes of the Fixture Type */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Fixture Type")
	TArray<FDMXFixtureMode> Modes;
};

#if WITH_EDITOR
/** Notification when data type changed */
DECLARE_MULTICAST_DELEGATE_TwoParams(FDataTypeChangeDelegate, const UDMXEntityFixtureType*, const FDMXFixtureMode&);
#endif

DECLARE_MULTICAST_DELEGATE_OneParam(FDMXOnFixtureTypeChangedDelegate, const UDMXEntityFixtureType* /** ChangedFixtureType */);


/** 
 * Class to describe a type of Fixture. Fixture Patches can be created from Fixture Types (see UDMXEntityFixturePatch::ParentFixtureTypeTemplate).
 */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "DMX Fixture Type"))
class DMXRUNTIME_API UDMXEntityFixtureType
	: public UDMXEntity
{
	GENERATED_BODY()

public:
	UDMXEntityFixtureType();

	/** Creates a new Fixture Type in the DMX Library */
	UFUNCTION(BlueprintCallable, Category = "DMX|Fixture Type")
	static UDMXEntityFixtureType* CreateFixtureTypeInLibrary(FDMXEntityFixtureTypeConstructionParams ConstructionParams, const FString& DesiredName = TEXT(""), bool bMarkDMXLibraryDirty = true);

	/** Removes a Fixture Type from a DMX Library */
	UFUNCTION(BlueprintCallable, Category = "DMX|Fixture Type")
	static void RemoveFixtureTypeFromLibrary(FDMXEntityFixtureTypeRef FixtureTypeRef);

	//~ Begin UObject interface
	virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	virtual bool Modify(bool bAlwaysMarkDirty = true) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent) override;
	virtual void PostEditUndo() override;
#endif
	//~ End UObject interface

#if WITH_EDITOR
	/**
	 * Acquires the GDTF file name of this Fixture Type. If bWithExtension is true, appends the .gdtf extension
	 *
	 * Note this is a slow operation that will load the GDTF Source if required and look up the filename from asset import data.
	 * For Fixture Types that do not stem from an imported GDTF, a filename is generated based on the Fixture Type name.
	 */
	FString GetCleanGDTFFileNameSynchronous(bool bWithExtension) const;

	UE_DEPRECATED(5.5, "Setting GDTFs this way is not supported. Instead set the GDTFSource and generate the modes via UDMXGDTF.")
	UFUNCTION(BlueprintCallable, Category = "Fixture Settings", meta = (DeprecatedFunction, DeprecationMessage = "Setting GDTFs from blueprints was never fully supported and now deprecated. Instead please refer to the Create Fixture Type In DMX Library function."))
	void SetModesFromDMXImport(UDMXImport* DMXImportAsset);
#endif // WITH_EDITOR

	/** Returns a delegate that is and should be broadcast whenever a Fixture Type changed */
	static FDMXOnFixtureTypeChangedDelegate& GetOnFixtureTypeChanged();

	/** The Category of the Fixture, useful for Filtering */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Fixture Type", meta = (DisplayName = "DMX Category"))
	FDMXFixtureCategory DMXCategory;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Fixture Type")
	TArray<FDMXFixtureMode> Modes;

	/** 
	 * Modulators applied right before a patch of this type is received. 
	 * NOTE: Modulators only affect the patch's normalized values! Untouched values are still available when accesing raw values. 
	 */
	UPROPERTY(EditAnywhere, Instanced, Category = "Fixture Type", meta = (DisplayPriority = "50"))
	TArray<TObjectPtr<UDMXModulator>> InputModulators;

private:
	/** Delegate that should be broadcast whenever a Fixture Type changed */
	static FDMXOnFixtureTypeChangedDelegate OnFixtureTypeChangedDelegate;


	//////////////////////////////////////////////////
	// Helpers to edit the Fixture Type

	// Fixture Mode related
public:
	/**
	 * Adds a Mode to the Modes Array
	 * 
	 * @param(optional)						The Base Mode Name when generating a name
	 *
	 * @return								The Index of the newly added Mode.
	 */	
	int32 AddMode(FString BaseModeName = FString("Mode"));

	/** 
	 * Duplicates the Modes at specified Indices 
	 *
	 * @param InModeIndices					The indicies of the Modes to duplicate
	 * @param OutNewModeIndices				The indices of the newly created Modes.
	 */
	void DuplicateModes(TArray<int32> InModeIndicesToDuplicate, TArray<int32>& OutNewModeIndices);

	/** Deletes the Modes at specified Indices */
	void RemoveModes(const TArray<int32>& ModeIndicesToDelete);

	/** Sets a Mode Name for specified Mode 
	 * 
	 * @param InModeIndex					The index of the Mode in the Modes array
	 * @param InDesiredModeName				The desired Name that should be set
	 * @param OutUniqueModeName				The unique Name that was set
	*/
	void SetModeName(int32 InModeIndex, const FString& InDesiredModeName, FString& OutUniqueModeName);

	/** Enables or disables the Matrix, reorders Function channels accordingly
	 *
	 * @param ModeIndex						The index of the Mode for which the Matrix should be enabled or disabled.
	 * @param bEnableMatrix					Whether to enable or disable the Matrix
	*/
	void SetFixtureMatrixEnabled(int32 ModeIndex, bool bEnableMatrix);

	/**
	 * Updates the channel span of the Mode.
	 *
	 * @param ModeIndex						The Index of the Mode for which the Channel Span should be updated.
	 */
	void UpdateChannelSpan(int32 ModeIndex);

	/** Aligns alls channels of the functions in the Mode to be consecutive */
	void AlignFunctionChannels(int32 InModeIndex);

	// Fixture Function related
public:
	/** 
	 * Adds a new Function to the Mode's Functions array
	 *
	 * @param InModeIndex					The index of the Mode, that will have the Function added to its Functions array.
	 * @return								The Index of the newly added Function.
	 */
	int32 AddFunction(int32 InModeIndex);

	/** 
	 * Inserts a Function to the Mode's Function Array
	 *
	 * @param InModeIndex					The index of the Mode, that will have the Function added to its Functions array.
	 * @param InInsertAtIndex				If a valid Index, the Function will be inserted at this Index, and subsequent Function's Channels will be reodered after it.
	 * @param InOutNewFunction				The function that will be inserted.
	 * @return								The Index of the newly added Function.
	 */
	int32 InsertFunction(int32 InModeIndex, int32 InInsertAtIndex, FDMXFixtureFunction& InOutNewFunction);

	/**
	 * Adds a Function to the Mode's Function Array
	 *
	 * @param InModeIndex					The index of the Mode in which the Functions to duplicate reside.
	 * @param InFunctionIndicesToDuplicate	The indices of the Functions to duplicate.
	 * @param OutNewFunctionIndices			The Function indices where the newly added Functions reside
	 */
	void DuplicateFunctions(int32 InModeIndex, const TArray<int32>& InFunctionIndicesToDuplicate, TArray<int32>& OutNewFunctionIndices);

	/**
	 * Removes Functions from the Mode's Function Array
	 *
	 * @param InModeIndex					The index of the Mode in which the Functions to remove reside
	 * @param FunctionIndicesToDelete		The indices of the Functions to remove.
	 */
	void RemoveFunctions(int32 ModeIndex, TArray<int32> FunctionIndicesToDelete);

	/**
	 * Reorders a function to reside at the Insert At Index, subsequently reorders other affected Functions
	 *
	 * @param ModeIndex						The Index of the Mode in which the Functions reside
	 * @param FunctionIndex					The Index of the Function that is reorderd
	 * @param InsertAtIndex					The Index of the Function where the function is inserted.
	 */
	void ReorderFunction(int32 ModeIndex, int32 FunctionToReorderIndex, int32 InsertAtIndex);

	/** Sets a Mode Name for specified Mode
	 *
	 * @param InModeIndex					The index of the Mode in the Modes array
	 * @param InFunctionIndex				The index of the Function in the Mode's Function array
	 * @param DesiredFunctionName			The desired Name that should be set
	 * @param OutUniqueFunctionName			The unique Name that was set
	*/
	void SetFunctionName(int32 InModeIndex, int32 InFunctionIndex, const FString& InDesiredFunctionName, FString& OutUniqueFunctionName);

	/** Sets a Starting Channel for the Function, aligns it to other functions
	 *
	 * @param InModeIndex					The index of the Mode in the Modes array
	 * @param InFunctionIndex				The index of the Function in the Mode's Function array
	 * @param InDesiredStartingChannel		The desired Starting Channel that should be set
	 * @param OutStartingChannel			The resulting Starting Channel that was set
	*/
	void SetFunctionStartingChannel(int32 InModeIndex, int32 InFunctionIndex, int32 InDesiredStartingChannel, int32& OutStartingChannel);

	/**
	 * Clamps the Default Value of the Function by its Data Type
	 *
	 * @param ModeIndex						The Index of the Mode in which the Functions reside
	 * @param FunctionIndex					The Index of the Function for which the Default Value is clamped
	 */
	UE_DEPRECATED(5.5, "Removed as physical values were introduced to FDMXFixtureFunction (editor only) . Please handle the default value of the function per use case.")
	void ClampFunctionDefautValueByDataType(int32 ModeIndex, int32 FunctionIndex);

	// Fixture Matrix related
public:
	/** Adds a new cell attribute to the Mode */
	void AddCellAttribute(int32 ModeIndex);

	/** Removes a cell attribute from the Mode */
	void RemoveCellAttribute(int32 ModeIndex, int32 CellAttributeIndex);

	/**
	 * Reorders the Fixture Matrix to reside after a function, subsequently reorders other affected Functions
	 *
	 * @param FixtureType					The Fixture Type in which the Functions reside
	 * @param ModeIndex						The Index of the Mode in which the Functions reside
	 * @param InsertAfterFunctionIndex		The Index of the Function after which the Matrix is inserted. If an invalid index is specified, the Matrix will added after the last Function Channel.
	 */
	void ReorderMatrix(int32 ModeIndex, int32 InsertAtFunctionIndex);

	/**
	 * Updates the channel span of the Mode.
	 *
	 * @param ModeIndex						The Index of the Mode for which the YCells should be updated
	 */
	void UpdateYCellsFromXCells(int32 ModeIndex);

	/**
	 * Updates the channel span of the Mode.
	 *
	 * @param ModeIndex						The Index of the Mode for which the XCells should be updated
	 */
	void UpdateXCellsFromYCells(int32 ModeIndex);


	// Conversions. @TODO, move to FDMXConversions
public:
	//~ Conversions to/from Bytes, Int and Normalized Float values.
	static void FunctionValueToBytes(const FDMXFixtureFunction& InFunction, uint32 InValue, uint8* OutBytes);
	static void IntToBytes(EDMXFixtureSignalFormat InSignalFormat, bool bUseLSB, uint32 InValue, uint8* OutBytes);

	static uint32 BytesToFunctionValue(const FDMXFixtureFunction& InFunction, const uint8* InBytes);
	static uint32 BytesToInt(EDMXFixtureSignalFormat InSignalFormat, bool bUseLSB, const uint8* InBytes);

	static void FunctionNormalizedValueToBytes(const FDMXFixtureFunction& InFunction, float InValue, uint8* OutBytes);
	static void NormalizedValueToBytes(EDMXFixtureSignalFormat InSignalFormat, bool bUseLSB, float InValue, uint8* OutBytes);

	static float BytesToFunctionNormalizedValue(const FDMXFixtureFunction& InFunction, const uint8* InBytes);
	static float BytesToNormalizedValue(EDMXFixtureSignalFormat InSignalFormat, bool bUseLSB, const uint8* InBytes);

	/** The GDTF that initializes this Fixture Type. When changed, reinitializes with data from the GDTF. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Fixture Type", meta = (DisplayName = "GDTF Source"))
	TSoftObjectPtr<UDMXImportGDTF> GDTFSource;

#if WITH_EDITORONLY_DATA
	/** If checked, generates a new GDTF instead of exporting the imported GDTF. This adopts changes in editor but in most cases will result in data loss and is not recommended. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Fixture Type", meta = (DisplayName = "Generate GDTF (not recommended)"))
	bool bExportGeneratedGDTF = false;

#if WITH_EDITORONLY_DATA
	/**
	 * The Actor Class that is spawned when the DMX Library dropped onto a Level.
	 * Only Actors that implement the MVR Fixture Actor Interface can be used.
	 *
	 * Can be left blank. If so, any Actor Class with the most matching Attributes will be spawned.
	 */
	UPROPERTY(EditAnywhere, Category = "MVR", Meta = (MustImplement = "/Script/DMXFixtureActorInterface.DMXMVRFixtureActorInterface"))
	TSoftClassPtr<AActor> ActorClassToSpawn;
#endif // WITH_EDITORONLY_DATA

	/** If true only shows latest GDTF mode revisions in editor */
	UPROPERTY()
	bool bShowOnlyLatestGDTFModeRevisions = true;
#endif

	//////////////////////////////////////////////////
	// Deprecated Members
	
public:
#if WITH_EDITORONLY_DATA
	/** DEPRECATED 5.5 - The GDTF from which this Fixture Type was setup. */
	UE_DEPRECATED(5.5, "Changed to a soft object pointer to reduce the memory footprint of Fixture Types. Please refer to GDTFSource instead.")
	UPROPERTY(BlueprintReadOnly, Category = "Fixture Settings", meta = (DeprecatedProperty, DeprecationMessage = "Changed to a soft object pointer to reduce the memory footprint of Fixture Types. Please use GDTF Source instead."))
	TObjectPtr<UDMXImport> DMXImport;

	UE_DEPRECATED(5.5, "bFixtureMatrixEnabled is deprecated. Instead now each Mode has a FixtureMatrixEnabled property.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "FixtureMatrixEnabled is deprecated. Instead now each Mode has a FixtureMatrixEnabled property."))
	bool bFixtureMatrixEnabled_DEPRECATED = false;
#endif
};
