// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "DMXEntityReference.generated.h"

class UDMXLibrary;
class UDMXEntity;
class UDMXEntityController;
class UDMXEntityFixtureType;
class UDMXEntityFixturePatch;

/**
 * Represents an Entity from a DMX Library.
 * Used to allow objects outside the DMX Library package to store references to UDMXEntity objects
 */
USTRUCT(meta = (DisplayName = "DMX Entity Reference"))
struct DMXRUNTIME_API FDMXEntityReference
{
	GENERATED_BODY()

public:
	/**
	 * The parent DMX library of the Entity
	 * Automatically set when calling SetEntity with a valid Entity.
	 */
	UPROPERTY(EditAnywhere, Category = "DMX")
	TObjectPtr<UDMXLibrary> DMXLibrary;

	/** Display the DMX Library asset picker. True by default, for Blueprint variables */
	UPROPERTY() 
	bool bDisplayLibraryPicker;

public:
	/** Construct with invalid values by default */
	FDMXEntityReference();

	/** Construct as a reference to an existing Entity and its type as EntityType */
	FDMXEntityReference(UDMXEntity* InEntity);

	/** Set the Entity and DMX Library this struct represents */
	void SetEntity(UDMXEntity* NewEntity);

	/** Get the Entity referenced by this struct or nullptr if none was set. */
	UDMXEntity* GetEntity() const;

	/** Get the type of entity this reference points to */
	TSubclassOf<UDMXEntity> GetEntityType() const;

public:
	/** Comparison operators */
	bool operator==(const FDMXEntityReference& Other) const;
	bool operator!=(const FDMXEntityReference& Other) const;

	/** Gets a hash from the DMX Library and EntityID values */
	FORCEINLINE_DEBUGGABLE friend uint32 GetTypeHash(const FDMXEntityReference& EntityRef)
	{
		// Get the 4 Guid int32 values (A, B, C, D) as 2 uint64 ones.
		const uint64* EntityId64 = reinterpret_cast<const uint64*>(&EntityRef.EntityId);

		// Take the Library address and the EntityID to make a hash from both
		uint64 ToHash[] =
		{
			reinterpret_cast<uintptr_t>(ToRawPtr(EntityRef.DMXLibrary)),
			static_cast<uint64>(EntityId64[0]),
			static_cast<uint64>(EntityId64[1])
		};
		
		return CityHash64(reinterpret_cast<char*>(&ToHash), sizeof(ToHash));
	}

protected:
	UPROPERTY()
	TSubclassOf<UDMXEntity> EntityType;
	
private:
	/** The entity's unique ID */
	UPROPERTY(EditAnywhere, Category = "DMX") // Without EditAnywhere here the value is not saved on components in a Level
	FGuid EntityId;

	/** Cached value this entity stands for, to speed up access instead of looking it up from the library array */
	mutable TWeakObjectPtr<UDMXEntity> CachedEntity;
};

/**
 * Represents a Controller from a DMX Library.
 * Used to store a reference to a Controller outside the DMX Library
 */
struct 	UE_DEPRECATED(5.5, "Controllers are no longer used and deprecated in favor of the DMX Port System.") FDMXEntityControllerRef;
USTRUCT(BlueprintType, meta = (Deprecated = "Controllers are no longer used and deprecated in favor of the DMX Port System."))
struct DMXRUNTIME_API FDMXEntityControllerRef
	: public FDMXEntityReference
{
	GENERATED_BODY()

public:
	FDMXEntityControllerRef();
	FDMXEntityControllerRef(UDMXEntityController* InController);

	/** Specialized GetEntity() to return it already cast */
	UDMXEntityController* GetController() const;
};

/**
 * Represents a Fixture Type from a DMX Library.
 * Used to store a reference to a Fixture Type outside the DMX Library
 */
USTRUCT(BlueprintType, meta = (DisplayName = "DMX Fixture Type Ref"))
struct DMXRUNTIME_API FDMXEntityFixtureTypeRef
	: public FDMXEntityReference
{
	GENERATED_BODY()

public:
	FDMXEntityFixtureTypeRef();
	FDMXEntityFixtureTypeRef(UDMXEntityFixtureType* InFixtureType);

	/** Specialized GetEntity() to return it already cast */
	UDMXEntityFixtureType* GetFixtureType() const;
};

/**
 * Represents a Fixture Patch from a DMX Library.
 * Used to store a reference to a Fixture Patch outside the DMX Library
 */
USTRUCT(BlueprintType, meta = (DisplayName = "DMX Fixture Patch Ref"))
struct DMXRUNTIME_API FDMXEntityFixturePatchRef
	: public FDMXEntityReference
{
	GENERATED_BODY()

public:
	FDMXEntityFixturePatchRef();
	FDMXEntityFixturePatchRef(UDMXEntityFixturePatch* InFixturePatch);

	/** Specialized GetEntity() to return it already cast */
	UDMXEntityFixturePatch* GetFixturePatch() const;
};

/** Extend type conversions to handle Entity Reference structs */
UCLASS(meta = (BlueprintThreadSafe))
class DMXRUNTIME_API UDMXEntityReferenceConversions
	: public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS // FDMXEntityControllerRef used in args is deprecated. Leave it to that to raise deprecated warnings in C++.
	UFUNCTION(BlueprintPure, meta = (DeprecatedFunction, DeprecationMessage = "Controllers are no longer used and deprecated in favor of the DMX Port System.", DisplayName = "ToController (ControllerReference)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|DMX")
	static UDMXEntityController* Conv_ControllerRefToObj(const FDMXEntityControllerRef& InControllerRef);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UFUNCTION(BlueprintPure, meta = (DisplayName = "ToFixtureType (FixtureTypeReference)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|DMX")
	static UDMXEntityFixtureType* Conv_FixtureTypeRefToObj(const FDMXEntityFixtureTypeRef& InFixtureTypeRef);

	UFUNCTION(BlueprintPure, meta = (DisplayName = "ToFixturePatch (FixturePatchReference)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|DMX")
	static UDMXEntityFixturePatch* Conv_FixturePatchRefToObj(const FDMXEntityFixturePatchRef& InFixturePatchRef);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS // FDMXEntityControllerRef used as ret val is deprecated. Leave it to that to raise deprecated warnings in C++.
	UFUNCTION(BlueprintPure, meta = (DeprecatedFunction, DeprecationMessage = "Controllers are no longer used and deprecated in favor of the DMX Port System.", DisplayName = "ToControllerRef (Controller)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|DMX")
	static FDMXEntityControllerRef Conv_ControllerObjToRef(UDMXEntityController* InController);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UFUNCTION(BlueprintPure, meta = (DisplayName = "ToFixtureTypeRef (FixtureType)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|DMX")
	static FDMXEntityFixtureTypeRef Conv_FixtureTypeObjToRef(UDMXEntityFixtureType* InFixtureType);

	UFUNCTION(BlueprintPure, meta = (DisplayName = "ToFixturePatchRef (FixturePatch)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|DMX")
	static FDMXEntityFixturePatchRef Conv_FixturePatchObjToRef(UDMXEntityFixturePatch* InFixturePatch);
};
