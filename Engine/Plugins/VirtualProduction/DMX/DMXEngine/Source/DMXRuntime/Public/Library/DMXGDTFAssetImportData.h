// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXTypes.h"

#include "CoreMinimal.h"
#include "EditorFramework/AssetImportData.h"

#include "DMXGDTFAssetImportData.generated.h"

class UDMXImportGDTF;


UCLASS()
class DMXRUNTIME_API UDMXGDTFAssetImportData 
	: public UAssetImportData
{
	GENERATED_BODY()

public:
	/** Returns the imported file path and name */
	const FString& GetFilePathAndName() const { return FilePathAndName; };

#if WITH_EDITOR
	/** Creates new asset GDTF Asset Import Data from the File. Returns the GDTF Asset Import Data or nullptr if not a valid GDTF File */
	void SetSourceFile(const FString& InFilePathAndName);
#endif // WITH_EDITOR 

	/** Returns the source data the asset was generated from */
	const TArray64<uint8>& GetRawSourceData() const { return RawSourceData.ByteArray; }

protected:
	//~ Begin UObject interface
	virtual void PostLoad() override;
	//~ End UObject interface

private:
	/** The imported file path and name */
	UPROPERTY()
	FString FilePathAndName;

	/** The raw GDTF zip file as byte array */
	UPROPERTY()
	FDMXByteArray64 RawSourceData;
};
