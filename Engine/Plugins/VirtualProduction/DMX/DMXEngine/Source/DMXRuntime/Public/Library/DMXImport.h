// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "DMXImport.generated.h"

namespace UE::DMX::DMXImport::Private
{
	template <typename EnumType> 
	UE_DEPRECATED(5.5, "Please refer to the new GDTF Description in UDMXDeviceType instead.")
	static EnumType GetEnumValueFromString(const FString& String)
	{
		return static_cast<EnumType>(StaticEnum<EnumType>()->GetValueByName(FName(*String)));
	}

	UE_DEPRECATED(5.5, "Please refer to the new GDTF Description in UDMXDeviceType instead.")
	DMXRUNTIME_API FDMXColorCIE ParseColorCIE(const FString& InColor);

	UE_DEPRECATED(5.5, "Please refer to the new GDTF Description in UDMXDeviceType instead.")
	DMXRUNTIME_API FMatrix ParseMatrix(FString&& InMatrixStr);
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXColorCIE is deprecated. Please refer to the new GDTF Description in UDMXDeviceType instead") FDMXColorCIE
{
	GENERATED_BODY()

	FDMXColorCIE()
		: X(0.f)
		, Y(0.f)
		, YY(0)
	{}

	UPROPERTY(EditAnywhere, Category = Color, meta = (ClampMin = "0", ClampMax = "1.0"))
	float X;

	UPROPERTY(EditAnywhere, Category = Color, meta = (ClampMin = "0", ClampMax = "1.0"))
	float Y;

	UPROPERTY(EditAnywhere, Category = Color, meta = (ClampMin = "0", ClampMax = "255"))
	uint8 YY;
};

UCLASS(BlueprintType, Blueprintable, Meta = (DeprecatedNode, DeprecationMessage = "This class is now deprecated in favor of a DMXEntityFixtureType based workflow. Please refer to members of DMXEntityFixtureType instead."))
class UE_DEPRECATED(5.5, "UDMXImportFixtureType is deprecated. Please refer to the new GDTF Description in UDMXDeviceType instead") DMXRUNTIME_API UDMXImportFixtureType
	: public UObject
{
	GENERATED_BODY()
};

UCLASS(BlueprintType, Blueprintable, Meta = (DeprecatedNode, DeprecationMessage = "This class is now deprecated in favor of a DMXEntityFixtureType based workflow. Please refer to members of DMXEntityFixtureType instead."))
class UE_DEPRECATED(5.5, "UDMXImportAttributeDefinitions is deprecated. Please refer to the new GDTF Description in UDMXDeviceType instead") DMXRUNTIME_API UDMXImportAttributeDefinitions
	: public UObject
{
	GENERATED_BODY()
};

UCLASS(BlueprintType, Blueprintable, Meta = (DeprecatedNode, DeprecationMessage = "This class is now deprecated in favor of a DMXEntityFixtureType based workflow. Please refer to members of DMXEntityFixtureType instead."))
class UE_DEPRECATED(5.5, "UDMXImportWheels is deprecated. Please refer to the new GDTF Description in UDMXDeviceType instead") DMXRUNTIME_API UDMXImportWheels
	: public UObject
{
	GENERATED_BODY()
};

UCLASS(BlueprintType, Blueprintable, Meta = (DeprecatedNode, DeprecationMessage = "This class is now deprecated in favor of a DMXEntityFixtureType based workflow. Please refer to members of DMXEntityFixtureType instead."))
class UE_DEPRECATED(5.5, "UDMXImportPhysicalDescriptions is deprecated. Please refer to the new GDTF Description in UDMXDeviceType instead") DMXRUNTIME_API UDMXImportPhysicalDescriptions
	: public UObject
{
	GENERATED_BODY()
};

UCLASS(BlueprintType, Blueprintable, Meta = (DeprecatedNode, DeprecationMessage = "This class is now deprecated in favor of a DMXEntityFixtureType based workflow. Please refer to members of DMXEntityFixtureType instead."))
class UE_DEPRECATED(5.5, "UDMXImportModels is deprecated. Please refer to the new GDTF Description in UDMXDeviceType instead") DMXRUNTIME_API UDMXImportModels
	: public UObject
{
	GENERATED_BODY()
};

UCLASS(BlueprintType, Blueprintable, Meta = (DeprecatedNode, DeprecationMessage = "This class is now deprecated in favor of a DMXEntityFixtureType based workflow. Please refer to members of DMXEntityFixtureType instead."))
class UE_DEPRECATED(5.5, "UDMXImportGeometries is deprecated. Please refer to the new GDTF Description in UDMXDeviceType instead") DMXRUNTIME_API UDMXImportGeometries
	: public UObject
{
	GENERATED_BODY()
};

class UE_DEPRECATED(5.5, "UDMXImportDMXModes is deprecated. Please refer to the new GDTF Description in UDMXDeviceType instead") UDMXImportDMXModes;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
UCLASS(BlueprintType, Blueprintable, Meta = (DeprecatedNode, DeprecationMessage = "This class is now deprecated in favor of a DMXEntityFixtureType based workflow. Please refer to members of DMXEntityFixtureType instead."))
class DMXRUNTIME_API UDMXImportDMXModes
	: public UObject
{
	GENERATED_BODY()

};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

UCLASS(BlueprintType, Blueprintable, Meta = (DeprecatedNode, DeprecationMessage = "This class is now deprecated in favor of a DMXEntityFixtureType based workflow. Please refer to members of DMXEntityFixtureType instead."))
class UE_DEPRECATED(5.5, "UDMXImportProtocols is deprecated. Please refer to the new GDTF Description in UDMXDeviceType instead") DMXRUNTIME_API UDMXImportProtocols
	: public UObject
{
	GENERATED_BODY()

};

UCLASS(BlueprintType, Blueprintable, abstract)
class DMXRUNTIME_API UDMXImport
	: public UObject
{
	GENERATED_BODY()

public:
	template <typename TType>
	UE_DEPRECATED(5.5, "Instead please use NewObject.")
	TType* CreateNewObject()
	{
		FName NewName = MakeUniqueObjectName(this, TType::StaticClass());
		return NewObject<TType>(this, TType::StaticClass(), NewName, RF_Public);
	}

public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UDMXImportFixtureType> FixtureType_DEPRECATED;

	UPROPERTY()
	TObjectPtr<UDMXImportAttributeDefinitions> AttributeDefinitions_DEPRECATED;

	UPROPERTY()
	TObjectPtr<UDMXImportWheels> Wheels_DEPRECATED;

	UPROPERTY()
	TObjectPtr<UDMXImportPhysicalDescriptions> PhysicalDescriptions_DEPRECATED;

	UPROPERTY()
	TObjectPtr<UDMXImportModels> Models_DEPRECATED;

	UPROPERTY()
	TObjectPtr<UDMXImportGeometries> Geometries_DEPRECATED;
#endif // WITH_EDITORONLY_DATA

	UPROPERTY()
	TObjectPtr<UDMXImportDMXModes> DMXModes_DEPRECATED;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UDMXImportProtocols> Protocols_DEPRECATED;
#endif // WITH_EDITORONLY_DATA
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};
