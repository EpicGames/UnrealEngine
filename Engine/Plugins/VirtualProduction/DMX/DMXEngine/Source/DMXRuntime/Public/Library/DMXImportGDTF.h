// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Library/DMXImport.h"

#include "DMXImportGDTF.generated.h"

class UDMXGDTF;
class UDMXGDTFAssetImportData;
class UTexture2D;
namespace UE::DMX::GDTF { class FDMXGDTFDescription; }


/** 
 * A GDTF imported into Unreal Engine. Note this object holds GDTF source data and may be memory heavy.
 * 
 * To access the GDTF description in lightweight fashion, please refer to UDMXGDTF (C++ only). 
 */
UCLASS(Blueprintable, BlueprintType)
class DMXRUNTIME_API UDMXImportGDTF
	: public UDMXImport
{
	GENERATED_BODY()

public:
	/** Constructor */
	UDMXImportGDTF();

	//~ Begin UObject interface
	virtual void PostLoad() override;
	//~ End UObject interface

	/** Loads the GDTF stored in this object */
	UDMXGDTF* LoadGDTF() const;

	/** DEPRECATED 5.4 */
	UE_DEPRECATED(5.5, "UDMXImportGDTF::GetDMXModes is deprecated in favor of UDMXGDTF. See also newly added UDMXImportGDTF::LoadGDTF to read out GDTF data of this asset.")
	PRAGMA_DISABLE_DEPRECATION_WARNINGS // Suppress deprecation warnings for the deprecated UDMXImportGDTFDMXModes return type
	UFUNCTION(BlueprintPure, Category = "DMXGDTF|Import Data", Meta = (DeprecatedFunction, DeprecationMessage = "This member is deprecated in favor of a DMX Entity Fixture Type based workflow. Please refer to members of DMX Entity Fixture Type instead"))
	class UDMXImportGDTFDMXModes* GetDMXModes() const { return Cast<UDMXImportGDTFDMXModes>(DMXModes_DEPRECATED); }
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITORONLY_DATA
	/** DEPRECATED 5.1 in favor of GDTFAssetImportData */
	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of GDTFAssetImportData, see UDMXImportGDTF::GetGDTFAssetImportData."))
	FString SourceFilename_DEPRECATED;
#endif

	/** Returns GDTF Asset Import Data for this GDTF */
	UDMXGDTFAssetImportData* GetGDTFAssetImportData() const { return GDTFAssetImportData; }

#if WITH_EDITOR
	/** Returns the Actor Class to spawn when spawned from a DMX Library */
	const TSoftClassPtr<AActor>& GetActorClass() const { return ActorClass; }
#endif

private:
#if WITH_EDITORONLY_DATA
	/** 
	 * The Actor Class that corresponds to this GDTF. Only Actors that implement the MVR Fixture Actor Interface can be used. 
	 *
	 * In the current Version this set as the Fixture Type Actor Class, when this GDTF is set for the Fixture Type. 
	 * 
	 * Can be left blank. If so, any Actor Class with the most matching Attributes will be spawned. 
	 */
	UPROPERTY(EditAnywhere, Category = "MVR", Meta = (MustImplement = "/Script/DMXFixtureActorInterface.DMXMVRFixtureActorInterface", AllowPrivateAccess = true))
	TSoftClassPtr<AActor> ActorClass;
#endif // WITH_EDITORONLY_DATA

	/** The Asset Import Data used to generate the GDTF asset or nullptr, if not generated from a GDTF file */
	UPROPERTY()
	TObjectPtr<UDMXGDTFAssetImportData> GDTFAssetImportData;
};


///////////////////////////////
//~ Begin Deprecated Members
// 

UENUM()
enum class UE_DEPRECATED(5.5, "FDMXImportGDTFType is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") EDMXImportGDTFType : uint8
{
    Multiply,
    Override
};

UENUM()
enum class UE_DEPRECATED(5.5, "FDMXImportGDTFSnap is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") EDMXImportGDTFSnap : uint8
{
    Yes,
    No,
    On,
    Off
};

UENUM()
enum class UE_DEPRECATED(5.5, "FDMXImportGDTFMaster is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") EDMXImportGDTFMaster : uint8
{
    None,
    Grand,
    Group
};

UENUM()
enum class UE_DEPRECATED(5.5, "FDMXImportGDTFDMXInvert is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") EDMXImportGDTFDMXInvert : uint8
{
    Yes,
    No
};

UENUM()
enum class UE_DEPRECATED(5.5, "FDMXImportGDTFLampType is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") EDMXImportGDTFLampType : uint8
{
    Discharge,
    Tungsten,
    Halogen,
    LED
};

UENUM()
enum class UE_DEPRECATED(5.5, "FDMXImportGDTFBeamType is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") EDMXImportGDTFBeamType : uint8
{
    Wash,
    Spot,
    None
};

UENUM()
enum class UE_DEPRECATED(5.5, "FDMXImportGDTFPrimitiveType is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") EDMXImportGDTFPrimitiveType : uint8
{
    Undefined,
    Cube,
    Cylinder,
    Sphere,
    Base,
    Yoke,
    Head,
    Scanner,
    Conventional,
    Pigtail
};

UENUM()
enum class UE_DEPRECATED(5.5, "FDMXImportGDTFPhysicalUnit is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") EDMXImportGDTFPhysicalUnit : uint8
{
    None,
    Percent,
    Length,
    Mass,
    Time,
    Temperature,
    LuminousIntensity,
    Angle,
    Force,
    Frequency,
    Current,
    Voltage,
    Power,
    Energy,
    Area,
    Volume,
    Speed,
    Acceleration,
    AngularSpeed,
    AngularAccc,
    WaveLength,
    ColorComponent
};

UENUM()
enum class UE_DEPRECATED(5.5, "FDMXImportGDTFMode is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") EDMXImportGDTFMode : uint8
{
    Custom,
    sRGB,
    ProPhoto,
    ANSI
};

UENUM()
enum class UE_DEPRECATED(5.5, "FDMXImportGDTFInterpolationTo is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") EDMXImportGDTFInterpolationTo : uint8
{
    Linear,
    Step,
    Log
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFActivationGroup is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") DMXRUNTIME_API FDMXImportGDTFActivationGroup
{
    GENERATED_BODY()

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FName Name;
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFFeature is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") DMXRUNTIME_API FDMXImportGDTFFeature
{
    GENERATED_BODY()

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FName Name;
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFFeatureGroup is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") DMXRUNTIME_API FDMXImportGDTFFeatureGroup
{
    GENERATED_BODY()

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FName Name;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FString Pretty;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    TArray<FDMXImportGDTFFeature> Features;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFAttribute is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") DMXRUNTIME_API FDMXImportGDTFAttribute
{
    GENERATED_BODY()

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FName Name;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FString Pretty;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFActivationGroup ActivationGroup;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFFeature Feature;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FString MainAttribute;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    EDMXImportGDTFPhysicalUnit PhysicalUnit = EDMXImportGDTFPhysicalUnit::None;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXColorCIE Color;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFFilter is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFFilter
{
    GENERATED_BODY()

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FName Name;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXColorCIE Color;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};


USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFWheelSlot is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFWheelSlot
{
    GENERATED_BODY()

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FName Name;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXColorCIE Color;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFFilter Filter;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

    UPROPERTY(VisibleAnywhere, Category = "Fixture Type")
    TObjectPtr<UTexture2D> MediaFileName = nullptr;
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFWheel is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFWheel
{
    GENERATED_BODY()

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FName Name;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    TArray<FDMXImportGDTFWheelSlot> Slots;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFMeasurementPoint is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFMeasurementPoint
{
    GENERATED_BODY()

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    float WaveLength = 0.f;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    float Energy = 0.f;
};


USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFMeasurement is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFMeasurement
{
    GENERATED_BODY()

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    float Physical = 0.f;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    float LuminousIntensity = 0.f;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    float Transmission = 0.f;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    EDMXImportGDTFInterpolationTo InterpolationTo = EDMXImportGDTFInterpolationTo::Linear;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    TArray<FDMXImportGDTFMeasurementPoint> MeasurementPoints;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFEmitter is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFEmitter
{
    GENERATED_BODY()

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FName Name;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXColorCIE Color;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    float DominantWaveLength = 0.f;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FString DiodePart;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFMeasurement Measurement;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFColorSpace is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFColorSpace
{
    GENERATED_BODY()

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    EDMXImportGDTFMode Mode = EDMXImportGDTFMode::sRGB;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FString Description;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXColorCIE Red;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXColorCIE Green;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXColorCIE Blue;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXColorCIE WhitePoint;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFDMXProfiles is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFDMXProfiles
{
    GENERATED_BODY()
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFCRIs is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFCRIs
{
    GENERATED_BODY()
};


USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFModel is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFModel
{
    GENERATED_BODY()

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FName Name;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    float Length = 0.f;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    float Width = 0.f;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    float Height = 0.f;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    EDMXImportGDTFPrimitiveType PrimitiveType = EDMXImportGDTFPrimitiveType::Undefined;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFGeometryBase is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFGeometryBase
{
	GENERATED_BODY()

	FDMXImportGDTFGeometryBase()
	{}

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FName Name;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FName Model;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FMatrix Position = FMatrix::Identity;
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFBeam is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFBeam
{
    GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FName Name;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FName Model;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FMatrix Position = FMatrix::Identity;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    EDMXImportGDTFLampType LampType = EDMXImportGDTFLampType::Discharge;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    float PowerConsumption = 0.f;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    float LuminousFlux = 0.f;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    float ColorTemperature = 0.f;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    float BeamAngle = 0.f;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    float FieldAngle = 0.f;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    float BeamRadius = 0.f;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    EDMXImportGDTFBeamType BeamType = EDMXImportGDTFBeamType::Wash;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    uint8 ColorRenderingIndex = 0;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFTypeAxis is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFTypeAxis
{
    GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FName Name;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FName Model;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FMatrix Position = FMatrix::Identity;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    TArray<FDMXImportGDTFBeam> Beams;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFGeneralAxis is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFGeneralAxis
{
    GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FName Name;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FName Model;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FMatrix Position = FMatrix::Identity;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    TArray<FDMXImportGDTFTypeAxis> Axis;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFTypeGeometry is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFTypeGeometry
{
    GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FName Name;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FName Model;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FMatrix Position = FMatrix::Identity;
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFFilterBeam is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFFilterBeam
{
    GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FName Name;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FName Model;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FMatrix Position = FMatrix::Identity;
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFFilterColor is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFFilterColor
{
    GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FName Name;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FName Model;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FMatrix Position = FMatrix::Identity;
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFFilterGobo is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFFilterGobo
{
    GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FName Name;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FName Model;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FMatrix Position = FMatrix::Identity;
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFFilterShaper is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFFilterShaper
{
    GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FName Name;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FName Model;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FMatrix Position = FMatrix::Identity;
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFBreak is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFBreak
{
    GENERATED_BODY()

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    int32 DMXOffset = 0;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    uint8 DMXBreak = 0;
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFGeometryReference is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFGeometryReference
{
    GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FName Name;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FName Model;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FMatrix Position = FMatrix::Identity;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    TArray<FDMXImportGDTFBreak> Breaks;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFGeneralGeometry is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFGeneralGeometry
{
    GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FName Name;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FName Model;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
	FMatrix Position = FMatrix::Identity;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFGeneralAxis Axis;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFTypeGeometry Geometry;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFFilterBeam FilterBeam;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFFilterColor FilterColor;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFFilterGobo FilterGobo;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFFilterShaper FilterShaper;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFGeometryReference GeometryReference;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFDMXValue is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") DMXRUNTIME_API FDMXImportGDTFDMXValue
{
    GENERATED_BODY()

    FDMXImportGDTFDMXValue()
        : Value(0)
        , ValueSize(1)
    {
    }

    FDMXImportGDTFDMXValue(const FString& InDMXValueStr);

    UPROPERTY(EditAnywhere, BlueprintReadOnly, BlueprintReadOnly, Category = "DMX")
    int32 Value;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, BlueprintReadOnly, Category = "DMX")
    uint8 ValueSize;
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFChannelSet is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFChannelSet
{
    GENERATED_BODY()

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FString Name;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFDMXValue DMXFrom;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    float PhysicalFrom = 0.f;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    float PhysicalTo = 0.f;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    int32 WheelSlotIndex = 0;
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFChannelFunction is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFChannelFunction
{
    GENERATED_BODY()

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FName Name;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFAttribute Attribute;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FString OriginalAttribute;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFDMXValue DMXFrom;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFDMXValue DMXValue;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    float PhysicalFrom = 0.f;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    float PhysicalTo = 0.f;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    float RealFade = 0.f;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFWheel Wheel;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFEmitter Emitter;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFFilter Filter;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    EDMXImportGDTFDMXInvert DMXInvert = EDMXImportGDTFDMXInvert::No;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
		
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FString ModeMaster;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFDMXValue ModeFrom;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFDMXValue ModeTo;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    TArray<FDMXImportGDTFChannelSet> ChannelSets;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFLogicalChannel is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFLogicalChannel
{
    GENERATED_BODY()

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFAttribute Attribute;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    EDMXImportGDTFSnap Snap = EDMXImportGDTFSnap::No;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    EDMXImportGDTFMaster Master = EDMXImportGDTFMaster::None;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    float MibFade = 0.f;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    float DMXChangeTimeLimit = 0.f;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFChannelFunction ChannelFunction;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFDMXChannel is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") DMXRUNTIME_API FDMXImportGDTFDMXChannel
{
    GENERATED_BODY()

	/** Parses the offset of the channel. Returns false if no valid offset is specified */
    bool ParseOffset(const FString& InOffsetStr);

    UPROPERTY(EditAnywhere, BlueprintReadOnly, BlueprintReadOnly, Category = "DMX")
    int32 DMXBreak = 0;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, BlueprintReadOnly, Category = "DMX")
    TArray<int32> Offset;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFDMXValue Default;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFDMXValue Highlight;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

    UPROPERTY(EditAnywhere, BlueprintReadOnly, BlueprintReadOnly, Category = "DMX")
    FName Geometry;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFLogicalChannel LogicalChannel;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFRelation is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFRelation
{
    GENERATED_BODY()

    UPROPERTY(EditAnywhere, BlueprintReadOnly, BlueprintReadOnly, Category = "DMX")
    FString Name;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, BlueprintReadOnly, Category = "DMX")
    FString Master;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, BlueprintReadOnly, Category = "DMX")
    FString Follower;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, BlueprintReadOnly, Category = "DMX")
    EDMXImportGDTFType Type = EDMXImportGDTFType::Multiply;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFFTMacro is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFFTMacro
{
    GENERATED_BODY()

    UPROPERTY(EditAnywhere, BlueprintReadOnly, BlueprintReadOnly, Category = "DMX")
    FName Name;
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5, "FDMXImportGDTFDMXMode is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") FDMXImportGDTFDMXMode
{
    GENERATED_BODY()

    UPROPERTY(EditAnywhere, BlueprintReadOnly, BlueprintReadOnly, Category = "DMX")
    FName Name;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, BlueprintReadOnly, Category = "DMX")
    FName Geometry;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    UPROPERTY(EditAnywhere, BlueprintReadOnly, BlueprintReadOnly, Category = "DMX")
    TArray<FDMXImportGDTFDMXChannel> DMXChannels;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, BlueprintReadOnly, Category = "DMX")
    TArray<FDMXImportGDTFRelation> Relations;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, BlueprintReadOnly, Category = "DMX")
    TArray<FDMXImportGDTFFTMacro> FTMacros;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

class UE_DEPRECATED(5.5, "UDMXImportGDTFFixtureType is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") UDMXImportGDTFFixtureType;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
UCLASS(BlueprintType, Blueprintable, Meta = (DeprecatedNode, DeprecationMessage = "This class is now deprecated in favor of a DMXEntityFixtureType based workflow. Please refer to members of DMXEntityFixtureType instead."))
class DMXRUNTIME_API UDMXImportGDTFFixtureType
    : public UDMXImportFixtureType
{
    GENERATED_BODY()

PRAGMA_ENABLE_DEPRECATION_WARNINGS

public:
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Fixture Type")
    FName Name;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Fixture Type")
    FString ShortName;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Fixture Type")
    FString LongName;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Fixture Type")
    FString Manufacturer;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Fixture Type")
    FString Description;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Fixture Type")
    FString FixtureTypeID;

    UPROPERTY(VisibleAnywhere, Category = "Fixture Type")
    TObjectPtr<UTexture2D> Thumbnail = nullptr;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Fixture Type")
    FString RefFT;
};

class UE_DEPRECATED(5.5, "UDMXImportGDTFAttributeDefinitions is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") UDMXImportGDTFAttributeDefinitions;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
UCLASS(BlueprintType, Blueprintable, Meta = (DeprecatedNode, DeprecationMessage = "This class is now deprecated in favor of a DMXEntityFixtureType based workflow. Please refer to members of DMXEntityFixtureType instead."))
class DMXRUNTIME_API UDMXImportGDTFAttributeDefinitions
    : public UDMXImportAttributeDefinitions
{
    GENERATED_BODY()

public:
    bool FindFeature(const FString& InQuery, FDMXImportGDTFFeature& OutFeature) const;

    bool FindAtributeByName(const FName& InName, FDMXImportGDTFAttribute& OutAttribute) const;

public:
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    TArray<FDMXImportGDTFActivationGroup> ActivationGroups;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    TArray<FDMXImportGDTFFeatureGroup> FeatureGroups;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    TArray<FDMXImportGDTFAttribute> Attributes;

PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

class UE_DEPRECATED(5.5, "UDMXImportGDTFWheels is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") UDMXImportGDTFWheels;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
UCLASS(BlueprintType, Blueprintable, Meta = (DeprecatedNode, DeprecationMessage = "This class is now deprecated in favor of a DMXEntityFixtureType based workflow. Please refer to members of DMXEntityFixtureType instead."))
class DMXRUNTIME_API UDMXImportGDTFWheels
    : public UDMXImportWheels
{
    GENERATED_BODY()

public:
    bool FindWeelByName(const FName& InName, FDMXImportGDTFWheel& OutWheel) const;

public:
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    TArray<FDMXImportGDTFWheel> Wheels;

PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

class UE_DEPRECATED(5.5, "UDMXImportGDTFPhysicalDescriptions is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") UDMXImportGDTFPhysicalDescription;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
UCLASS(BlueprintType, Blueprintable, Meta = (DeprecatedNode, DeprecationMessage = "This class is now deprecated in favor of a DMXEntityFixtureType based workflow. Please refer to members of DMXEntityFixtureType instead."))
class DMXRUNTIME_API UDMXImportGDTFPhysicalDescriptions
    : public UDMXImportPhysicalDescriptions
{
    GENERATED_BODY()

public:
    bool FindEmitterByName(const FName& InName, FDMXImportGDTFEmitter& OutEmitter) const;

public:
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    TArray<FDMXImportGDTFEmitter> Emitters;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFColorSpace ColorSpace;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFDMXProfiles DMXProfiles;

    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    FDMXImportGDTFCRIs CRIs;

PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

class UE_DEPRECATED(5.5, "UDMXImportGDTFModels is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") UDMXImportGDTFModels;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
UCLASS(BlueprintType, Blueprintable, Meta = (DeprecatedNode, DeprecationMessage = "This class is now deprecated in favor of a DMXEntityFixtureType based workflow. Please refer to members of DMXEntityFixtureType instead."))
class DMXRUNTIME_API UDMXImportGDTFModels
    : public UDMXImportModels
{
    GENERATED_BODY()

public:
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    TArray<FDMXImportGDTFModel> Models;

PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

class UE_DEPRECATED(5.5, "UDMXImportGDTFGeometries is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") UDMXImportGDTFGeometries;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
UCLASS(BlueprintType, Blueprintable, Meta = (DeprecatedNode, DeprecationMessage = "This class is now deprecated in favor of a DMXEntityFixtureType based workflow. Please refer to members of DMXEntityFixtureType instead."))
class DMXRUNTIME_API UDMXImportGDTFGeometries
    : public UDMXImportGeometries
{
    GENERATED_BODY()

public:
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    TArray<FDMXImportGDTFGeneralGeometry> GeneralGeometry;

PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

class UE_DEPRECATED(5.5, "UDMXImportGDTFDMXModes is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") UDMXImportGDTFDMXModes;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
UCLASS(BlueprintType, Blueprintable, Meta = (DeprecatedNode, DeprecationMessage = "This class is now deprecated in favor of a DMXEntityFixtureType based workflow. Please refer to members of DMXEntityFixtureType instead."))
class DMXRUNTIME_API UDMXImportGDTFDMXModes
    : public UDMXImportDMXModes
{
    GENERATED_BODY()

public:
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    TArray<FDMXImportGDTFDMXMode> DMXModes;

	UFUNCTION(BlueprintPure, Category = "DMXGDTF|Import Data")
	TArray<FDMXImportGDTFChannelFunction> GetDMXChannelFunctions(const FDMXImportGDTFDMXMode& InMode);

PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

class UE_DEPRECATED(5.5, "UDMXImportGDTFProtocols is deprecated in favor of its corresponding type in the DMXGDTF module. See also UDMXImportGDTF::LoadGDTF.") UDMXImportGDTFProtocols;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
UCLASS(BlueprintType, Blueprintable, Meta = (DeprecatedNode, DeprecationMessage = "This class is now deprecated in favor of a DMXEntityFixtureType based workflow. Please refer to members of DMXEntityFixtureType instead."))
class DMXRUNTIME_API UDMXImportGDTFProtocols
    : public UDMXImportProtocols
{
    GENERATED_BODY()

PRAGMA_ENABLE_DEPRECATION_WARNINGS

public:
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX")
    TArray<FName> Protocols;
};

//~ End Deprecated Types
///////////////////////////////

