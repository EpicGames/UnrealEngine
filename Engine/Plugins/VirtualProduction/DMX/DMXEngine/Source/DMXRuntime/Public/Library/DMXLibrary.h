// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Library/DMXObjectBase.h"

#include "IO/DMXInputPortReference.h"
#include "IO/DMXOutputPortReference.h"
#include "Library/DMXEntity.h"

#include "Templates/SubclassOf.h"
#include "Misc/Guid.h"

#include "DMXLibrary.generated.h"

class UDMXEntity;
class UDMXEntityFixturePatch;
class UDMXEntityFixtureType;
class UDMXMVRGeneralSceneDescription;


/** Custom struct of in put and output port references for custom details customization with an enabled state */
USTRUCT(BlueprintType)
struct DMXRUNTIME_API FDMXLibraryPortReferences
{
	GENERATED_BODY()

public:
	/** Map of input port references of a Library */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, NonTransactional, Category = "DMX", Meta = (DisplayName = "Input Ports"))
	TArray<FDMXInputPortReference> InputPortReferences;

	/** Output ports of the Library of a Library */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, NonTransactional, Category = "DMX", Meta = (DisplayName = "Output Ports"))
	TArray<FDMXOutputPortReference> OutputPortReferences;
};


DECLARE_MULTICAST_DELEGATE_TwoParams(FDMXOnEntityArrayChangedDelegate, class UDMXLibrary*, TArray<UDMXEntity*>);

/** DEPRECATED 5.0 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnEntitiesUpdated_DEPRECATED, class UDMXLibrary*);

/** 
 * DMX Library.
 * 
 * Holds Fixture Types and Fixture Patches that can be used to send and receive DMX.
 */
UCLASS(BlueprintType, Blueprintable, Config = DMXEditor, AutoExpandCategories = DMX)
class DMXRUNTIME_API UDMXLibrary
	: public UDMXObjectBase
{
	GENERATED_BODY()

	/** Friend DMXEntity so they can register and unregister themselves with the Library */
	friend class UDMXEntity;

public:
	/** Constructor */
	UDMXLibrary();

protected:
	// ~Begin UObject Interface
	virtual void PostInitProperties() override;
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
	virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	// ~End UObject Interface

	/** Registers an Entity with this Library */
	void RegisterEntity(UDMXEntity* Entity);

	/** Unregisters an Entity from this Library */
	void UnregisterEntity(UDMXEntity* Entity);

public:
	UE_DEPRECATED(5.0, "Deprecated in favor of new UDMXEntityFixtureType::CreateFixtureType and UDMXEntityFixturePatch::CreateFixturePatch that support creating patches in blueprints.")
	UDMXEntity* GetOrCreateEntityObject(const FString& InName, TSubclassOf<UDMXEntity> DMXEntityClass);

	/**
	 * Returns an Entity named InSearchName. If none exists, return nullptr
	 * This is not very reliable since Entities of different types can have the same name.
	 */
	UDMXEntity* FindEntity(const FString& InSearchName) const;

	/**
	 * Returns an Entity with the passed in ID.
	 * The most reliable method since Entities ID are always unique.
	 */
	UDMXEntity* FindEntity(const FGuid& Id);

	/**
	 * The finds the index of an existing entity.
	 * @return The index of the entity or INDEX_NONE if not found or the Entity doesn't belong to this Library.
	 */
	int32 FindEntityIndex(UDMXEntity* InEntity) const;

	/** Adds an Entity to the library */
	UE_DEPRECATED(5.0, "Entites no longer can be added or removed explicitly.")
	void AddEntity(UDMXEntity* InEntity);

	/** Move an Entity to a specific index in the Entities Array. */
	void SetEntityIndex(UDMXEntity* InEntity, const int32 NewIndex);

	/** Removes an Entity from this DMX Library searching it by name. */
	UE_DEPRECATED(5.0, "Entites no longer can be added or removed explicitly.")
	void RemoveEntity(const FString& EntityName);
	
	/** Removes an Entity from this DMX Library. */
	UE_DEPRECATED(5.0, "Entites no longer can be added or removed explicitly.")
	void RemoveEntity(UDMXEntity* InEntity);

	/** Empties this DMX Library array of Entities */
	UE_DEPRECATED(5.0, "Entites no longer can be added or removed explicitly.")
	void RemoveAllEntities();

	/** Returns all Entities in this DMX Library */
	const TArray<UDMXEntity*>& GetEntities() const;

	/** Get an array with entities from the specified UClass, but not typecast. */
	TArray<UDMXEntity*> GetEntitiesOfType(TSubclassOf<UDMXEntity> InEntityClass) const;

	/** Get an array of Entities from the specified template type, already cast. */
	template <typename EntityType>
	TArray<EntityType*> GetEntitiesTypeCast() const
	{
		TArray<EntityType*> TypeCastEntities;
		for (UDMXEntity* Entity : Entities)
		{
			if (EntityType* EntityCast = Cast<EntityType>(Entity))
			{
				TypeCastEntities.Add(EntityCast);
			}
		}
		return TypeCastEntities;
	}

	/**
	 * Calls Predicate on all Entities of the template type.
	 * This is the version without break.
	 */
	template <typename EntityType>
	void ForEachEntityOfType(TFunction<void(EntityType*)> Predicate) const
	{
		for (UDMXEntity* Entity : Entities)
		{
			if (EntityType* EntityCast = Cast<EntityType>(Entity))
			{
				Predicate(EntityCast);
			}
		}
	}

	/**
	 * Calls Predicate on all Entities of the template type
	 * Return false from the predicate to break the iteration loop or true to keep iterating.
	 */
	template <typename EntityType>
	void ForEachEntityOfTypeWithBreak(TFunction<bool(EntityType*)> Predicate) const
	{
		for (UDMXEntity* Entity : Entities)
		{
			if (EntityType* EntityCast = Cast<EntityType>(Entity))
			{
				if (!Predicate(EntityCast)) 
				{ 
					break; 
				}
			}
		}
	}

	/**
	 * Calls Predicate on all Entities of the passed in type.
	 * This is the version without break.
	 * Casting is left to the caller.
	 */
	void ForEachEntityOfType(TSubclassOf<UDMXEntity> InEntityClass, TFunction<void(UDMXEntity*)> Predicate) const;

	/**
	 * Calls Predicate on all Entities of the passed in type.
	 * Return false from the predicate to break the iteration loop or true to keep iterating.
	 * Casting is left to the caller.
	 */
	void ForEachEntityOfTypeWithBreak(TSubclassOf<UDMXEntity> InEntityClass, TFunction<bool(UDMXEntity*)> Predicate) const;

	/** Returns a delegate that is broadcast when entities were added to the library  */
	static FDMXOnEntityArrayChangedDelegate& GetOnEntitiesAdded();

	/** Returns a delegate that is broadcast when entities were removed from the library  */
	static FDMXOnEntityArrayChangedDelegate& GetOnEntitiesRemoved();

	/** */
	UE_DEPRECATED(5.0, "Deprecated in favor of the more expressive UDMXLibrary::GetOnEntitiesAdded() and UDMXLibrary::GetOnEntitiesRemoved() that also forwards the added resp. removed entities")
	FOnEntitiesUpdated_DEPRECATED& GetOnEntitiesUpdated();

	/** Returns all local Universe IDs in Ports */
	TSet<int32> GetAllLocalUniversesIDsInPorts() const;

	/** Returns the input ports */
	const TSet<FDMXInputPortSharedRef>& GetInputPorts() const { return InputPorts; }

	/** Returns the output ports */
	const TSet<FDMXOutputPortSharedRef>& GetOutputPorts() const { return OutputPorts; }

	/** Returns all ports as a set, slower than GetInputPorts and GetOutputPorts. */
	TSet<FDMXPortSharedRef> GenerateAllPortsSet() const;

	/** Updates the ports from what's set in the Input and Output Port References arrays */
	void UpdatePorts();

	/** 
	 * Sets the MVR General Scene Description of the Library. 
	 * Note, this will not add any patches that occur in the new General Scene Description. 
	 * Note, this will not remove any patches that do not occur in the new General Scene Description. 
	 */
	void SetMVRGeneralSceneDescription(UDMXMVRGeneralSceneDescription* NewGeneralSceneDescription);

#if WITH_EDITOR
	/** Updates the General Scene Description to reflect the Library */
	UDMXMVRGeneralSceneDescription* UpdateGeneralSceneDescription();
#endif

	/** 
	 * Returns the General Scene Description of the Library. 
	 * May not always represent the state of the library. 
	 * Use WriteGeneralSceneDescription to update it to get an exact version.
	 */
	UDMXMVRGeneralSceneDescription* GetLazyGeneralSceneDescription() const { return GeneralSceneDescription; }

	/** Returns the Entity that was last added to the Library */
	TWeakObjectPtr<UDMXEntity> GetLastAddedEntity() const { return LastAddedEntity; }

#if WITH_EDITOR
	static FName GetPortReferencesPropertyName() { return GET_MEMBER_NAME_CHECKED(UDMXLibrary, PortReferences); }
	static FName GetEntitiesPropertyName() { return GET_MEMBER_NAME_CHECKED(UDMXLibrary, Entities); }
	static FName GetGeneralSceneDescriptionPropertyName() { return GET_MEMBER_NAME_CHECKED(UDMXLibrary, GeneralSceneDescription); }
#endif // WITH_EDITOR

protected:
#if WITH_EDITOR
	/** 
	 * Upgrades libraries that use controllers (before 4.27) to use ports instead (from 4.27 on). 
	 * Creates corresponding ports if they do not exist yet. 
	 */
	void UpgradeFromControllersToPorts();
#endif // WITH_EDITOR

	/** Ports of the Library */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, NonTransactional, Category = "DMX", Meta = (ShowOnlyInnerProperties))
	FDMXLibraryPortReferences PortReferences;

private:
	/** All Fixture Types and Fixture Patches in the Library */
	UPROPERTY()
	TArray<TObjectPtr<UDMXEntity>> Entities;

	/** The General Scene Description of this Library */
	UPROPERTY()
	TObjectPtr<UDMXMVRGeneralSceneDescription> GeneralSceneDescription;

	/** The entity that was added last to the Library */
	TWeakObjectPtr<UDMXEntity> LastAddedEntity;

	/** The input ports available to the library, according to the InputPortReferences array */
	TSet<FDMXInputPortSharedRef> InputPorts;

	/** The output ports available to the library, according to the OutputPortReferences array */
	TSet<FDMXOutputPortSharedRef> OutputPorts;

	/** Delegate broadcast when Entities were added */
	static FDMXOnEntityArrayChangedDelegate OnEntitiesAddedDelegate;

	/** Delegate broadcast when Entities were removed */
	static FDMXOnEntityArrayChangedDelegate OnEntitiesRemovedDelegate;

	/** DEPRECATED 5.0 */
	FOnEntitiesUpdated_DEPRECATED OnEntitiesUpdated_DEPRECATED;
};
