// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "DMXObjectBase.generated.h"

UCLASS(abstract)
class DMXRUNTIME_API UDMXObjectBase
	: public UObject
{
	GENERATED_BODY()
};
