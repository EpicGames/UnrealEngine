// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXTypes.h"

#include "CoreMinimal.h"
#include "EditorFramework/AssetImportData.h"

#include "DMXMVRAssetImportData.generated.h"

class FDMXXmlFile;
class UDMXMVRGeneralSceneDescription;


UCLASS()
class DMXRUNTIME_API UDMXMVRAssetImportData 
	: public UAssetImportData
{
	GENERATED_BODY()

public:
	/** Returns the imported file path and name */
	const FString& GetFilePathAndName() const { return FilePathAndName; };

#if WITH_EDITOR
	/** Sets the source file and initializes members from it */
	void SetSourceFile(const FString& InFilePathAndName);

	/** Returns the raw source file as byte array, as it was imported */
	const TArray64<uint8>& GetRawSourceData() const { return RawSourceData.ByteArray; }
#endif // WITH_EDITOR 

protected:
	//~ Begin UObject interface
	virtual void PostLoad() override;
	//~ End UObject interface

private:
	/** The imported file path and name */
	UPROPERTY()
	FString FilePathAndName;

#if WITH_EDITORONLY_DATA
	/** The raw source file as byte array, as it was imported */
	UPROPERTY()
	FDMXByteArray64 RawSourceData;
#endif // WITH_EDITORONLY_DATA
};
