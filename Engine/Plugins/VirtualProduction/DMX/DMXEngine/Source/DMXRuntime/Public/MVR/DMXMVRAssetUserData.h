// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/AssetUserData.h"

#include "DMXMVRAssetUserData.generated.h"


/** Asset user data for Actors in an MVR Scene */
UCLASS()
class DMXRUNTIME_API UDMXMVRAssetUserData
	: public UAssetUserData
{
	GENERATED_BODY()

public:
	/** Returns MVR Asset User Data for the Actor, or nullptr if there is no asset user data */
	static UDMXMVRAssetUserData* GetMVRAssetUserData(const AActor& Actor);

	/** Returns the Value in Meta data given a Key, or an empty String if the Key doesn't exist */
	static FString GetMVRAssetUserDataValueForKey(const AActor& Actor, const FName Key);

	/** Sets the Value for Key in Meta Data. Creates the Key if it doesn't exist. */
	static bool SetMVRAssetUserDataValueForKey(AActor& InOutActor, FName InKey, const FString& InValue);

	/** MVR Meta Data for the Actor */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "MVR")
	TMap<FName, FString> MetaData;

	/** The key for the MVR UUID meta data */
	static const FName MVRFixtureUUIDMetaDataKey;

	/** The key for the Fixture Patch meta data */
	static const FName FixturePatchMetaDataKey;
};
