// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Templates/SharedPointer.h"
#include "UObject/GCObject.h"
#include "UObject/StrongObjectPtr.h"

class UDMXEntityFixturePatch;

class AActor;
struct FAssetData;


class DMXRUNTIME_API FDMXMVRFixtureActorLibrary
	: public FGCObject
	, public TSharedFromThis<FDMXMVRFixtureActorLibrary>
{
public:
	/** Constructor */
	FDMXMVRFixtureActorLibrary();

	/** Returns the Actor that can represent the patch best */
	UClass* FindMostAppropriateActorClassForPatch(const UDMXEntityFixturePatch* Patch) const;

protected:
	//~ Begin FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FDMXMVRFixtureActorLibrary");
	}
	//~ End FGCObject interface

private:
	/** Array of actors that implement the MVR Actor Interface */
	TArray<TObjectPtr<AActor>> MVRActors;
};
