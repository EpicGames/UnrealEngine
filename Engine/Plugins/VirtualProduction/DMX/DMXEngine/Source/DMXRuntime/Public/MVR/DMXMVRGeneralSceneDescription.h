// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Misc/Optional.h"
#include "Serialization/Archive.h"

#include "DMXMVRGeneralSceneDescription.generated.h"

class FXmlFile;
class UDMXComponent;
class UDMXEntityFixturePatch;
class UDMXLibrary;
class UDMXMVRAssetImportData;
class UDMXMVRClassNode;
class UDMXMVRRootNode;
class UDMXMVRFixtureNode;

namespace UE::DMX
{
	struct FDMXMVRGeneralSceneDescriptionWorldParams
	{
		UWorld* World = nullptr;

		/** If checked, exports patches that are not in use in the level */
		bool bExportPatchesNotPresentInWorld = false;

		/** If checked, creates individual MVR Fixtures for each Fixture that uses the same Patch in the Level. */
		bool bCreateMultiPatchFixtures = false;

		/** If checked, exports the fixtures with transforms as in the current level */
		bool bUseTransformsFromLevel = true;
	};
}


/** MVR General Scene Description Object */
UCLASS()
class DMXRUNTIME_API UDMXMVRGeneralSceneDescription
	: public UObject
{
	GENERATED_BODY()

	using FDMXMVRGeneralSceneDescriptionWorldParams = UE::DMX::FDMXMVRGeneralSceneDescriptionWorldParams;

public:
	/** Constructor */
	UDMXMVRGeneralSceneDescription();

	/** Gets the MVR Fixture Nodes in this General Scene Description */
	void GetFixtureNodes(TArray<UDMXMVRFixtureNode*>& OutFixtureNodes) const;

	/** Returns the Fixture Node corresponding to the UUID, or nullptr if it cannot be found */
	UDMXMVRFixtureNode* FindFixtureNode(const FGuid& FixtureUUID) const;

#if WITH_EDITOR
	/** Creates an MVR General Scene Description from an Xml File */
	static UDMXMVRGeneralSceneDescription* CreateFromXmlFile(TSharedRef<FXmlFile> GeneralSceneDescriptionXml, UObject* Outer, FName Name = NAME_None, EObjectFlags Flags = RF_NoFlags);

	/** Creates an MVR General Scene Description from a DMX Library */
	static UDMXMVRGeneralSceneDescription* CreateFromDMXLibrary(const UDMXLibrary& DMXLibrary, UObject* Outer, FName Name = NAME_None, EObjectFlags Flags = RF_NoFlags);

	/** DEPRECATED 5.5 */
	UE_DEPRECATED(5.5, "Changed to WriteDMXLibrary for better readability and consitency with new members.")
	void WriteDMXLibraryToGeneralSceneDescription(const UDMXLibrary& DMXLibrary);
	
	/**
	 * Writes the Library to the General Scene Description, effectively removing inexisting and adding
	 * new MVR Fixtures, according to what MVR Fixture UUIDs the Fixture Patches of the Library contain.
	 * 
	 * If world params are specified, considers these.
	 */
	void WriteDMXLibrary(const UDMXLibrary& DMXLibrary, FDMXMVRGeneralSceneDescriptionWorldParams WorldParams = FDMXMVRGeneralSceneDescriptionWorldParams());

	/** Removes a fixture node from the General Scene Description. */
	void RemoveFixtureNode(const FGuid& FixtureUUID);

	/** Returns true if an Xml File can be created. Gives a reason if no MVR can be exported */
	bool CanCreateXmlFile(FText& OutReason) const;

	/** Creates an General Scene Description Xml File from this. */
	TSharedPtr<FXmlFile> CreateXmlFile() const;

	/** Returns MVR Asset Import Data for this asset */
	UDMXMVRAssetImportData* GetMVRAssetImportData() const { return MVRAssetImportData; }

private:
	/** 	
	 * Writes the Fixture Patch to the General Scene Description.
	 * 
	 * Optionally a transform can be specified for the patch.
	 * 
	 * Optionally a MultiPatch UUID can be passed so the patch is added as a multi patch.
	 * When a MultiPatch UUID is provided, the parent with related UUID is expected to exist already.
	 */
	void WriteFixturePatch(const UDMXEntityFixturePatch& FixturePatch, const FTransform& Transform, const FGuid& MultiPatchUUID = FGuid());

	/** Makes sure the node has a unique MVR UUID and Fixture ID */
	void  SanetizeFixtureNode(UDMXMVRFixtureNode& FixtureNode);

	/** Parses a General Scene Description Xml File. Only ment to be used for initialization (ensured) */
	[[nodiscard]] bool ParseGeneralSceneDescriptionXml(const TSharedRef<FXmlFile>& GeneralSceneDescriptionXml);

	/** Returns a map of DMX components with their actor from a world */
	TMap<const UDMXComponent*, const AActor*> GetDMXComponentToActorMap(const UWorld* World) const;
#endif

	/** Returns the Fixture IDs currently in use, as number, sorted from lowest to highest */
	TArray<int32> GetNumericalFixtureIDsInUse(const UDMXLibrary& DMXLibrary);

	/** The Root Node of the General Scene Description */
	UPROPERTY()
	TObjectPtr<UDMXMVRRootNode> RootNode;

#if WITH_EDITORONLY_DATA
	/** Import Data for this asset */
	UPROPERTY()
	TObjectPtr<UDMXMVRAssetImportData> MVRAssetImportData;
#endif
};
