// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "GameFramework/Actor.h"

#include "DMXMVRSceneActor.generated.h"

class FDMXMVRFixtureActorLibrary;
class UFactory;
class UDMXEntityFixturePatch;
class UDMXEntityFixtureType;
class UDMXImportGDTF;
class UDMXLibrary;
class UDMXMVRSceneComponent;

USTRUCT(BlueprintType)
struct DMXRUNTIME_API FDMXMVRSceneFixtureTypeToActorClassPair
{
	GENERATED_BODY()

	/** The Fixture Type of the Actor. */
	UPROPERTY(VisibleAnywhere, Category = "MVR")
	TSoftObjectPtr<UDMXEntityFixtureType> FixtureType;

	/** The Actor Class that should be or was spawned. Only Actors that implement the MVR Fixture Actor Interface can be used. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = MVR, NoClear, meta = (MustImplement = "/Script/DMXFixtureActorInterface.DMXMVRFixtureActorInterface"))
	TSoftClassPtr<AActor> ActorClass;
};

struct UE_DEPRECATED(5.5, "Instead use FDMXMVRSceneFixtureTypeToActorClassPair") FDMXMVRSceneGDTFToActorClassPair;
USTRUCT(BlueprintType, meta = (Deprecated = "Deprecated 5.5. Instead use DMXMVRSceneFixtureTypeToActorClassPair"))
struct DMXRUNTIME_API FDMXMVRSceneGDTFToActorClassPair
{
	GENERATED_BODY()

	/** The GDTF of the Actor. */
	UPROPERTY(VisibleAnywhere, Category = "MVR")
	TSoftObjectPtr<UDMXImportGDTF> GDTF;
	
	/** The Actor Class that should be or was spawned. Only Actors that implement the MVR Fixture Actor Interface can be used. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = MVR, NoClear, meta = (MustImplement = "/Script/DMXFixtureActorInterface.DMXMVRFixtureActorInterface"))
	TSoftClassPtr<AActor> ActorClass;
};

UCLASS(NotBlueprintable)
class DMXRUNTIME_API ADMXMVRSceneActor
	: public AActor
{
	GENERATED_BODY()

public:
	ADMXMVRSceneActor();
	~ADMXMVRSceneActor();

	//~ Begin AActor interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
	virtual void PostRegisterAllComponents() override;
#if WITH_EDITOR
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End AActor interface

	/** Sets the dmx library for this MVR actor. Should only be called once, further calls will have no effect and hit an ensure condition */
	void SetDMXLibrary(UDMXLibrary* NewDMXLibrary);

	/** Refreshes the scene from the DMX Library */
	void RefreshFromDMXLibrary();

	/** Returns an array of actors that are currently spawned in the scene for the specified Fixture Type */
	TArray<AActor*> GetActorsSpawnedForFixtureType(const UDMXEntityFixtureType* FixtureType) const;

	/** Returns an array of actors that are currently spawned in the scene for specified GDTF */
	UE_DEPRECATED(5.5, "MVRSceneActor now spawns actors for all Fixture Types, not only those that have a GDTF asset. Instead now use GetActorsSpawnedForFixtureType")
	TArray<AActor*> GetActorsSpawnedForGDTF(const UDMXImportGDTF* GDTF) const;
#endif

	/** Returns the DMX Library of this MVR Scene Actor */
	UDMXLibrary* GetDMXLibrary() const { return DMXLibrary; }

	/** Returns the MVR UUID To Related Actor Map */
	const TArray<TSoftObjectPtr<AActor>>& GetRelatedActors() const { return RelatedActors; };

#if WITH_EDITORONLY_DATA
	/** If checked, respawns Fixture Actors deleted from the MVR Scene */
	UPROPERTY(EditAnywhere, Category = "MVR", meta = (DisplayName = "Respawn Deleted Actors"))
	bool bRespawnDeletedActorsOnRefresh = false;

	/** If checked, updates transforms from the DMX Library */
	UPROPERTY(EditAnywhere, Category = "MVR", meta = (DisplayName = "Update Transforms"))
	bool bUpdateTransformsOnRefresh = false;
#endif

#if WITH_EDITOR
	// Property name getters
	static FName GetDMXLibraryPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(ADMXMVRSceneActor, DMXLibrary); }
	static FName GetRelatedAcctorsPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(ADMXMVRSceneActor, RelatedActors); }
	static FName GetFixtureTypeToActorClassesPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(ADMXMVRSceneActor, FixtureTypeToActorClasses); }
	static FName GetMVRSceneRootPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(ADMXMVRSceneActor, MVRSceneRoot); }

	UE_DEPRECATED(5.5, "GDTFToDefaultActorClasses is no longer used as all Fixture Types can generate GDTFs. To upgrade please refer to FixtureTypeToActorClasses.")
	static FName GetGDTFToDefaultActorClassesPropertyNameChecked() { 
		PRAGMA_DISABLE_DEPRECATION_WARNINGS	
		return GET_MEMBER_NAME_CHECKED(ADMXMVRSceneActor, GDTFToDefaultActorClasses_DEPRECATED); 
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif

private:
	/** Set MVR UUIDs for related Actors */
	void EnsureMVRUUIDsForRelatedActors();

#if WITH_EDITOR
	/** Updates the FixtureTypeToDefaultActorClasses map so it only contains those entries present both in the DMX Library and the MVR Scene */
	void UpdateFixtureTypeToDefaultActorClasses(const TSharedRef<FDMXMVRFixtureActorLibrary>& MVRFixtureActorLibrary);
#endif // WITH_EDITOR

	/** Gets the Fixture Patch of an Actor, or nullptr if the Fixture Patch cannot be retrieved anymore */
	UDMXEntityFixturePatch* GetFixturePatchFromActor(AActor* Actor) const;

	/** Sets the Fixture Patch on an Actor if possible or fails silently */
	void SetFixturePatchOnActor(AActor* Actor, UDMXEntityFixturePatch* FixturePatch);

#if WITH_EDITOR
	/** Called when a sub-level is loaded */
	void OnMapChange(uint32 MapChangeFlags);

	/** Called when an actor got deleted in editor */
	void OnActorDeleted(AActor* DeletedActor);

	/** Called when an asset was imported */
	void OnAssetPostImport(UFactory* InFactory, UObject* ActorAdded);

	/** Replaces related Actors that use the Default Actor Class for the Fixture Type with an instance of the new Default Actor Class */
	void HandleDefaultActorClassForFixtureTypeChanged();

	/** Spawns an MVR Actor in this Scene. Returns the newly spawned Actor, or nullptr if no Actor could be spawned. */
	AActor* SpawnMVRActor(const TSubclassOf<AActor>& ActorClass, UDMXEntityFixturePatch * FixturePatch, const FTransform & Transform, AActor * Template = nullptr);

	/** Replaces an MVR Actor in this Scene with another. Returns the newly spawned Actor, or nullptr if no Actor could be spawned. */
	AActor* ReplaceMVRActor(AActor* ActorToReplace, const TSubclassOf<AActor>& ClassOfNewActor);

	/** Upgrades this Actor created before 5.5 to use the FixtureTypeToActorClasses array instead of the deprecated GDTFToDefaultActorClasses array */
	void UpgradeToFixtureTypeToActorClasses();
#endif // WITH_EDITOR

	/** The DMX Library this Scene Actor uses */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "MVR", meta = (AllowPrivateAccess = true))
	TObjectPtr<UDMXLibrary> DMXLibrary;

	/** The actors that created along with this scene */
	UPROPERTY(VisibleAnywhere, Category = "Actor", AdvancedDisplay, meta = (AllowPrivateAccess = true))
	TArray<TSoftObjectPtr<AActor>> RelatedActors;
	
	/** DEPRECATED 5.5, instead use FixtureTypeToActorClasses */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY()
	TArray<FDMXMVRSceneGDTFToActorClassPair> GDTFToDefaultActorClasses_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** The actor class that is spawned for a specific Fixture Type by default (can be overriden per MVR UUID, see below) */
	UPROPERTY(EditAnywhere, Category = "MVR", meta = (DispayName = "Fixture Type to Spawned Actor"))
	TArray<FDMXMVRSceneFixtureTypeToActorClassPair> FixtureTypeToActorClasses;

	/** Array holding MVR Fixture UUIDs that were explicitly removed from the scene */
	UPROPERTY()
	TArray<FGuid> DeletedMVRFixtureUUIDs;

#if WITH_EDITORONLY_DATA
	/** The FixtureTypeToActorClasses cached of PreEditChange, to find changes. */
	UPROPERTY(Transient)
	TArray<FDMXMVRSceneFixtureTypeToActorClassPair> FixtureTypeToActorClasses_PreEditChange;
#endif

	/** The root component to which all actors are attached initially */
	UPROPERTY(VisibleAnywhere, Category = "Actor", AdvancedDisplay, meta = (AllowPrivateAccess = true))
	TObjectPtr<USceneComponent> MVRSceneRoot;
};
