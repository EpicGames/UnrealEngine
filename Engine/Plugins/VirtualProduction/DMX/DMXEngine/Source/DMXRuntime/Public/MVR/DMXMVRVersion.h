// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


/** Version of the MVR standard currently being used by the engine */
struct DMXRUNTIME_API FDMXMVRVersion
{
	static const int32 MajorVersion = 1;

	static const int32 MinorVersion = 5;

	static const FString GetMajorVersionAsString();
	static const FString GetMinorVersionAsString();

	/** Returns true if the provided version is the latest version */
	static bool IsLatest(int32 InMajorVersion, int32 InMinorVersion)
	{
		return
			InMajorVersion == MajorVersion &&
			InMinorVersion == MinorVersion;
	}
};
