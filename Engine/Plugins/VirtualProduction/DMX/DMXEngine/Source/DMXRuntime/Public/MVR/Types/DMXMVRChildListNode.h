// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DMXMVRChildListNode.generated.h"

class UDMXMVRFixtureNode;
class UDMXMVRParametricObjectNodeBase;

class FXmlNode;


/** This node defines a generic graphical object. */
UCLASS()
class DMXRUNTIME_API UDMXMVRChildListNode
	: public UObject
{
	GENERATED_BODY()

public:
	/** Initializes the Child List Node and its Children from a Child List Xml Node. Does not merge, clears previous entries. */
	void InitializeFromChildListXmlNode(const FXmlNode& ChildListXmlNode);

	/** Creates a DMX Xml Node as per MVR standard in parent, or logs warnings if no compliant Node can be created. */
	void CreateXmlNodeInParent(FXmlNode& ParentNode) const;

	/** Returns the Fixture Nodes contained in this Layer. Note, does not clear the array. */
	void GetFixtureNodes(TArray<UDMXMVRFixtureNode*>& OutFixtureNodes) const;

	/** Returns true if the Parametric Object Node is contained in this Child List */
	bool Contains(UDMXMVRParametricObjectNodeBase* ParametricObjectNode);

	/** Returns a pointer ot the Parametric Object Node that corresponds to the UUID, or nullptr if there is no corresponding Parametric Object Node */
	TObjectPtr<UDMXMVRParametricObjectNodeBase>* FindParametricObjectNodeByUUID(const FGuid& UUID);

	/** Creates a Parametric Object Node as a Child in the Child List Node */
	template<typename ParametricObjectNodeClass, typename TEnableIf<TIsDerivedFrom<ParametricObjectNodeClass, UDMXMVRParametricObjectNodeBase>::Value, bool>::Type = true>
	ParametricObjectNodeClass* CreateParametricObject()
	{
		ParametricObjectNodeClass* NewParametricObject = NewObject<ParametricObjectNodeClass>(this);
		ParametricObjectNodes.Add(NewParametricObject);

		return NewParametricObject;
	}

	/** Removes a Parametric Object Child from the Child List. Returns true if the Parametric Object Node was contained and Removed */
	bool RemoveParametricObject(UDMXMVRParametricObjectNodeBase* ParametricObjectNodeToRemove);

	/** Returns the Parametric Object Node Children of this Child List Node */
	const TArray<TObjectPtr<UDMXMVRParametricObjectNodeBase>>& GetParametricObjectNodes() const { return ParametricObjectNodes; };

private:
	/** A list of geometrical representation objects that are part of the object. */
	UPROPERTY()
	TArray<TObjectPtr<UDMXMVRParametricObjectNodeBase>> ParametricObjectNodes;
};
