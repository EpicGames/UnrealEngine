// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXMVRParametricObjectNodeBase.h"

#include "DMXOptionalTypes.h"

#include "CoreMinimal.h"
#include "Misc/Optional.h"
#include "Serialization/Archive.h"

#include "DMXMVRFixtureNode.generated.h"

class FXmlNode;


/** The container for DMX Addresses for this fixture. */
USTRUCT()
struct FDMXMVRFixtureAddresses
{
	GENERATED_BODY()

	FDMXMVRFixtureAddresses()
		: Universe(1)
		, Address(1)
	{}

	FDMXMVRFixtureAddresses(int32 InUniverse, int32 InAddress)
		: Universe(InUniverse)
		, Address(InAddress)
	{}

	/** Creates a DMX Xml Node as per MVR standard in parent, or logs warnings if no compliant Node can be created. */
	void CreateXmlNodeInParent(FXmlNode& ParentNode) const;

	UPROPERTY()
	int32 Universe;

	UPROPERTY()
	int32 Address;
};

/** Defines a Mapping */
USTRUCT()
struct FDMXMVRFixtureMapping
{
	GENERATED_BODY()

	/** Creates a DMX Xml Node as per MVR standard in parent, or logs warnings if no compliant Node can be created. */
	void CreateXmlNodeInParent(FXmlNode& ParentNode) const;

	/** The unique identifier of the MappingDefinition node that will be the source of the mapping. */
	FGuid LinkedDef;

	/** The offset in pixels in x direction from top left corner of the source that will be used for the mapped object. */
	TOptional<int32> UX;

	/** The offset in pixels in y direction from top left corner of the source that will be used for the mapped object. */
	TOptional<int32> UY;

	/** The size in pixels in x direction from top left of the starting point. */
	TOptional<int32> OX;

	/** The size in pixels in y direction from top left of the starting point. */
	TOptional<int32> OY;

	/** The rotation around the middle point of the defined rectangle in degree. Positive direction is counter cock wise. */
	TOptional<float> RZ;
};

/** The Gobo used for the fixture. The image ressource must apply to the GDTF standard. */
USTRUCT()
struct FDMXMVRFixtureGobo
{
	GENERATED_BODY()

	/** The node value is the Gobo used for the fixture. The image ressource must apply to the GDTF standard. Use a FileName to specify. */
	UPROPERTY()
	FString Value;

	/** The roation of the Gobo in degree. */
	UPROPERTY()
	FDMXOptionalFloat Rotation;

	friend FArchive& operator<<(FArchive& Ar, FDMXMVRFixtureGobo& Gobo)
	{
		Ar << Gobo.Value;
		Ar << Gobo.Rotation;

		return Ar;
	}
};

USTRUCT()
struct FDMXOptionalMVRFixtureGobo
{
	GENERATED_BODY()

	/** Creates a DMX Xml Node as per MVR standard in parent, or logs warnings if no compliant Node can be created. */
	void CreateXmlNodeInParent(FXmlNode& ParentNode) const;

	TOptional<FDMXMVRFixtureGobo> Value;

	bool Serialize(FArchive& Ar)
	{
		Ar << Value;
		return true;
	}

	friend bool operator==(const FDMXOptionalMVRFixtureGobo& A, const FDMXOptionalMVRFixtureGobo& B)
	{
		const bool bBothSet = A.Value.IsSet() && B.Value.IsSet();
		if (bBothSet)
		{
			return 
				A.Value.GetValue().Value == B.Value.GetValue().Value &&
				A.Value.GetValue().Rotation == B.Value.GetValue().Rotation;
		}
		else
		{
			return A.Value.IsSet() == B.Value.IsSet();
		}
	}

	friend bool operator!=(const FDMXOptionalMVRFixtureGobo& A, const FDMXOptionalMVRFixtureGobo& B)
	{
		return !(A == B);
	}
};

template<>
struct TStructOpsTypeTraits<FDMXOptionalMVRFixtureGobo>
	: public TStructOpsTypeTraitsBase2<FDMXOptionalMVRFixtureGobo>
{
	enum
	{
		WithSerializer = true,
		WithIdenticalViaEquality = true
	};
};


/** This node defines a light fixture object. */
UCLASS()
class DMXRUNTIME_API UDMXMVRFixtureNode
	: public UDMXMVRParametricObjectNodeBase
{
	GENERATED_BODY()

public:
	/** Initializes the Fixture Node and its Children from a Fixture Xml Node. Does not merge, clears previous entries. */
	bool InitializeFromFixtureXmlNode(const FXmlNode& FixtureXmlNode);

	//~Begin UDMXMVRParametricObjectNodeBase Interface
	virtual void CreateXmlNodeInParent(FXmlNode& ParentNode) const override;
	//~End UDMXMVRParametricObjectNodeBase Interface

	/** Returns true if the data is sufficient to fullfill the mvr standard. If bLogInvalidReason is set to true, logs the reason for the fixture being invalid */
	bool IsValid(bool bLogInvalidReason) const;

	/** The name of the object. */
	UPROPERTY()
	FString Name;

	/** 
	 * The unique identifier of the parent multipatch fixture. When this value is set, you may not define a FixtureID or CustomID for this fixture. 
	 * The FixtureID and CustomID from the object defined as multi parent also applies to this object. 
	 */
	UPROPERTY()
	FDMXOptionalGuid MultiPatch;

	/** The Class the object belongs to. */
	UPROPERTY()
	FDMXOptionalGuid Classing;

	/** The name of the file containing the GDTF information for this light fixture. */
	UPROPERTY()
	FString GDTFSpec;

	/** The name of the used DMX mode. This has to match the name of a DMXMode in the GDTF file. */
	UPROPERTY()
	FString GDTFMode;

	/** A focus point reference that this lighting fixture aims at if this reference exists. */
	UPROPERTY()
	FDMXOptionalGuid Focus;

	/** Defines if a Object cast Shadow. */
	UPROPERTY()
	FDMXOptionalBool CastShadow;

	/** A position reference that this lighting fixture belongs to if this reference exists. */
	UPROPERTY()
	FDMXOptionalGuid Position;

	/** The Fixture Id of the lighting fixture. This is the short name of the fixture. */
	UPROPERTY()
	FString FixtureID;

	/** The unit number of the lighting fixture in a position. */
	UPROPERTY()
	int32 UnitNumber = 0;
	
	/** Sets the Universe ID, in UE form (Index 0 is ID 1) */
	void SetUniverseID(int32 UniverseID) { Addresses.Universe = UniverseID - 1; }

	/** Returns the Universe ID, in UE form (Index 0 is ID 1) */
	int32 GetUniverseID() const { return Addresses.Universe + 1; }

	/** Sets the Address of the MVR Fixture */
	void SetStartingChannel(int32 Address) { Addresses.Address = Address; }

	/** Returns the Address of the MVR Fixture */
	int32 GetStartingChannel() const { return Addresses.Address; }

private:
	/** The container for DMX Addresses for this fixture. */
	UPROPERTY()
	FDMXMVRFixtureAddresses Addresses;

public:
	/** A color assigned to a fixture. If it is not defined, there is no color for the fixture. */
	UPROPERTY()
	FDMXOptionalColorCIE1931 CIEColor;

	/** The Fixture Type ID is a value that can be used as a short name of the Fixture Type. This does not have to be unique. The default value is 0. */
	UPROPERTY()
	FDMXOptionalInt32 FixtureTypeId;

	/** The Custom ID is a value that can be used as a short name of the Fixture Instance. This does not have to be unique. The default value is 0. */
	UPROPERTY()
	FDMXOptionalInt32 CustomId;

	/** The container for Mappings for this fixture. */
	UPROPERTY()
	TArray<FDMXMVRFixtureMapping> Mappings;

	/** The Gobo used for the fixture. The image ressource must apply to the GDTF standard. */
	UPROPERTY()
	FDMXOptionalMVRFixtureGobo Gobo;
};
