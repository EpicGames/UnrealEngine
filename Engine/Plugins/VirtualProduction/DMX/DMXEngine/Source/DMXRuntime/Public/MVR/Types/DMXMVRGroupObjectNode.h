// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXMVRParametricObjectNodeBase.h"

#include "DMXOptionalTypes.h"

#include "CoreMinimal.h"

#include "DMXMVRGroupObjectNode.generated.h"

class UDMXMVRChildListNode;
class UDMXMVRFixtureNode;
class UDMXMVRParametricObjectNodeBase;

class FXmlNode;


/** This node defines logical group of objects. The child objects are located inside a local coordinate system. */
UCLASS()
class DMXRUNTIME_API UDMXMVRGroupObjectNode
	: public UDMXMVRParametricObjectNodeBase
{
	GENERATED_BODY()

public:	
	/** Constructor */
	UDMXMVRGroupObjectNode();

	/** Initializes the Child List Node and its Children from a Group Object Xml Node. Does not merge, clears previous entries. */
	void InitializeFromGroupObjectXmlNode(const FXmlNode& GroupObjectXmlNode);

	//~Begin UDMXMVRParametricObjectNodeBase Interface
	virtual void CreateXmlNodeInParent(FXmlNode& ParentNode) const override;
	//~End UDMXMVRParametricObjectNodeBase Interface

	/** Returns the Fixture Nodes contained in this Layer. Note, does not clear the array. */
	void GetFixtureNodes(TArray<UDMXMVRFixtureNode*>& OutFixtureNodes) const;

	/** Returns true if the Parametric Object Node is contained in this Group Object Node */
	bool Contains(UDMXMVRParametricObjectNodeBase* ParametricObjectNode);

	/** Returns a pointer ot the Parametric Object Node that corresponds to the UUID, or nullptr if there is no corresponding Parametric Object Node */
	TObjectPtr<UDMXMVRParametricObjectNodeBase>* FindParametricObjectNodeByUUID(const FGuid& InUUID);

	/** Removes the Parametric Object Node from Child List. Returns true if the Node was removed. */
	bool RemoveParametricObjectNode(UDMXMVRParametricObjectNodeBase* ParametricObjectNode);

	/** Returns a list of graphic objects that are part of the group. */
	UDMXMVRChildListNode* GetChildListNode() const { return ChildListNode; }

	/**
	 * The name of the GroupObject 
	 */
	UPROPERTY()
	FDMXOptionalString Name;

private:
	/** A list of graphic objects that are part of the group. */
	UPROPERTY()
	TObjectPtr<UDMXMVRChildListNode> ChildListNode;
};
