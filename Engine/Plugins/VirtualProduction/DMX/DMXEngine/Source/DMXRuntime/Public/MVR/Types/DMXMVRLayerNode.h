// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXOptionalTypes.h"

#include "CoreMinimal.h"

#include "DMXMVRLayerNode.generated.h"

class UDMXMVRChildListNode;
class UDMXMVRFixtureNode;
class UDMXMVRParametricObjectNodeBase;

class FXmlNode;


/** A Layer node in the Layers Node of a Scene. */
UCLASS()
class DMXRUNTIME_API UDMXMVRLayerNode
	: public UObject
{
	GENERATED_BODY()

public:
	/** Constructor */
	UDMXMVRLayerNode();

	/** Initializes the Layer Node and its Children from a Layer Xml Node. Does not merge, clears previous entries. */
	void InitializeFromLayerXmlNode(const FXmlNode& LayerXmlNode);

	/** Creates a DMX Xml Node as per MVR standard in parent, or logs warnings if no compliant Node can be created. */
	void CreateXmlNodeInParent(FXmlNode& ParentNode) const;

	/** Returns the Fixture Nodes contained in this Layer. Note, does not clear the array. */
	void GetFixtureNodes(TArray<UDMXMVRFixtureNode*>& OutFixtureNodes) const;

	/** Returns true if the Parametric Object Node is contained in this layer */
	bool Contains(UDMXMVRParametricObjectNodeBase* ParametricObjectNode);

	/** Returns a pointer ot the Parametric Object Node that corresponds to the UUID, or nullptr if there is no corresponding Parametric Object Node */
	TObjectPtr<UDMXMVRParametricObjectNodeBase>* FindParametricObjectNodeByUUID(const FGuid& InUUID) const;

	/** Returns a list of graphic objects that are part of the layer. */
	UDMXMVRChildListNode* GetChildListNode() const { return ChildListNode; }

	/** The unique identifier of the object. */
	UPROPERTY()
	FGuid UUID;

	/** The name of the object */
	UPROPERTY()
	FDMXOptionalString Name;

	/**
	 * The transformation matrix that defines the location and orientation of this the layer inside its global coordinate space. 
	 * This effectively defines local coordinate space for the objects inside. The Matrix of the Layer is only allowed to have an elevation, but no rotation.
	 * 
	 * UE specific: Note, we do not use an FVector since the standard clearly specifies a transformation matrix, even tho it also forbids any rotation.
	 * While it makes the code more readable and future proof, the user of this struct is responsible to follow the standard directly.
	 */
	UPROPERTY()
	FDMXOptionalTransform Matrix;

private:
	/** A list of graphic objects that are part of the group. */
	UPROPERTY()
	TObjectPtr<UDMXMVRChildListNode> ChildListNode;
};
