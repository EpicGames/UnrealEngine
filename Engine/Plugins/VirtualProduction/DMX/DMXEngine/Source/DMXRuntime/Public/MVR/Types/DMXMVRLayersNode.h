// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "XmlNode.h"

#include "DMXMVRLayersNode.generated.h"

class UDMXMVRFixtureNode;
class UDMXMVRLayerNode;
class UDMXMVRParametricObjectNodeBase;

class FXmlNode;


/** This node defines a list of layers inside the scene. The layer is a container of graphical objects defining a local coordinate system. */
UCLASS()
class DMXRUNTIME_API UDMXMVRLayersNode
	: public UObject
{
	GENERATED_BODY()

public:
	/** Initializes the Layers Node and its Children from a Layers Xml Node. Does not merge, clears previous entries. */
	void InitializeFromLayersXmlNode(const FXmlNode& LayersXmlNode);

	/** Creates a DMX Xml Node as per MVR standard in parent, or logs warnings if no compliant Node can be created. */
	void CreateXmlNodeInParent(FXmlNode& ParentNode) const;

	/** Returns the Fixture Nodes contained in this Layer. Note, does not clear the array. */
	void GetFixtureNodes(TArray<UDMXMVRFixtureNode*>& OutFixtureNodes) const;

	/** Returns a pointer ot the Parametric Object Node that corresponds to the UUID, or nullptr if there is no corresponding Parametric Object Node */
	TObjectPtr<UDMXMVRParametricObjectNodeBase>* FindParametricObjectNodeByUUID(const FGuid& UUID) const;

	/** Creates a new Layer. Returns the newly created Layer */
	UDMXMVRLayerNode* CreateLayer();

	/** Removes a Layer. Ensures the Layer is contained in the Layers of this Node */
	void RemoveLayer(UDMXMVRLayerNode* LayerNodeToRemove);

	/** Returns the Layers of this Node */
	const TArray<UDMXMVRLayerNode*>& GetLayerNodes() const { return LayerNodes; };

private:
	/** A[n array of] layer representation[s]. */
	UPROPERTY()
	TArray<TObjectPtr<UDMXMVRLayerNode>> LayerNodes;
};
