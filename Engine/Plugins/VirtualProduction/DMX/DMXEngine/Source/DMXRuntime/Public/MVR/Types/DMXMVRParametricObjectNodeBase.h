// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXOptionalTypes.h"

#include "CoreMinimal.h"
#include "Misc/Guid.h"

#include "DMXMVRParametricObjectNodeBase.generated.h"

class UDMXMVRGroupObjectNode;
class UDMXMVRLayerNode;

class FXmlNode;



/** UE specific: The base class for all Children in a Child List Node, whereas the standard refers to them as 'Parametric Objects'. */
UCLASS(Abstract)
class DMXRUNTIME_API UDMXMVRParametricObjectNodeBase
	: public UObject
{
	GENERATED_BODY()

public:
	/** Constructor */
	UDMXMVRParametricObjectNodeBase();

	/** Creates a DMX Xml Node as per MVR standard in parent, or logs warnings if no compliant Node can be created. */
	virtual void CreateXmlNodeInParent(FXmlNode& ParentNode) const PURE_VIRTUAL(UDMXMVRParametricObjectNodeBase::CreateXmlNodeInParent, return;);

	/** Returns the Layer this Node resides in */
	UDMXMVRLayerNode* GetLayer();

	/** Returns the Layer this Node resides in */
	const UDMXMVRLayerNode* GetLayer() const;

	/** Returns the Group this Node resides in, or null if not in a Group */
	UDMXMVRGroupObjectNode* GetGroup();

	/** Returns the Group this Node resides in, or null if not in a Group */
	const UDMXMVRGroupObjectNode* GetGroup() const;

	/** Gets the Transform in the absolute coordinate system */
	FTransform GetTransformAbsolute() const;

	/** Sets the Transform in the absolute coordinate system */
	void SetTransformAbsolute(FTransform NewTransform);

	/** The unique identifier of the object. */
	UPROPERTY()
	FGuid UUID;

	/** The location of the object inside the parent coordinate system. */
	UPROPERTY(VisibleAnywhere, Category = "DMX")
	FDMXOptionalTransform Matrix;
};
