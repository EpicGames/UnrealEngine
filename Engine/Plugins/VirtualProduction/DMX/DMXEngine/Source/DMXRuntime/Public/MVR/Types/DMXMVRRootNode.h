// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DMXMVRRootNode.generated.h"

class UDMXMVRChildListNode;
class UDMXMVRFixtureNode;
class UDMXMVRLayerNode;
class UDMXMVRParametricObjectNodeBase;
class UDMXMVRSceneNode;
class UDMXMVRUserDataNode;

class FXmlFile;


/** 
 * The root node of an MVR General Scene Description as typically held in an MVR's GeneralSceneDescription.xml. 
 * 
 * UE specific: Note, while properties follow the standard closely, only children relevant to the Engine are implemented.
 */
UCLASS()
class DMXRUNTIME_API UDMXMVRRootNode
	: public UObject
{
	GENERATED_BODY()

public:
	/** Constructor */
	UDMXMVRRootNode();

	/** 
	 * Initializes the Object and its children from a General Scene Description Xml File. 
	 * Does not merge, clears previous entries. 
	 * Returns false if not a valid MVR General Scene Description Xml 
	 */
	[[nodiscard]] bool InitializeFromGeneralSceneDescriptionXml(const TSharedRef<FXmlFile>& GeneralSceneDescriptionXml);

	/** Creates a DMX Xml Node as per MVR standard, returns nullptr if the Xml File cannot be created. */
	TSharedPtr<FXmlFile> CreateXmlFile();

	/** Returns the Fixture Nodes contained in this Layer. Note, does not clear the array. */
	void GetFixtureNodes(TArray<UDMXMVRFixtureNode*>& OutFixtureNodes) const;

	/** Returns the first Child List Node in the Scene */
	UDMXMVRChildListNode& GetOrCreateFirstChildListNode();

	/** Returns a pointer ot the Parametric Object Node that corresponds to the UUID, or nullptr if there is no corresponding Parametric Object Node */
	TObjectPtr<UDMXMVRParametricObjectNodeBase>* FindParametricObjectNodeByUUID(const FGuid& UUID) const;

	/** Removes the Parametric Object Node from the General Scene Description. Returns true if the Node was removed. */
	bool RemoveParametricObjectNode(UDMXMVRParametricObjectNodeBase* ParametricObjectNode);

	/** Returns the node that contains information about the mvr scene */
	UDMXMVRSceneNode* GetSceneNode() const { return SceneNode; }

private:
	/** This node contains a collection of user data nodes defined and used by provider applications if required. */
	UPROPERTY()
	TObjectPtr<UDMXMVRUserDataNode> UserDataNode;

	/** This node contains information about the scene. */
	UPROPERTY()
	TObjectPtr<UDMXMVRSceneNode> SceneNode;
};
