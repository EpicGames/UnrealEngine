// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DMXMVRSceneNode.generated.h"

class UDMXMVRLayersNode;

class FXmlNode;


/**
 * This node contains information about the scene.
 *
 * UE Specific: Note, while properties follow the standard closely, only properties relevant to the Engine are implemented.
 */
UCLASS()
class DMXRUNTIME_API UDMXMVRSceneNode
	: public UObject
{
	GENERATED_BODY()

public:
	/** Constructor */
	UDMXMVRSceneNode();

	/** Initializes the Scene Node andd its Children from a Scene Xml Node. Does not merge, clears previous entries. */
	void InitializeFromSceneXmlNode(const FXmlNode& SceneXmlNode);

	/** Creates a DMX Xml Node as per MVR standard in parent, or logs warnings if no compliant Node can be created. */
	void CreateXmlNodeInParent(FXmlNode& ParentNode) const;

	/** Returns the node that defines a list of layers inside the scene. The layer is a container of graphical objects defining a local coordinate system. */
	UDMXMVRLayersNode* GetLayersNode() const { return LayersNode; }

private:
	/** This node defines a list of layers inside the scene. The layer is a container of graphical objects defining a local coordinate system. */
	UPROPERTY()
	TObjectPtr<UDMXMVRLayersNode> LayersNode;
};
