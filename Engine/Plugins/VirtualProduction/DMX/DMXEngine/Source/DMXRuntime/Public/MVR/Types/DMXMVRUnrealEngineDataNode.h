// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DMXMVRUnrealEngineDataNode.generated.h"

class FXmlNode;


/** 
 * This node contains a collection of data specified by the provider application.
 * UE sepecific: Used to add UE specific meta data
 */
UCLASS()
class DMXRUNTIME_API UDMXMVRUnrealEngineDataNode
	: public UObject
{
	GENERATED_BODY()

public:
	/** Creates a DMX Xml Node as per MVR standard in parent, or logs warnings if no compliant Node can be created. */
	void CreateXmlNodeInParent(FXmlNode& ParentNode) const;

	/** The Provider Name when the Data node is provided by Unreal Engine */
	static constexpr TCHAR ProviderNameUnrealEngine[] = TEXT("UnrealEngine");
};
