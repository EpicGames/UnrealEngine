// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DMXMVRUserDataNode.generated.h"

class UDMXMVRDataNode;
class UDMXMVRUnrealEngineDataNode;

class FXmlNode;


/** This node contains a collection of user data nodes defined and used by provider applications if required. */
UCLASS()
class DMXRUNTIME_API UDMXMVRUserDataNode
	: public UObject
{
	GENERATED_BODY()

public:
	/** Constructor */
	UDMXMVRUserDataNode();

	/** Creates a DMX Xml Node as per MVR standard in parent, or logs warnings if no compliant Node can be created. */
	void CreateXmlNodeInParent(FXmlNode& ParentNode) const;
	
private:
	/** UE Specific: The UE's proprietary data node */
	UPROPERTY()
	TObjectPtr<UDMXMVRUnrealEngineDataNode> UnrealEngineDataNode;
};
