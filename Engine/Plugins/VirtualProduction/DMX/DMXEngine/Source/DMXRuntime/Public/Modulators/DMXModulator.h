// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXAttribute.h"
#include "DMXTypes.h"

#include "CoreMinimal.h"
#include "UObject/Object.h"

#include "DMXModulator.generated.h"

class UDMXEntityFixturePatch;

/** 
 * Base class for custom modulators. Override Modulate and ModulateMatrix functions in the blueprints to implement functionality.
 * Input maps hold all attribute values of the patch. Output Maps can be freely defined, but Attributes not present in the patch will be ignored.
 */
UCLASS(BlueprintType, Blueprintable, EditInlineNew, Abstract)
class DMXRUNTIME_API UDMXModulator
	: public UObject

{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintNativeEvent, Category = "DMX")
	void Modulate(UDMXEntityFixturePatch* FixturePatch, const TMap<FDMXAttributeName, float>& InNormalizedAttributeValues, TMap<FDMXAttributeName, float>& OutNormalizedAttributeValues);
	virtual void Modulate_Implementation(UDMXEntityFixturePatch* FixturePatch, const TMap<FDMXAttributeName, float>& InNormalizedAttributeValues, TMap<FDMXAttributeName, float>& OutNormalizedAttributeValues) {};

	UFUNCTION(BlueprintNativeEvent, Category = "DMX")
	void ModulateMatrix(UDMXEntityFixturePatch* FixturePatch, const TArray<FDMXNormalizedAttributeValueMap>& InNormalizedMatrixAttributeValues, TArray<FDMXNormalizedAttributeValueMap>& OutNormalizedMatrixAttributeValues);
	virtual void ModulateMatrix_Implementation(UDMXEntityFixturePatch* FixturePatch, const TArray<FDMXNormalizedAttributeValueMap>& InNormalizedMatrixAttributeValues, TArray<FDMXNormalizedAttributeValueMap>& OutNormalizedMatrixAttributeValues) {};
};
