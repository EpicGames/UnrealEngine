// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolCommon.h"
#include "MovieSceneSection.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Library/DMXEntityReference.h"

#include "MovieSceneDMXLibrarySection.generated.h"

enum class EDMXFixtureSignalFormat : uint8;
class UDMXEntityFixturePatch;

USTRUCT()
struct FDMXFixtureFunctionChannel
{
	GENERATED_BODY()

	/** Attribute Name of the function */
	UPROPERTY()
	FName AttributeName = NAME_None;

	/** Function animation curve. */
	UPROPERTY()
	FMovieSceneFloatChannel Channel;

	/** 
	 * For Cell Functions the coordinate of the cell
	 * For common functions -1.
	 */
	UPROPERTY()
	FIntPoint CellCoordinate = FIntPoint(-1, -1);

	/** Default value to use when this Function is disabled in the track. */
	UPROPERTY()
	uint32 DefaultValue = 0;

	/**
	 * Whether or not to display this Function in the Patch's group
	 * If false, the Function's default value is sent to DMX protocols.
	 */
	UPROPERTY()
	bool bEnabled = true;

	/** True if the function is a cell function */
	bool IsCellFunction() const { return CellCoordinate.X != -1 && CellCoordinate.Y != -1; }
};

USTRUCT()
struct FDMXFixturePatchChannel
{
	GENERATED_BODY()

	/** The outer library of the channel */
	UPROPERTY()
	TObjectPtr<UDMXLibrary> DMXLibrary;

	/** Points to the Fixture Patch */
	UPROPERTY()
	FDMXEntityFixturePatchRef Reference;

	/** Fixture function float channels */
	UPROPERTY()
	TArray<FDMXFixtureFunctionChannel> FunctionChannels;

	/**
	 * Allows Sequencer to animate the fixture using a mode and not have it break
	 * simply by the user changing the active mode in the DMX Library.
	 */
	UPROPERTY()
	int32 ActiveMode = INDEX_NONE;

	void SetFixturePatch(UDMXEntityFixturePatch* InPatch);

	/** Makes sure the number of Float Channels matches the number of functions in the selected Patch mode */
	void UpdateNumberOfChannels(bool bResetDefaultValues = false);
};

/**
* Cached info of fixture function channels. Exists to streamline performance.
*
* Without this class, data for all tracks would have to be prepared each tick, leading to significant overhead.
*
* Besides caching values, the instance deduces how the track should be evaluated:
*
* 1. bNeedsEvaluation - These channels need update each tick
* 2. bNeedsInitialization - These channels need update only in the first tick!
* 3. Other tracks - These do not need update ever.
*
* Profiled, issues were apparent in 4.26 with a great number of sequencer channels (attributes).
*/
struct FDMXCachedFunctionChannelInfo
{
	FDMXCachedFunctionChannelInfo() = delete;

	FDMXCachedFunctionChannelInfo(const TArray<FDMXFixturePatchChannel>& FixturePatchChannels, int32 InPatchChannelIndex, int32 InFunctionChannelIndex);

	/** Returns the function channel, or nullptr if it got moved or deleted */
	const FDMXFixtureFunctionChannel* TryGetFunctionChannel(const TArray<FDMXFixturePatchChannel>& FixturePatchChannels) const;

	bool NeedsInitialization() const { return bNeedsInitialization; }

	bool NeedsEvaluation() const { return bNeedsEvaluation; }

	int32 GetUniverseID() const { return UniverseID; }

	int32 GetStartingChannel() const { return StartingChannel; }

	EDMXFixtureSignalFormat GetSignalFormat() const { return SignalFormat; }

	bool ShouldUseLSBMode() const { return bLSBMode; }

private:
	/**
	 * Gets the cell channels, but unlike subsystem's methods doesn't sort channels by pixel mapping distribution.
	 * If the cell coordinate would be sorted here, it would be sorted on receive again causing doubly sorting.
	 */
	void GetMatrixCellChannelsAbsoluteNoSorting(UDMXEntityFixturePatch* FixturePatch, const FIntPoint& CellCoordinate, TMap<FName, int32>& OutAttributeToAbsoluteChannelMap) const;

	bool bNeedsInitialization;
	bool bNeedsEvaluation;

	int32 PatchChannelIndex;
	
	int32 FunctionChannelIndex;

	int32 NumFunctionChannels;

	FName AttributeName;

	int32 UniverseID;

	int32 StartingChannel;

	EDMXFixtureSignalFormat SignalFormat;

	bool bLSBMode;
};

/** A DMX Fixture Patch section */
UCLASS()
class DMXRUNTIME_API UMovieSceneDMXLibrarySection
	: public UMovieSceneSection
{
public:

	GENERATED_BODY()

	UMovieSceneDMXLibrarySection();

public:
	// Begin UObject interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
	virtual void PostEditImport() override;
#if WITH_EDITOR
	void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	// ~End UObject interface

	/** Refreshes the channels. Useful e.g. when underlying DMX Library changes */
	void RefreshChannels();

	/** 
	 * If true, all values are interpreted as normalized values (0.0 to 1.0) 
	 * and are mapped to the actual value range of a patch automatically. 
	 * 
	 * If false, values are interpreted as absolute values, depending on the data type of a patch:
	 * 0-255 for 8bit, 0-65'536 for 16bit, 0-16'777'215 for 24bit. 32bit is not fully supported in this mode.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Movie Scene")
	bool bUseNormalizedValues;

	/** Adds a single patch to the section */
	UFUNCTION(BlueprintCallable, Category = "Movie Scene")
	void AddFixturePatch(UDMXEntityFixturePatch* InPatch);

	/** Adds all patches to the secion */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	void AddFixturePatches(const TArray<FDMXEntityFixturePatchRef>& InFixturePatchRefs);

	/** Remove all Functions of a Fixture Patch */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	void RemoveFixturePatch(UDMXEntityFixturePatch* InPatch);

	/** Remove all Functions of a Fixture Patch, searching it by name */
	void RemoveFixturePatch(const FName& InPatchName);
	
	/** Check if this Section animates a Fixture Patch's Functions */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	bool ContainsFixturePatch(UDMXEntityFixturePatch* InPatch) const;

	/** Sets the active mode for a Fixture Patch */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	void SetFixturePatchActiveMode(UDMXEntityFixturePatch* InPatch, int32 InActiveMode);

	/**
	 * Toggle the visibility and evaluation of a Fixture Patch's Function.
	 * When invisible, the Function won't send its data to DMX Protocol.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	void ToggleFixturePatchChannel(UDMXEntityFixturePatch* InPatch, int32 InChannelIndex);

	/**
	 * Toggle the visibility and evaluation of a Fixture Patch's Function searching
	 * both the Patch and Function by name.
	 * When invisible, the Function won't send its data to DMX Protocol.
	 */
	void ToggleFixturePatchChannel(const FName& InPatchName, const FName& InChannelName);

	/** Returns whether a Fixture Patch's Function curve channel is currently enabled */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	bool GetFixturePatchChannelEnabled(UDMXEntityFixturePatch* InPatch, int32 InChannelIndex) const;

	/** Get a list of the Fixture Patches being animated by this Section */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	TArray<UDMXEntityFixturePatch*> GetFixturePatches() const;

	/** Returns the channel for specified patch or nullptr if the patch is not in use */
	FDMXFixturePatchChannel* GetPatchChannel(UDMXEntityFixturePatch* Patch);

	/** Get the list of animated Fixture Patches and their curve channels */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	int32 GetNumPatches() const { return FixturePatchChannels.Num(); }

	const TArray<FDMXFixturePatchChannel>& GetFixturePatchChannels() const { return FixturePatchChannels; }

	TArray<FDMXFixturePatchChannel>& GetMutableFixturePatchChannels() { return FixturePatchChannels; }

	/**
	 * Used only by the Take Recorder to prevent Track evaluation from sending
	 * DMX data while recording it.
	 */
	void SetIsRecording(bool bNewState) { bIsRecording = bNewState; }
	
	/** 
	 * Checked in evaluation to prevent sending DMX data while recording it with
	 * the Take Recorder.
	 */
	bool GetIsRecording() const { return bIsRecording; }

	/** Precaches data for playback */
	void RebuildPlaybackCache() const;

	/** Evaluates DMX at given fame time and sends it */
	void EvaluateAndSendDMX(const FFrameTime& InFrameTime) const;

protected:
	/** Sends DMX for channels that need initialization (one time evaluation) only */
	void SendDMXForChannelsToInitialize() const;

	/** Update the displayed Patches and Function channels in the section */
	void UpdateChannelProxy(bool bResetDefaultChannelValues = false);

	/** The Fixture Patches being controlled by this section and their respective chosen mode */
	UPROPERTY()
	TArray<FDMXFixturePatchChannel> FixturePatchChannels;

	/**
	 * When recording DMX data into this track, this is set to true to prevent
	 * track evaluation from sending data to DMX simultaneously.
	 */
	bool bIsRecording;

	//////////////////////
	// Cache
private:	
	/** True if the cache was updated */
	mutable bool bNeedsSendChannelsToInitialize;

	/** Cached channel info for functions that need initialization (one time evaluation) only */
	mutable TArray<FDMXCachedFunctionChannelInfo> CachedChannelsToInitialize;

	/** Cached channel info for functions that need continous evaluation */
	mutable TArray<FDMXCachedFunctionChannelInfo> CachedChannelsToEvaluate;

	/** The output ports which should be used during playback */
	mutable TSet<FDMXOutputPortSharedRef> CachedOutputPorts;
};
