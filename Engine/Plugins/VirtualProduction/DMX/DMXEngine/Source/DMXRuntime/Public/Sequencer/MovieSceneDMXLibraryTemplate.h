// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolCommon.h"
#include "Sequencer/MovieSceneDMXLibrarySection.h"

#include "CoreMinimal.h"
#include "Evaluation/MovieSceneEvalTemplate.h"

#include "MovieSceneDMXLibraryTemplate.generated.h"

enum class EDMXFixtureSignalFormat : uint8;
class UDMXLibrary;



/** Template that performs evaluation of Fixture Patch sections */
USTRUCT()
struct FMovieSceneDMXLibraryTemplate
	: public FMovieSceneEvalTemplate
{
	GENERATED_BODY()

	FMovieSceneDMXLibraryTemplate() : Section(nullptr) {}
	FMovieSceneDMXLibraryTemplate(const UMovieSceneDMXLibrarySection& InSection);

private:
	virtual void SetupOverrides() override
	{
		EnableOverrides(RequiresSetupFlag);
	}

	virtual UScriptStruct& GetScriptStructImpl() const override { return *StaticStruct(); }
	virtual void Setup(FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) const override;
	virtual void Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const override;

private:
	UPROPERTY()
	TObjectPtr<const UMovieSceneDMXLibrarySection> Section;
};
