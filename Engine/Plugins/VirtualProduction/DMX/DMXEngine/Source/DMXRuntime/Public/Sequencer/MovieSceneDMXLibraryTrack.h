// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Compilation/IMovieSceneTrackTemplateProducer.h"
#include "MovieSceneNameableTrack.h"
#include "MovieSceneDMXLibraryTrack.generated.h"

class UDMXLibrary;

/**
 * Handles manipulation of DMX Libraries in a movie scene.
 */
UCLASS()
class DMXRUNTIME_API UMovieSceneDMXLibraryTrack
	: public UMovieSceneNameableTrack, public IMovieSceneTrackTemplateProducer
{
	GENERATED_BODY()

public:
	UDMXLibrary* GetDMXLibrary() const { return Library; }
	void SetDMXLibrary(UDMXLibrary* InLibrary);

public:
	UMovieSceneDMXLibraryTrack();

	// ~Begin UMovieSceneTrack Interface
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual void AddSection(UMovieSceneSection& Section) override;
	virtual void RemoveSection(UMovieSceneSection& Section) override;
	virtual void RemoveSectionAt(int32 SectionIndex) override;
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	virtual bool IsEmpty() const override;
	virtual bool HasSection(const UMovieSceneSection& Section) const override;
	virtual void RemoveAllAnimationData() override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual bool SupportsMultipleRows() const override;
	virtual FMovieSceneEvalTemplatePtr CreateTemplateForSection(const UMovieSceneSection& InSection) const override;
	// ~End UMovieSceneTrack Interface

private:
	/** The sections owned by this track .*/
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;

	/** The DMX Library to manipulate */
	UPROPERTY()
	TObjectPtr<UDMXLibrary> Library;
};
